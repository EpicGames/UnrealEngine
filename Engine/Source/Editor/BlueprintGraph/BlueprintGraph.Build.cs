// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class BlueprintGraph : ModuleRules
{
	public BlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		OverridePackageType = PackageOverrideType.EngineDeveloper;
		 
		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core", 
				"CoreUObject", 
				"Engine",
                "InputCore",
				"Slate",
				"DeveloperSettings"
			}
		);

		PrivateDependencyModuleNames.AddRange( 
			new string[] {
				
                "KismetCompiler",
				"EditorFramework",
				"UnrealEd",
                "GraphEditor",
				"SlateCore",
                "Kismet",
                "KismetWidgets",
                "PropertyEditor",
				"ToolMenus",
				"AssetTools",
				"EditorSubsystem",
				"UMG",
			}
		);

		CircularlyReferencedDependentModules.AddRange(
            new string[] {
                "KismetCompiler",
                "UnrealEd",
                "GraphEditor",
                "Kismet",
            }
		); 
	}
}
