// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FAssetData;
class UEdGraph;
enum class EK2NewNodeFlags;

/** Struct to handle Asset Blueprint Graph actions
 * Essentially this is used to help generate asset nodes/responses for Blueprint graphs and keeping Blueprint code from being dependent on something it shouldn't
 * For example, this can be used to help with dragging/dropping assets on a Blueprint graph
 */
struct FAssetBlueprintGraphActions
{
	virtual ~FAssetBlueprintGraphActions() = default;

	/**
	 * Gets the hover message when when holding a asset over a Blueprint graph from a drag/drop operation
	 * @param AssetData The asset being dragged/dropped
	 * @param HoverGraph The Blueprint graph to drop the asset on
	 * @return Message to display when hovering.
	 */
	virtual FText GetGraphHoverMessage(const FAssetData& AssetData, const UEdGraph* HoverGraph) const = 0;

	/**
	 * Tries to create a node from the asset passed on the graph passed in
	 * @param AssetData The asset being dragged/dropped
	 * @param ParentGraph The Blueprint graph to drop the asset on
	 * @param Location Location to spawn the new asset node at
	 * @param Options Flags for node creation
	 * @return Bool of whether a node was successfully created
	 */
	virtual bool TryCreatingAssetNode(const FAssetData& AssetData, UEdGraph* ParentGraph, const FVector2D Location, EK2NewNodeFlags Options) const = 0;
};
