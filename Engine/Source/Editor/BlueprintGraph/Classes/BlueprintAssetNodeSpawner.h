// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintAssetNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraph;
class UEdGraphNode;
class UObject;

/**
 * Takes care of spawning various asset related nodes (nodes associated with 
 * an asset).
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintAssetNodeSpawner : public UBlueprintNodeSpawner
{
	GENERATED_BODY()

public:
	/**
	 * Creates a new UBlueprintAssetNodeSpawner for the supplied asset.
	 * Does not do any compatibility checking to ensure that the asset is
	 * viable for blueprint use.
	 * 
	 * @param  InNodeClass			The type of node you want the spawner to create.
	 * @param  InAssetData			The asset you want assigned to new nodes.
	 * @param  InOuter				Optional outer for the new spawner (if left null, the transient package will be used).
	 * @param  InPostSpawnDelegate	A delegate to perform specialized node setup post-spawn.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintAssetNodeSpawner* Create(TSubclassOf<UEdGraphNode> const InNodeClass, const FAssetData& InAssetData, UObject* InOuter = nullptr, FCustomizeNodeDelegate InPostSpawnDelegate = FCustomizeNodeDelegate());

	/**
	 * Retrieves the asset that this assigns to spawned nodes.
	 *
	 * @return The asset that this class was initialized with.
	 */
	UE_API const FAssetData& GetAssetData() const;

private:
	/** The asset to configure new nodes with. */
	UPROPERTY()
	FAssetData AssetData;
};

#undef UE_API
