// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEventNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "CoreMinimal.h"
#include "K2Node_Event.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/FieldPath.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "BlueprintBoundEventNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FMulticastDelegateProperty;
class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UK2Node_Event;
class UObject;

/**
 * Takes care of spawning UK2Node_Event nodes. Acts as the "action" portion of
 * certain FBlueprintActionMenuItems. Will not spawn a new event node if one
 * associated with the specified function already exits (instead, Invoke() will
 * return the existing one). Evolved from FEdGraphSchemaAction_K2AddEvent and 
 * FEdGraphSchemaAction_K2ViewNode.
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintBoundEventNodeSpawner : public UBlueprintEventNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintEventNodeSpawner for delegate bound events. Does
	 * not come bound, instead it is left up to the ui to bind 
	 *
	 * @param  NodeClass		The event node type that you want this to spawn.
	 * @param  EventDelegate	The delegate that you want to trigger the spawned event.
	 * @param  Outer			Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintBoundEventNodeSpawner* Create(TSubclassOf<UK2Node_Event> NodeClass, FMulticastDelegateProperty* EventDelegate, UObject* Outer = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

	// UBlueprintEventNodeSpawner interface
	UE_API virtual UK2Node_Event const* FindPreExistingEvent(UBlueprint* Blueprint, FBindingSet const& Bindings) const override;
	// End UBlueprintEventNodeSpawner interface

	// IBlueprintNodeBinder interface
	UE_API virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const override;
	UE_API virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const override;
	// End IBlueprintNodeBinder interface

	/** @return  */
	UE_API FMulticastDelegateProperty const* GetEventDelegate() const;

private:
	/** */
	UPROPERTY()
	TFieldPath<FMulticastDelegateProperty> EventDelegate;
};

#undef UE_API
