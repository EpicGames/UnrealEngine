// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintBoundNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UObject;

/**
 * Takes care of spawning various bound nodes. Acts as the 
 * "action" portion of certain FBlueprintActionMenuItems. 
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintBoundNodeSpawner : public UBlueprintNodeSpawner
{
	GENERATED_UCLASS_BODY()
	DECLARE_DELEGATE_RetVal_OneParam(bool, FCanBindObjectDelegate, UObject const*);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnBindObjectDelegate, UEdGraphNode*, UObject*);
	DECLARE_DELEGATE_RetVal_TwoParams(UEdGraphNode*, FFindPreExistingNodeDelegate, const UBlueprint*, IBlueprintNodeBinder::FBindingSet const& );

public:
	/**
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintBoundNodeSpawner* Create(TSubclassOf<UEdGraphNode> NodeClass, UObject* Outer = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

	// FBlueprintNodeBinder interface
	UE_API virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const override;
	UE_API virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const override;
	// End FBlueprintNodeBinder interface

public:
	/**
	 * A delegate to perform specialized node binding verification
	 */
	FCanBindObjectDelegate CanBindObjectDelegate;

	/**
	 * A delegate to perform specialized node setup during binding
	 */
	FOnBindObjectDelegate OnBindObjectDelegate;

	/**
	 * A delegate to find a node that is already spawned, instead of spawning a node
	 * we will focus on the pre-existing node found by the delegate
	 */
	FFindPreExistingNodeDelegate FindPreExistingNodeDelegate;
};

#undef UE_API
