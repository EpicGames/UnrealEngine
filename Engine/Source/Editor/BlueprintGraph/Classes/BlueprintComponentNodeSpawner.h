// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "Components/ActorComponent.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintComponentNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UActorComponent;
class UEdGraph;
class UEdGraphNode;
class UObject;

/**
 * Takes care of spawning UK2Node_AddComponent nodes. Acts as the "action" 
 * portion of certain FBlueprintActionMenuItems. Evolved from 
 * FEdGraphSchemaAction_K2AddComponent.
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintComponentNodeSpawner : public UBlueprintNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintComponentNodeSpawner for the specified class. 
	 * Does not do any compatibility checking to ensure that the class is 
	 * viable as a spawnable component (do that before calling this).
	 *
	 * @param  ComponentClass	The component type you want spawned nodes to spawn.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintComponentNodeSpawner* Create(const struct FComponentTypeEntry& Entry);

	// UBlueprintNodeSpawner interface
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual FBlueprintActionUiSpec GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	UE_API virtual bool IsTemplateNodeFilteredOut(const FBlueprintActionFilter& Filter) const override;
	// End UBlueprintNodeSpawner interface
	
	// IBlueprintNodeBinder interface
	UE_API virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const override;
	UE_API virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const override;
	// End IBlueprintNodeBinder interface

	/**
	 * Retrieves the component class that this configures spawned nodes with.
	 *
	 * @return The component type that this class was initialized with.
	 */
	UE_API TSubclassOf<UActorComponent> GetComponentClass() const;

private:
	/** The component class to configure new nodes with. */
	UPROPERTY()
	TSubclassOf<UActorComponent> ComponentClass;

	/** The name of the component class to configure new nodes with. */
	UPROPERTY()
	FString ComponentName;
	
	/** The name of the asset name that needs to be loaded */
	UPROPERTY()
	FString ComponentAssetName;
};

#undef UE_API
