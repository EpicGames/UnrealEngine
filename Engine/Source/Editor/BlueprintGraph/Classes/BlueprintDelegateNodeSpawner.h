// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintFieldNodeSpawner.h"
#include "CoreMinimal.h"
#include "K2Node_BaseMCDelegate.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "BlueprintDelegateNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FMulticastDelegateProperty;
class UK2Node_BaseMCDelegate;
class UObject;

/**
 * Takes care of spawning various nodes associated with delegates. Serves as the 
 * "action" portion for certain FBlueprintActionMenuItems. Evolved from 
 * FEdGraphSchemaAction_K2Delegate, FEdGraphSchemaAction_K2AssignDelegate, etc.
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintDelegateNodeSpawner : public UBlueprintFieldNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintDelegateNodeSpawner for the specified property.
	 * Does not do any compatibility checking to ensure that the property is
	 * accessible from blueprints (do that before calling this).
	 *
	 * @param  NodeClass	The node type that you want the spawner to spawn.
	 * @param  Property		The property you want assigned to spawned nodes.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintDelegateNodeSpawner* Create(TSubclassOf<UK2Node_BaseMCDelegate> NodeClass, FMulticastDelegateProperty const* const Property, UObject* Outer = nullptr);

	/**
	 * Accessor to the delegate property that this spawner wraps (the delegate
	 * that this will assign spawned nodes with).
	 *
	 * @return The delegate property that this was initialized with.
	 */
	UE_API FMulticastDelegateProperty const* GetDelegateProperty() const;
};

#undef UE_API
