// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "K2Node_Event.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintEventNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UFunction;
class UK2Node_Event;
class UObject;

/**
 * Takes care of spawning UK2Node_Event nodes. Acts as the "action" portion of
 * certain FBlueprintActionMenuItems. Will not spawn a new event node if one
 * associated with the specified function already exits (instead, Invoke() will
 * return the existing one). Evolved from FEdGraphSchemaAction_K2AddEvent and 
 * FEdGraphSchemaAction_K2ViewNode.
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintEventNodeSpawner : public UBlueprintNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintEventNodeSpawner for the specified function.
	 * Does not do any compatibility checking to ensure that the function is
	 * viable as a blueprint event (do that before calling this).
	 *
	 * @param  EventFunc	The function you want assigned to new nodes.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintEventNodeSpawner* Create(UFunction const* const EventFunc, UObject* Outer = nullptr);

	/**
	 * Creates a new UBlueprintEventNodeSpawner for custom events. The
	 * CustomEventName can be left blank if the node will pick one itself on
	 * instantiation.
	 *
	 * @param  NodeClass		The event node type that you want this to spawn.
	 * @param  CustomEventName	The name you want assigned to the event.
	 * @param  Outer			Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintEventNodeSpawner* Create(TSubclassOf<UK2Node_Event> NodeClass, FName CustomEventName, UObject* Outer = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API bool IsForCustomEvent() const;
	
	/**
	 * Retrieves the function that this assigns to spawned nodes (defines the
	 * event's signature). Can be null if this spawner was for a custom event.
	 *
	 * @return The event function that this class was initialized with.
	 */
	UE_API UFunction const* GetEventFunction() const;

	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API virtual UK2Node_Event const* FindPreExistingEvent(UBlueprint* Blueprint, FBindingSet const& Bindings) const;

protected:
	/** The function to configure new nodes with. */
	UPROPERTY()
    TObjectPtr<UFunction const> EventFunc;

	/** The custom name to configure new event nodes with. */
	UPROPERTY()
	FName CustomEventName;	
};

#undef UE_API
