// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "K2Node.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/FieldPath.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintFieldNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FProperty;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UField;
class UK2Node;
class UObject;

/**
 * Takes care of spawning various field related nodes (nodes associated with 
 * functions, enums, structs, properties, etc.). Acts as the "action" portion
 * for certain FBlueprintActionMenuItems. 
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintFieldNodeSpawner : public UBlueprintNodeSpawner
{
	GENERATED_UCLASS_BODY()
	DECLARE_DELEGATE_TwoParams(FSetNodeFieldDelegate, UEdGraphNode*, FFieldVariant);

public:
	/**
	 * Creates a new UBlueprintFieldNodeSpawner for the supplied field.
	 * Does not do any compatibility checking to ensure that the field is
	 * viable for blueprint use.
	 * 
	 * @param  NodeClass	The type of node you want the spawner to create.
	 * @param  Field		The field you want assigned to new nodes.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @param  OwnerClass	The class that the variable is a member of or the class it is associated with if it is in a sidecar data structure.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintFieldNodeSpawner* Create(TSubclassOf<UK2Node> NodeClass, FFieldVariant Field, UObject* Outer = nullptr, UClass const* OwnerClass = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

	/** Callback to define how the field should be applied to new nodes */
	FSetNodeFieldDelegate SetNodeFieldDelegate;

	/**
	 * Retrieves the field that this assigns to spawned nodes (defines the 
	 * node's signature).
	 *
	 * @return The field that this class was initialized with.
	 */
	UE_API FFieldVariant GetField() const;

protected:

	/**
	 * Sets the field for this spawner 
	 */
	UE_API void SetField(FFieldVariant InField);

	/** The owning class to configure new nodes with. */
	UPROPERTY()
	TObjectPtr<UClass const> OwnerClass;

private:
	/** The field to configure new nodes with. */
	UPROPERTY()
	TObjectPtr<UField> Field;

	UPROPERTY()
	TFieldPath<FProperty> Property;
};

#undef UE_API
