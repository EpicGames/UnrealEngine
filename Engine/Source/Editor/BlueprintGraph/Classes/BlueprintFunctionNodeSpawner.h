// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintFieldNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintFunctionNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraph;
class UEdGraphNode;
class UFunction;
class UK2Node_CallFunction;
class UObject;
struct FBlueprintActionContext;

/**
 * Takes care of spawning various UK2Node_CallFunction nodes. Acts as the 
 * "action" portion of certain FBlueprintActionMenuItems. 
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintFunctionNodeSpawner : public UBlueprintFieldNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintFunctionNodeSpawner for the specified function. 
	 * Does not do any compatibility checking to ensure that the function is 
	 * viable as a blueprint function call (do that before calling this).
	 *
	 * @param  Function		The function you want assigned to new nodes.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintFunctionNodeSpawner* Create(UFunction const* const Function, UObject* Outer = nullptr);

	/**
	 * Creates a new UBlueprintFunctionNodeSpawner for the specified function.
	 * Does not do any compatibility checking to ensure that the function is
	 * viable as a blueprint function call (do that before calling this).
	 * 
	 * @param  NodeClass	The type of node you want the spawner to create.
	 * @param  Function		The function you want assigned to new nodes.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintFunctionNodeSpawner* Create(TSubclassOf<UK2Node_CallFunction> NodeClass, UFunction const* const Function, UObject* Outer = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual void Prime() override;
	UE_API virtual FBlueprintActionUiSpec GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface

	// IBlueprintNodeBinder interface
	UE_API virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const override;
	UE_API virtual bool CanBindMultipleObjects() const override;
	UE_API virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const override;
	// End IBlueprintNodeBinder interface

	/**
	 * Retrieves the function that this assigns to spawned nodes (defines the 
	 * node's signature).
	 *
	 * @return The function that this class was initialized with.
	 */
	UE_API UFunction const* GetFunction() const;
};

#undef UE_API
