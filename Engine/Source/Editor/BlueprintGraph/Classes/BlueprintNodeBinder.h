// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "Templates/Requires.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakFieldPtr.h"

#include <type_traits>

class UEdGraphNode;

class FBindingObject
{
	TObjectPtr<UObject> Object;
	FField* Field;
	bool bIsUObject;
public:
	FBindingObject()
		: bIsUObject(false)
	{}
	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<T, UObject*>)
	>
	FBindingObject(T InObject)
		: Object(InObject)
		, bIsUObject(true)
	{}
	FBindingObject(FField* InField)
		: Field(InField)
		, bIsUObject(false)
	{}
	FBindingObject(const FFieldVariant& InFieldOrObject)
	{
		bIsUObject = InFieldOrObject.IsUObject();
		if (bIsUObject)
		{
			Object = InFieldOrObject.ToUObject();
		}
		else
		{
			Field = InFieldOrObject.ToField();
		}
	}
	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<T, UObject*>)
	>
	FBindingObject& operator=(T InObject)
	{
		Object = ImplicitConv<UObject*>(InObject);
		Field = nullptr;
		bIsUObject = true;
		return *this;
	}
	FBindingObject& operator=(FField* InField)
	{
		Object = nullptr;
		Field = InField;
		bIsUObject = false;
		return *this;
	}
	FBindingObject& operator=(TYPE_OF_NULLPTR)
	{
		Object = nullptr;
		Field = nullptr;
		return *this;
	}
	bool IsUObject() const
	{
		return bIsUObject;
	}
	bool IsValid() const
	{
		return bIsUObject ? Object != nullptr : Field != nullptr;
	}
	FName GetFName() const
	{
		return bIsUObject ? Object->GetFName() : Field->GetFName();
	}
	FString GetName() const
	{
		return bIsUObject ? Object->GetName() : Field->GetName();
	}
	FString GetPathName() const
	{
		return bIsUObject ? Object->GetPathName() : Field->GetPathName();
	}
	FString GetFullName() const
	{
		return bIsUObject ? Object->GetFullName() : Field->GetFullName();
	}
	bool IsA(const UClass* InClass) const
	{
		return bIsUObject && Object != nullptr && Object->IsA(InClass);
	}
	bool IsA(const FFieldClass* InClass) const
	{
		return !bIsUObject && Field != nullptr && Field->IsA(InClass);
	}
	template <typename T>
	bool IsA() const
	{
		if constexpr (std::is_base_of_v<UObject, T>)
		{
			if (bIsUObject && Object)
			{
				return Object->IsA(T::StaticClass());
			}
		}
		else
		{
			if (!bIsUObject && Field != nullptr)
			{
				return Field->IsA(T::StaticClass());
			}
		}
		return false;
	}
	template <typename T>
	T* Get() const
	{
		if constexpr (std::is_base_of_v<UObject, T>)
		{
			if (bIsUObject && Object)
			{
				return Cast<T>(Object);
			}
		}
		else
		{
			if (!bIsUObject && Field != nullptr)
			{
				return CastField<T>(Field);
			}
		}
		return nullptr;
	}

	friend uint32 GetTypeHash(const FBindingObject& BindingObject)
	{
		return BindingObject.bIsUObject ? GetTypeHash(BindingObject.Object) : GetTypeHash(BindingObject.Field);
	}

	bool operator==(const FBindingObject &Other) const
	{
		return bIsUObject == Other.bIsUObject && Object == Other.Object && Field == Other.Field;
	}
	bool operator!=(const FBindingObject &Other) const
	{
		return bIsUObject != Other.bIsUObject || Object != Other.Object || Field != Other.Field;
	}

	friend bool operator==(const FBindingObject &Lhs, const UObject* Rhs)
	{
		return Lhs.IsUObject() && Lhs.Object == Rhs;
	}
	friend bool operator!=(const FBindingObject &Lhs, const UObject* Rhs)
	{
		return !Lhs.IsUObject() || Lhs.Object != Rhs;
	}
	friend bool operator==(const UObject* Lhs, const FBindingObject &Rhs)
	{
		return Rhs.IsUObject() && Rhs.Object == Lhs;
	}
	friend bool operator!=(const UObject* Lhs, const FBindingObject &Rhs)
	{
		return !Rhs.IsUObject() || Rhs.Object != Lhs;
	}

	friend bool operator==(const FBindingObject &Lhs, const FField* Rhs)
	{
		return !Lhs.IsUObject() && Lhs.Field == Rhs;
	}
	friend bool operator!=(const FBindingObject &Lhs, const FField* Rhs)
	{
		return Lhs.IsUObject() || Lhs.Field != Rhs;
	}
	friend bool operator==(const FField* Lhs, const FBindingObject &Rhs)
	{
		return !Rhs.IsUObject() && Rhs.Field == Lhs;
	}
	friend bool operator!=(const FField* Lhs, const FBindingObject &Rhs)
	{
		return Rhs.IsUObject() || Rhs.Field != Lhs;
	}

	friend bool operator==(const FBindingObject &Lhs, TYPE_OF_NULLPTR)
	{
		return Lhs.IsUObject() ? Lhs.Object == nullptr : Lhs.Field == nullptr;
	}
	friend bool operator!=(const FBindingObject &Lhs, TYPE_OF_NULLPTR)
	{
		return Lhs.IsUObject() ? Lhs.Object != nullptr : Lhs.Field != nullptr;
	}
	friend bool operator==(TYPE_OF_NULLPTR, const FBindingObject &Rhs)
	{
		return Rhs.IsUObject() ? Rhs.Object == nullptr : Rhs.Field == nullptr;
	}
	friend bool operator!=(TYPE_OF_NULLPTR, const FBindingObject &Rhs)
	{
		return Rhs.IsUObject() ? Rhs.Object != nullptr : Rhs.Field != nullptr;
	}

	void AddStructReferencedObjects(FReferenceCollector& Collector);
};

template<>
struct TStructOpsTypeTraits<FBindingObject> : public TStructOpsTypeTraitsBase2<FBindingObject>
{
	enum
	{
		WithAddStructReferencedObjects = true,
	};
};

class IBlueprintNodeBinder
{
public:
	/** */
	typedef TSet< FBindingObject > FBindingSet;

public:
	/**
	 * Checks to see if the specified object can be bound by this.
	 * 
	 * @param  BindingCandidate	The object you want to check for.
	 * @return True if BindingCandidate can be bound by this controller, false if not.
	 */
	virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const = 0;

	/**
	 * Determines if this will accept more than one binding (used to block multiple 
	 * bindings from being applied to nodes that can only have one).
	 * 
	 * @return True if this will accept multiple bindings, otherwise false.
	 */
	virtual bool CanBindMultipleObjects() const = 0;

	/**
	 * Attempts to bind all bindings to the supplied node.
	 * 
	 * @param  Node	 The node you want bound to.
	 * @return True if all bindings were successfully applied, false if any failed.
	 */
	bool ApplyBindings(UEdGraphNode* Node, FBindingSet const& Bindings) const
	{
		uint32 BindingCount = 0;
		for (const FBindingObject& Binding : Bindings)
		{
			if (Binding.IsValid() && BindToNode(Node, Binding))
			{
				++BindingCount;
				if (!CanBindMultipleObjects())
				{
					break;
				}
			}
		}
		return (BindingCount == Bindings.Num());
	}

protected:
	/**
	 * Attempts to apply the specified binding to the supplied node.
	 * 
	 * @param  Node		The node you want bound.
	 * @param  Binding	The binding you want applied to Node.
	 * @return True if the binding was successful, false if not.
	 */
	virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const = 0;
};
