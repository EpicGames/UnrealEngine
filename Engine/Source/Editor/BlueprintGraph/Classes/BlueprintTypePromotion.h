// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionDatabase.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Modules/ModuleManager.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#define UE_API BLUEPRINTGRAPH_API

class FText;
class UBlueprintFunctionNodeSpawner;
class UFunction;

/** 
* Contains behavior needed to handle type promotion in blueprints. 
* Creates a map of "Operations" to any of their matching UFunctions
* so that we can find the best possible match given several pin types.  
*/
class FTypePromotion : private FNoncopyable
{
public:

	/** Creates a new singleton instance of TypePromotion if there isn't one and returns a reference to it */
	static UE_API FTypePromotion& Get();

	/** Deletes the singleton instance of type promotion if there is one */
	static UE_API void Shutdown();

	/**
	* Find the function that is the best match given the pins to consider. 
	* Ex: Given "Add" operator and an array of two Vector pins, it will return "Add_VectorVector"
	*/
	static UE_API UFunction* FindBestMatchingFunc(FName Operation, const TArray<UEdGraphPin*>& PinsToConsider);

	/** Returns all functions for a specific operation. Will empty the given array and populate it with UFunction pointers */
	static UE_API void GetAllFuncsForOp(FName Operation, TArray<UFunction*>& OutFuncs);

	/** Get a set of the supported operator names for type promo. Ex: "Add", "Subtract", "Multiply" */
	static UE_API const TSet<FName>& GetAllOpNames();
	
	/** Set of comparison operator names (GreaterThan, LessThan, etc) */
	static UE_API const TSet<FName>& GetComparisonOpNames();

	/** Get the keywords metadata for the given operator name */
	static UE_API const FText& GetKeywordsForOperator(const FName Operator);

	/** Get the user facing version of this operator name */
	static UE_API const FText& GetUserFacingOperatorName(const FName Operator);

	/** Returns true if the given function is a comparison operator */
	static UE_API bool IsComparisonFunc(UFunction const* const Func);

	/** Returns true if the given op name is a comparison operator name */
	static UE_API bool IsComparisonOpName(const FName OpName);

	/**
	* Parse the name of the operator that this function matches to (Add, Subtract, etc)
	*
	* @param Func		The function to parse the operator name from
	*
	* @return FName		Name of the operator for this function, NO_OP if does not exist.
	*/
	static UE_API FName GetOpNameFromFunction(UFunction const* const Func);

	/**
	* Determine what type a given set of wildcard pins would result in
	*
	* @return	Pin type that is the "highest" of all the given pins
	*/
	static UE_API FEdGraphPinType GetPromotedType(const TArray<UEdGraphPin*>& WildcardPins);

	/** Returns true if the given function has been registered within the operator table */
	static UE_API bool IsFunctionPromotionReady(const UFunction* const FuncToConsider);

	/** Represents the possible results when comparing two types for promotion */
	enum class ETypeComparisonResult : uint8
	{
		TypeAHigher,
		TypeBHigher,
		TypesEqual,
		InvalidComparison
	};

	/**
	* Given the two pin types check which pin type is higher. 
	* Given two structs it will return equal, this does NOT compare PinDefaultSubobjects
	*/
	static UE_API ETypeComparisonResult GetHigherType(const FEdGraphPinType& A, const FEdGraphPinType& B);

	/** Returns true if A can be promoted to type B correctly, or if the types are equal */
	static UE_API bool IsValidPromotion(const FEdGraphPinType& A, const FEdGraphPinType& B);

	/** Returns true if the given input pin can correctly be converted to the output type as a struct */
	static UE_API bool HasStructConversion(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin);

	/** Returns true if the given function has a registered operator node spawner */
	static UE_API bool IsOperatorSpawnerRegistered(UFunction const* const Func);

	/** Function node spawner associated with this operation */
	static UE_API UBlueprintFunctionNodeSpawner* GetOperatorSpawner(FName OpName);

	/** keep track of the operator that this function provides so that we don't add multiple 
	to the BP context menu */
	static UE_API void RegisterOperatorSpawner(FName OpName, UBlueprintFunctionNodeSpawner* Spawner);

	/**
	* Clear all registered node spawners for operators from the system. This is necessary
	* during hot reload to ensure that all operator spawners are up to date.
	*/
	static UE_API void ClearNodeSpawners();

	/**
	 * Clear out the promotion table and rebuild it, reassessing each available UFunction.
	 * 
	 * @param Reason	The reason for refreshing the promotion tables, which can indicate
	 *					whether or not it came from a Live Coding recompile.
	 */
	static UE_API void RefreshPromotionTables(EReloadCompleteReason Reason = EReloadCompleteReason::None);

	/**
	* Get the "Primitive Promotion Table" which represents what base Pin types 
	* can be promoted to others. 
	*
	* @return Const pointer to the primitive promotion table
	*/
	static UE_API const TMap<FName, TArray<FName>>* const GetPrimitivePromotionTable();
	
	/**
	* Get a pointer to an array of available promotion types to a given pin type
	*
	* @param Type		The pin type to find available promotions to
	*
	* @return const TArray<FName>*	Pointer to the available promotions, nullptr if there are none.
	*/
	static UE_API const TArray<FName>* GetAvailablePrimitivePromotions(const FEdGraphPinType& Type);

private:

	UE_API explicit FTypePromotion();
	UE_API ~FTypePromotion();

	UE_API bool IsFunctionPromotionReady_Internal(const UFunction* const FuncToConsider) const;

	UE_API FEdGraphPinType GetPromotedType_Internal(const TArray<UEdGraphPin*>& WildcardPins) const;
	
	UE_API UFunction* FindBestMatchingFunc_Internal(FName Operation, const TArray<UEdGraphPin*>& PinsToConsider);

	UE_API void GetAllFuncsForOp_Internal(FName Operation, TArray<UFunction*>& OutFuncs);

	/** Returns true if the given function is a candidate to handle type promotion */
	static UE_API bool IsPromotableFunction(const UFunction* Function);

	/**
	* Determines which pin type is "higher" 
	* 
	* @see FTypePromotion::PromotionTable
	*/
	UE_API ETypeComparisonResult GetHigherType_Internal(const FEdGraphPinType& A, const FEdGraphPinType& B) const;

	/** Creates a lookup table of types and operations to their appropriate UFunction */
	UE_API void CreateOpTable();

	/** Creates the table of what types can be promoted to others */
	UE_API TMap<FName, TArray<FName>> CreatePromotionTable();

	UE_API void AddOpFunction(FName OpName, UFunction* Function);

	static UE_API FTypePromotion* Instance;

	/** A map of 'Type' to its 'available promotions'. See ctor for creation */
	const TMap<FName, TArray<FName>> PromotionTable;

	/**
	 * A single operator can have multiple functions associated with it; usually
	 * for handling different types (int*int, vs. int*vector), hence this array.
	 * This is the same implementation style as the Math Expression node.
	 */
	typedef TArray<UFunction*> FFunctionsList;

	/**
	 * Protects internal data from multi-threaded access.
	 */
	mutable FCriticalSection Lock;

	/**
	 * A lookup table, mapping operator strings (like "Add", "Multiply", etc.) to a list
	 * of associated functions.
	 */
	TMap<FName, FFunctionsList> OperatorTable;

	/** Map of operators to their node spawner so that we can clean up the context menu */
	TMap<FName, UBlueprintFunctionNodeSpawner*> OperatorNodeSpawnerMap;
};

namespace TypePromoDebug
{
	/** 
	* Checks if the type promotion editor pref is true or false
	* (The bEnableTypePromotion BP editor setting)
	*/
	static bool IsTypePromoEnabled()
	{
		return GetDefault<UBlueprintEditorSettings>()->bEnableTypePromotion;
	}
}

#undef UE_API
