// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintFieldNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "BlueprintNodeSpawner.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Templates/SubclassOf.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "BlueprintVariableNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FProperty;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UK2Node_Variable;
class UObject;
struct FBlueprintActionContext;

/**
 * Takes care of spawning variable getter/setter nodes. Serves as the "action"
 * portion for certain FBlueprintActionMenuItems. Evolved from 
 * FEdGraphSchemaAction_K2Var, and can spawn nodes for both member-variables and 
 * local function variables.
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintVariableNodeSpawner : public UBlueprintFieldNodeSpawner
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Creates a new UBlueprintVariableNodeSpawner, charged with spawning 
	 * a member-variable node (for a variable that has an associated FProperty) 
	 * 
	 * @param  NodeClass	The node type that you want the spawner to spawn.
	 * @param  VarProperty	The property that represents the member-variable you want nodes spawned for.
	 * @param  VarContext	The graph that the local variable belongs to.
	 * @param  OwnerClass	The class that the variable is a member of or the class it is associated with if it is in a sidecar data structure.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintVariableNodeSpawner* CreateFromMemberOrParam(TSubclassOf<UK2Node_Variable> NodeClass, FProperty const* VarProperty, UEdGraph* VarContext = nullptr, UClass* OwnerClass = nullptr);

	/**
	 * Creates a new UBlueprintVariableNodeSpawner, charged with spawning
	 * a local-variable node (for a variable that belongs to a specific graph).
	 * 
	 * @param  NodeClass	The node type that you want the spawner to spawn.
	 * @param  VarContext	The graph that the local variable belongs to.
	 * @param  VarDesc		Details the local variable (name, type, etc.)
	 * @param  VarProperty	The property that represents the local-variable you want nodes spawned for.
	 * @param  Outer		Optional outer for the new spawner (if left null, the transient package will be used).
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintVariableNodeSpawner* CreateFromLocal(TSubclassOf<UK2Node_Variable> NodeClass, UEdGraph* VarContext, FBPVariableDescription const& VarDesc, FProperty* VarProperty, UObject* Outer = nullptr);

	// UBlueprintNodeSpawner interface
	UE_API virtual void Prime() override;
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const override;
	UE_API virtual FBlueprintActionUiSpec GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const override;
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const override;
	// End UBlueprintNodeSpawner interface
	
	/**
	 * @return True if this is a user-created local variable
	 */
	UE_API bool IsUserLocalVariable() const;

	/**
	 * Since this spawner can wrap both local and member variables, we use this
	 * method to discern between the two.
	 * 
	 * @return True if this spawner wraps a local variable, false if not.
	 */
	UE_API bool IsLocalVariable() const;

	/**
	 * If this is a local variable, then this will return the UEdGraph that it
	 * belongs to, otherwise it pulls the outer from the wrapped member-variable 
	 * property.
	 * 
	 * @return Null if the variable is no longer valid, otherwise an object pointer to the variable's owner.
	 */
	UE_API FFieldVariant GetVarOuter() const;

	/**
	 * Accessor to the variable's property. Will be null if this is for a local 
	 * variable (as they don't have UProperties associated with them).
	 * 
	 * @return Null if this wraps a local variable (or if the variable property is stale), otherwise the property this was initialized with. 
	 */
	UE_API FProperty const* GetVarProperty() const;

	/**
	 * Utility function for easily accessing the variable's type (needs to pull
	 * the information differently if it is a local variable as opposed to a
	 * member variable with a FProperty).
	 * 
	 * @return A struct detailing the wrapped variable's type.
	 */
	UE_API FEdGraphPinType GetVarType() const;

private:
	/**
	 * Utility function for easily accessing the variable's name (needs to pull
	 * the information differently if it is a local variable as opposed to a
	 * member variable with a FProperty).
	 * 
	 * @return A friendly, user presentable, name for the variable that this wraps. 
	 */
	UE_API FText GetVariableName() const;

	/** The graph that the local variable belongs to (if this is a local variable spawner). */
	UPROPERTY()
	TObjectPtr<UEdGraph> LocalVarOuter;

	/** */
	UPROPERTY()
	FBPVariableDescription LocalVarDesc;
};

#undef UE_API
