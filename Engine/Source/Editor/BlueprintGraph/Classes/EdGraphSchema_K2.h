// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintImpl.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "EdGraphSchema_K2.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class AActor;
class FKismetCompilerContext;
class FMenuBuilder;
class FProperty;
class IAssetReferenceFilter;
class UBlueprint;
class UClass;
class UEnum;
class UK2Node;
class UScriptStruct;
class UToolMenu;
struct FAssetData;
struct FToolMenuSection;
template <typename T> struct TObjectPtr;

/** Reference to an structure (only used in 'docked' palette) */
USTRUCT()
struct FEdGraphSchemaAction_K2Struct : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2Struct"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	TObjectPtr<UStruct> Struct;

	void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		if( Struct )
		{
			Collector.AddReferencedObject(Struct);
		}
	}

	FName GetPathName() const
	{
		return Struct ? FName(*Struct->GetPathName()) : NAME_None;
	}

	FEdGraphSchemaAction_K2Struct() 
		: FEdGraphSchemaAction()
		, Struct(nullptr)
	{}

	FEdGraphSchemaAction_K2Struct(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, Struct(nullptr)
	{}
};

// Constants used for metadata, etc... in blueprints
struct FBlueprintMetadata
{
public:
	// Struct/Enum/Class:
	// If true, this class, struct, or enum is a valid type for use as a variable in a blueprint
	static UE_API const FName MD_AllowableBlueprintVariableType;

	// If true, this class, struct, or enum is not valid for use as a variable in a blueprint
	static UE_API const FName MD_NotAllowableBlueprintVariableType;

	// Class:
	// [ClassMetadata] If present, the component class can be spawned by a blueprint
	static UE_API const FName MD_BlueprintSpawnableComponent;

	/** If true, the class will be usable as a base for blueprints */
	static UE_API const FName MD_IsBlueprintBase;
	
	/** A listing of classes that this class is accessible from (and only those classes, if present).  Note that this determines the GRAPH CONTEXTS in which the node cannot be placed (e.g. right click menu, palette), and does NOT control menus when dragging off of a context pin (i.e. contextual drag) */
	static UE_API const FName MD_RestrictedToClasses;

	/// [ClassMetadata] Used for Actor and Component classes. If the native class cannot tick, Blueprint generated classes based this Actor or Component can have bCanEverTick flag overridden even if bCanBlueprintsTickByDefault is false.
	static UE_API const FName MD_ChildCanTick;

	/// [ClassMetadata] Used for Actor and Component classes. If the native class cannot tick, Blueprint generated classes based this Actor or Component can never tick even if bCanBlueprintsTickByDefault is true.
	static UE_API const FName MD_ChildCannotTick;

	/// [ClassMetadata] Used to make the first subclass of a class ignore all inherited showCategories and hideCategories commands
	static UE_API const FName MD_IgnoreCategoryKeywordsInSubclasses;

	/** Specifies which struct implements the custom thunk functions for this class */
	static UE_API const FName MD_CustomThunkTemplates;

	//    function metadata
	/** Specifies a UFUNCTION as Kismet protected, which can only be called from itself */
	static UE_API const FName MD_Protected;

	/** Marks a UFUNCTION as latent execution */
	static UE_API const FName MD_Latent;

	/** Indicates that the UFUNCTION implements its own thunk function */
	static UE_API const FName MD_CustomThunk;

	/** Marks a UFUNCTION as accepting variadic arguments */
	static UE_API const FName MD_Variadic;

	/** Marks a UFUNCTION as unsafe for use in the UCS, which prevents it from being called from the UCS.  Useful for things that spawn actors, etc that should never happen in the UCS */
	static UE_API const FName MD_UnsafeForConstructionScripts;

	// The category that a function appears under in the palette
	static UE_API const FName MD_FunctionCategory;

	// [FunctionMetadata] Indicates that the function is deprecated
	static UE_API const FName MD_DeprecatedFunction;

	// [FunctionMetadata] Indicates that the property is deprecated
	static UE_API const FName MD_DeprecatedProperty;

	// [FunctionMetadata] Supplies the custom message to use for deprecation
	static UE_API const FName MD_DeprecationMessage;

	// [FunctionMetadata] Indicates that the function should be drawn as a compact node with the specified body title
	static UE_API const FName MD_CompactNodeTitle;

	// [FunctionMetadata] Indicates that the function should be drawn with this title over the function name
	static UE_API const FName MD_DisplayName;

	// [FunctionMetadata] Indicates the display name of the return value pin
	static UE_API const FName MD_ReturnDisplayName;

	// [FunctionMetadata] Indicates that a particular function parameter is for internal use only, which means it will be both hidden and not connectible.
	static UE_API const FName MD_InternalUseParam;

	// [FunctionMetadata] Indicates that the function should appear as blueprint function even if it doesn't return a value.
	static UE_API const FName MD_ForceAsFunction;

	// [FunctionMetadata] Indicates that the function should be ignored when considered for blueprint type promotion
	static UE_API const FName MD_IgnoreTypePromotion;

	//    property metadata

	/** UPROPERTY will be exposed on "Spawn Blueprint" nodes as an input  */
	static UE_API const FName MD_ExposeOnSpawn;

	/** UPROPERTY uses the specified function as a getter rather than reading from the property directly */
	static UE_API const FName MD_PropertyGetFunction;

	/** UPROPERTY uses the specified function as a setter rather than writing to the property directly */
	static UE_API const FName MD_PropertySetFunction;

	/** UPROPERTY cannot be modified by other blueprints */
	static UE_API const FName MD_Private;

	/** If true, the self pin should not be shown or connectable regardless of purity, const, etc. similar to InternalUseParam */
	static UE_API const FName MD_HideSelfPin;

	/** If true, the specified UObject parameter will default to "self" if nothing is connected */
	static UE_API const FName MD_DefaultToSelf;

	/** The specified parameter should be used as the context object when retrieving a UWorld pointer (implies hidden and default-to-self) */
	static UE_API const FName MD_WorldContext;

	/** For functions that have the MD_WorldContext metadata but are safe to be called from contexts that do not have the ability to provide the world context (either through GetWorld() or ShowWorldContextPin class metadata */
	static UE_API const FName MD_CallableWithoutWorldContext;

	/** For functions that should be compiled in development mode only */
	static UE_API const FName MD_DevelopmentOnly;

	/** If true, an unconnected pin will generate a UPROPERTY under the hood to connect as the input, which will be set to the literal value for the pin.  Only valid for reference parameters. */
	static UE_API const FName MD_AutoCreateRefTerm;

	UE_DEPRECATED(5.6, "Use MD_HidePinAssetPicker instead.")
	static UE_API const FName MD_HideAssetPicker;

	/** For functions, the specified parameter should hide the asset picker on the pin, even if it is a valid UObject Asset. */
	/** For properties, forcibly hide the asset picker for the pin corresponding to the property. */
	static UE_API const FName MD_HidePinAssetPicker;

	/** If true, the hidden world context pin will be visible when the function is placed in a child blueprint of the class. */
	static UE_API const FName MD_ShowWorldContextPin;

	/** Comma delimited list of pins that should be hidden on this function. */
	static UE_API const FName MD_HidePin;

	static UE_API const FName MD_BlueprintInternalUseOnly;
	static UE_API const FName MD_BlueprintInternalUseOnlyHierarchical;
	static UE_API const FName MD_NeedsLatentFixup;

	static UE_API const FName MD_LatentInfo;
	static UE_API const FName MD_LatentCallbackTarget;

	/** If true, properties defined in the C++ private scope will be accessible to blueprints */
	static UE_API const FName MD_AllowPrivateAccess;

	/** Categories of functions to expose on this property */
	static UE_API const FName MD_ExposeFunctionCategories;

	// [InterfaceMetadata]
	static UE_API const FName MD_CannotImplementInterfaceInBlueprint;
	static UE_API const FName MD_ProhibitedInterfaces;

	/** Keywords used when searching for functions */
	static UE_API const FName MD_FunctionKeywords;

	/** Indicates that during compile we want to create multiple exec pins from an enum param */
	static UE_API const FName MD_ExpandEnumAsExecs;

	/** Synonym for MD_ExpandEnumAsExecs */
	static UE_API const FName MD_ExpandBoolAsExecs;

	static UE_API const FName MD_CommutativeAssociativeBinaryOperator;

	/** Metadata string that indicates to use the MaterialParameterCollectionFunction node. */
	static UE_API const FName MD_MaterialParameterCollectionFunction;

	/** Metadata string that sets the tooltip */
	static UE_API const FName MD_Tooltip;

	/** Metadata string that indicates the specified event can be triggered in editor */
	static UE_API const FName MD_CallInEditor;

	/** Metadata to identify an DataTable Pin. Depending on which DataTable is selected, we display different RowName options */
	static UE_API const FName MD_DataTablePin;

	/** Metadata that flags make/break functions for specific struct types. */
	static UE_API const FName MD_NativeMakeFunction;
	static UE_API const FName MD_NativeBreakFunction;
	static UE_API const FName MD_NativeDisableSplitPin;

	/** Metadata that flags function params that govern what type of object the function returns */
	static UE_API const FName MD_DynamicOutputType;
	/** Metadata that flags the function output param that will be controlled by the "MD_DynamicOutputType" pin */
	static UE_API const FName MD_DynamicOutputParam;

	static UE_API const FName MD_CustomStructureParam;

	static UE_API const FName MD_ArrayParam;
	static UE_API const FName MD_ArrayDependentParam;

	/** Metadata that flags TSet parameters that will have their type determined at blueprint compile time */
	static UE_API const FName MD_SetParam;

	/** Metadata that flags TMap function parameters that will have their type determined at blueprint compile time */
	static UE_API const FName MD_MapParam;
	static UE_API const FName MD_MapKeyParam;
	static UE_API const FName MD_MapValueParam;

	/** Metadata that identifies an integral property as a bitmask. */
	static UE_API const FName MD_Bitmask;
	/** Metadata that associates a bitmask property with a bitflag enum. */
	static UE_API const FName MD_BitmaskEnum;
	/** Metadata that identifies an enum as a set of explicitly-named bitflags. */
	static UE_API const FName MD_Bitflags;
	/** Metadata that signals to the editor that enum values correspond to mask values instead of bitshift (index) values. */
	static UE_API const FName MD_UseEnumValuesAsMaskValuesInEditor;
	
	/** Stub function used internally by animation blueprints */
	static UE_API const FName MD_AnimBlueprintFunction;

	/** Metadata that should be used with UPARAM to specify whether a TSubclassOf argument should use the DisplayName of classes */
	static UE_API const FName MD_ShowDisplayNames;
	
	/** Metadata that should be used with UPARAM to specify whether a TSubclassOf argument allows abstract classes */
	static UE_API const FName MD_AllowAbstractClasses;
	/** Metadata that should be used with UPARAM to specify which classes are allowed in selection */
	static UE_API const FName MD_AllowedClasses;

	/** Metadata that should be used with UPARAM to specify a function name that generates a list of available values */
	static UE_API const FName MD_GetOptions;

	/** Namespace into which a type can be optionally defined; if empty or not set, the type will belong to the global namespace (default). */
	static UE_API const FName MD_Namespace;

	/** Function or class marked as thread-safe. Opts class/function compilation into thread-safety checks. */
	static UE_API const FName MD_ThreadSafe;

	/** Function marked as explicitly not thread-safe. Opts function out of class-level thread-safety checks. */
	static UE_API const FName MD_NotThreadSafe;

	/** Metadata to add the property or function to the FieldNotification system. */
	static UE_API const FName MD_FieldNotify;
	
private:
	// This class should never be instantiated
	FBlueprintMetadata() {}
};

USTRUCT()
// Structure used to automatically convert blueprintcallable functions (that have blueprint parameter) calls (in bp graph) 
// into their never versions (with class param instead of blueprint).
struct FBlueprintCallableFunctionRedirect
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString ClassName;

	UPROPERTY()
	FString OldFunctionName;

	UPROPERTY()
	FString NewFunctionName;

	UPROPERTY()
	FString BlueprintParamName;

	UPROPERTY()
	FString ClassParamName;
};

USTRUCT()
struct FAllowedMutableContainerFunction
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UFunction> Function;

	UPROPERTY()
	FString ContainerParameterName;

	friend uint32 GetTypeHash(const FAllowedMutableContainerFunction& Input)
	{
		return GetTypeHash(Input.Function);
	}

	bool operator==(const FAllowedMutableContainerFunction& RHS) const
	{
		return (Function == RHS.Function);
	}
};

enum class EObjectReferenceType : uint8
{
	NotAnObject		= 0x00,
	ObjectReference = 0x01,
	ClassReference	= 0x02,
	SoftObject		= 0x04,
	SoftClass		= 0x08,
	AllTypes		= 0x0f,
};

/**
* Filter flags for GetVariableTypeTree
*/
enum class ETypeTreeFilter : uint8
{
	None			= 0x00, // No Exec or Wildcards
	AllowExec		= 0x01, // Include Executable pins
	AllowWildcard	= 0x02, // Include Wildcard pins
	IndexTypesOnly	= 0x04, // Exclude all pins that aren't index types
	RootTypesOnly	= 0x08	// Exclude all pins that aren't root types
};

ENUM_CLASS_FLAGS(ETypeTreeFilter);

UCLASS(MinimalAPI, config=Editor)
class UEdGraphSchema_K2 : public UEdGraphSchema
{
	GENERATED_UCLASS_BODY()

	// Allowable PinType.PinCategory values
	static UE_API const FName PC_Exec;
	static UE_API const FName PC_Boolean;
	static UE_API const FName PC_Byte;
	static UE_API const FName PC_Class;    // SubCategoryObject is the MetaClass of the Class passed thru this pin, or SubCategory can be 'self'. The DefaultValue string should always be empty, use DefaultObject.
	static UE_API const FName PC_SoftClass;
	static UE_API const FName PC_Int;
	static UE_API const FName PC_Int64;
	static UE_API const FName PC_Float;
	static UE_API const FName PC_Double;
	static UE_API const FName PC_Real;
	static UE_API const FName PC_Name;
	static UE_API const FName PC_Delegate;    // SubCategoryObject is the UFunction of the delegate signature
	static UE_API const FName PC_MCDelegate;  // SubCategoryObject is the UFunction of the delegate signature
	static UE_API const FName PC_Object;    // SubCategoryObject is the Class of the object passed thru this pin, or SubCategory can be 'self'. The DefaultValue string should always be empty, use DefaultObject.
	static UE_API const FName PC_Interface;	// SubCategoryObject is the Class of the object passed thru this pin.
	static UE_API const FName PC_SoftObject;		// SubCategoryObject is the Class of the AssetPtr passed thru this pin.
	static UE_API const FName PC_String;
	static UE_API const FName PC_Text;
	static UE_API const FName PC_Struct;    // SubCategoryObject is the ScriptStruct of the struct passed thru this pin, 'self' is not a valid SubCategory. DefaultObject should always be empty, the DefaultValue string may be used for supported structs.
	static UE_API const FName PC_Wildcard;    // Special matching rules are imposed by the node itself
	static UE_API const FName PC_Enum;    // SubCategoryObject is the UEnum object passed thru this pin
	static UE_API const FName PC_FieldPath;		// SubCategoryObject is the Class of the property passed thru this pin.

	// Common PinType.PinSubCategory values
	static UE_API const FName PSC_Self;    // Category=PC_Object or PC_Class, indicates the class being compiled

	static UE_API const FName PSC_Index;	// Category=PC_Wildcard, indicates the wildcard will only accept Int, Bool, Byte and Enum pins (used when a pin represents indexing a list)
	static UE_API const FName PSC_Bitmask;	// Category=PC_Byte or PC_Int, indicates that the pin represents a bitmask field. SubCategoryObject is either NULL or the UEnum object to which the bitmap is linked for bitflag name specification.

	// Pin names that have special meaning and required types in some contexts (depending on the node type)
	static UE_API const FName PN_Execute;    // Category=PC_Exec, singleton, input
	static UE_API const FName PN_Then;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_Completed;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_DelegateEntry;    // Category=PC_Exec, singleton, output; entry point for a dynamically bound delegate
	static UE_API const FName PN_EntryPoint;	// entry point to the ubergraph
	static UE_API const FName PN_Self;    // Category=PC_Object, singleton, input
	static UE_API const FName PN_Else;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_Loop;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_After;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_ReturnValue;		// Category=PC_Object, singleton, output
	static UE_API const FName PN_ObjectToCast;    // Category=PC_Object, singleton, input
	static UE_API const FName PN_Condition;    // Category=PC_Boolean, singleton, input
	static UE_API const FName PN_Start;    // Category=PC_Int, singleton, input
	static UE_API const FName PN_Stop;    // Category=PC_Int, singleton, input
	static UE_API const FName PN_Index;    // Category=PC_Int, singleton, output
	static UE_API const FName PN_Item;    // Category=PC_Int, singleton, output
	static UE_API const FName PN_CastSucceeded;    // Category=PC_Exec, singleton, output
	static UE_API const FName PN_CastFailed;    // Category=PC_Exec, singleton, output
	static UE_API const FString PN_CastedValuePrefix;    // Category=PC_Object, singleton, output; actual pin name varies depending on the type to be casted to, this is just a prefix

	// construction script function names
	static UE_API const FName FN_UserConstructionScript;
	static UE_API const FName FN_ExecuteUbergraphBase;

	// metadata keys

	//   class metadata


	// graph names
	static UE_API const FName GN_EventGraph;
	static UE_API const FName GN_AnimGraph;

	// variable names
	static UE_API const FText VR_DefaultCategory;

	// action grouping values
	static UE_API const int32 AG_LevelReference;

	// Somewhat hacky mechanism to prevent tooltips created for pins from including the display name and type when generating BP API documentation
	static UE_API bool bGeneratingDocumentation;

	// ID for checking dirty status of node titles against, increases every compile
	static UE_API int32 CurrentCacheRefreshID;

	// Pin Selector category for all object types
	static UE_API const FName AllObjectTypes;

	UPROPERTY(globalconfig)
	TArray<FBlueprintCallableFunctionRedirect> EditoronlyBPFunctionRedirects;

	UPROPERTY(globalconfig)
	bool bEnableMutableContainerWarnings;

	UPROPERTY(globalconfig)
	TSet<FAllowedMutableContainerFunction> AllowedMutableContainerFunctions;

public:

	//////////////////////////////////////////////////////////////////////////
	// FPinTypeInfo
	/** Class used for creating type tree selection info, which aggregates the various PC_* and PinSubtypes in the schema into a hierarchy */
	class FPinTypeTreeInfo
	{
	private:
		/** The pin type corresponding to the schema type */
		FEdGraphPinType PinType;
		uint8 PossibleObjectReferenceTypes;

		/** Asset Data, used when PinType.PinSubCategoryObject is not loaded yet */
		FAssetData CachedAssetData;

		/** The pin type description, localized */
		FText CachedDescription;

		/** A copy of the localized CachedDescription string, for sorting */
		TSharedPtr<FString> CachedDescriptionString;
	public:
		/** The children of this pin type */
		TArray< TSharedPtr<FPinTypeTreeInfo> > Children;

		/** Whether or not this pin type is selectable as an actual type, or is just a category, with some subtypes */
		bool bReadOnly;

		/** Text for regular tooltip */
		FText Tooltip;

	public:
		UE_API const FEdGraphPinType& GetPinType(bool bForceLoadedSubCategoryObject);
		const FEdGraphPinType& GetPinTypeNoResolve() const { return PinType; }
		void SetPinSubTypeCategory(const FName SubCategory)
		{
			PinType.PinSubCategory = SubCategory;
		}

		UE_API FPinTypeTreeInfo(const FText& InFriendlyName, const FName CategoryName, const UEdGraphSchema_K2* Schema, const FText& InTooltip, bool bInReadOnly = false);
		UE_API FPinTypeTreeInfo(const FName CategoryName, UObject* SubCategoryObject, const FText& InTooltip, bool bInReadOnly = false, uint8 InPossibleObjectReferenceTypes = 0);
		UE_API FPinTypeTreeInfo(const FText& InFriendlyName, const FName CategoryName, const FAssetData& AssetData, const FText& InTooltip, bool bInReadOnly = false, uint8 InPossibleObjectReferenceTypes = 0);
		UE_API FPinTypeTreeInfo(TSharedPtr<FPinTypeTreeInfo> InInfo);
		
		/** Returns a succinct menu description of this type */
		UE_API const FText& GetDescription() const;
		
		/** Returns the localized description as a string, for faster sorting */
		const FString& GetCachedDescriptionString() const
		{
			return *CachedDescriptionString;
		}

		FText GetToolTip() const
		{
			if (PinType.PinSubCategoryObject.IsValid())
			{
				if (Tooltip.IsEmpty())
				{
					if ( (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct) && PinType.PinSubCategoryObject->IsA(UScriptStruct::StaticClass()) )
					{
						return FText::FromString(PinType.PinSubCategoryObject->GetPathName());
					}
				}
			}
			return Tooltip;
		}

		uint8 GetPossibleObjectReferenceTypes() const
		{
			return PossibleObjectReferenceTypes;
		}

		UE_API const FAssetData& GetCachedAssetData() const;

	private:

		FPinTypeTreeInfo()
			: PossibleObjectReferenceTypes(0)
			, bReadOnly(false)
		{}

		UE_API FText GenerateDescription();
	};

public:
	UE_API void SelectAllNodesInDirection(TEnumAsByte<enum EEdGraphPinDirection> InDirection, UEdGraph* Graph, UEdGraphPin* InGraphPin);

	//~ Begin EdGraphSchema Interface
	UE_API virtual void GetContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual const FPinConnectionResponse CanCreateConnection(const UEdGraphPin* A, const UEdGraphPin* B) const override;
	UE_API virtual bool TryCreateConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	UE_API virtual bool CreateAutomaticConversionNodeAndConnections(UEdGraphPin* A, UEdGraphPin* B) const override;
	UE_API virtual bool CreatePromotedConnection(UEdGraphPin* A, UEdGraphPin* B) const override;
	UE_API virtual FString IsPinDefaultValid(const UEdGraphPin* Pin, const FString& NewDefaultValue, TObjectPtr<UObject> NewDefaultObject, const FText& InNewDefaultText) const override;
	UE_API virtual bool DoesSupportPinWatching() const	override;
	UE_API virtual bool IsPinBeingWatched(UEdGraphPin const* Pin) const override;
	UE_API virtual void ClearPinWatch(UEdGraphPin const* Pin) const override;
	UE_API virtual void TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified = true) const override;
	UE_API virtual void TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified = true) const override;
	UE_API virtual void TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified = true) const override;
	UE_API virtual bool DoesDefaultValueMatchAutogenerated(const UEdGraphPin& InPin) const override;
	UE_API virtual void ResetPinToAutogeneratedDefaultValue(UEdGraphPin* Pin, bool bCallModifyCallbacks = true) const override;
	UE_API virtual bool ShouldHidePinDefaultValue(UEdGraphPin* Pin) const override;
	UE_API virtual bool ShouldShowAssetPickerForPin(UEdGraphPin* Pin) const override;
	UE_API virtual FLinearColor GetPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual FLinearColor GetSecondaryPinTypeColor(const FEdGraphPinType& PinType) const override;
	UE_API virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	UE_API virtual void ConstructBasicPinTooltip(const UEdGraphPin& Pin, const FText& PinDescription, FString& TooltipOut) const override;
	UE_API virtual EGraphType GetGraphType(const UEdGraph* TestEdGraph) const override;
	UE_API virtual bool IsTitleBarPin(const UEdGraphPin& Pin) const override;
	UE_API virtual void BreakNodeLinks(UEdGraphNode& TargetNode) const override;
	UE_API virtual void BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const override;
	UE_API virtual void BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const override;
	UE_API virtual void ReconstructNode(UEdGraphNode& TargetNode, bool bIsBatchRequest=false) const override;
	UE_API virtual bool CanEncapuslateNode(UEdGraphNode const& TestNode) const override;
	UE_API virtual void HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const override;
	UE_API virtual void GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const override;
	UE_API virtual void DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const override;
	UE_API virtual void DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const override;
	UE_API virtual void DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const override;
	UE_API virtual void GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const override;
	UE_API virtual void GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const override;
	UE_API virtual bool CanDuplicateGraph(UEdGraph* InSourceGraph) const override;
	UE_API virtual UEdGraph* DuplicateGraph(UEdGraph* GraphToDuplicate) const override;
	UE_API virtual UEdGraphNode* CreateSubstituteNode(UEdGraphNode* Node, const UEdGraph* Graph, FObjectInstancingGraph* InstanceGraph, TSet<FName>& InOutExtraNames) const override;
	UE_API virtual int32 GetNodeSelectionCount(const UEdGraph* Graph) const override;
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> GetCreateCommentAction() const override;
	UE_API virtual bool FadeNodeWhenDraggingOffPin(const UEdGraphNode* Node, const UEdGraphPin* Pin) const override;
	UE_API virtual void BackwardCompatibilityNodeConversion(UEdGraph* Graph, bool bOnlySafeChanges) const override;
	virtual bool ShouldAlwaysPurgeOnModification() const override { return false; }
	UE_API virtual void SplitPin(UEdGraphPin* Pin, bool bNotify = true) const override;
	UE_API virtual void RecombinePin(UEdGraphPin* Pin) const override;
	UE_API virtual void OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const override;
	UE_API virtual UEdGraphPin* DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const override;
	UE_API virtual bool SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const override;
	UE_API virtual bool IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const override;
	UE_API virtual int32 GetCurrentVisualizationCacheID() const override;
	UE_API virtual void ForceVisualizationCacheClear() const override;
	UE_API virtual bool SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* NodeToDelete) const override;
	virtual bool CanVariableBeDropped(UEdGraph* InGraph, FProperty* InVariableToDrop) const override { return true; }
	UE_API virtual bool CanShowDataTooltipForPin(const UEdGraphPin& Pin) const override;

#if WITH_EDITORONLY_DATA
	UE_API virtual float GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "FGraphActionListBuilderBase::ActionGroup weight calculation was always performed on the first action, provide that if scoring a FGraphActionListBuilderBase::ActionGroup, or just provide the FEdGraphSchemaAction")
	UE_API virtual float GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const override;
	UE_API PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual FGraphSchemaSearchWeightModifiers GetSearchWeightModifiers() const override;
#endif // WITH_EDITORONLY_DATA	

	//~ End EdGraphSchema Interface

	/**
	 * Determine if this graph supports collapsing nodes into subgraphs
	 *
	 * @return True if this schema supports collapsed node subgraphs
	 */
	virtual bool DoesSupportCollapsedNodes() const { return true; }

	/**
	 *
	 *	Determine if this graph supports event dispatcher
	 * 
	 *	@return True if this schema supports Event Dispatcher 
	 */
	virtual bool DoesSupportEventDispatcher() const { return true; }

	/**
	* Configure the supplied variable node based on the supplied info
	*
	* @param	InVarNode			The variable node to be configured
	* @param	InVariableName		The name of the current variable
	* @param	InVaraiableSource	The source of the variable
	* @param	InTargetBlueprint	The blueprint this node will be used on
	*/
	static UE_API void ConfigureVarNode(class UK2Node_Variable* InVarNode, FName InVariableName, UStruct* InVariableSource, UBlueprint* InTargetBlueprint);

	/**
	* Creates a new variable getter node and adds it to ParentGraph
	*
	* @param	GraphPosition		The location of the new node inside the graph
	* @param	ParentGraph			The graph to spawn the new node in
	* @param	VariableName		The name of the variable
	* @param	Source				The source of the variable
	* @return	A pointer to the newly spawned node
	*/
	UE_API virtual class UK2Node_VariableGet* SpawnVariableGetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const;

	/**
	* Creates a new variable setter node and adds it to ParentGraph
	*
	* @param	GraphPosition		The location of the new node inside the graph
	* @param	ParentGraph			The graph to spawn the new node in
	* @param	VariableName		The name of the variable
	* @param	Source				The source of the variable
	* @return	A pointer to the newly spawned node
	*/
	UE_API virtual class UK2Node_VariableSet* SpawnVariableSetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const;

	// Returns whether the supplied Pin is a splittable struct
	UE_API bool PinHasSplittableStructType(const UEdGraphPin* InGraphPin) const;

	/** Returns true if the pin has a value field that can be edited inline */
	UE_API bool PinDefaultValueIsEditable(const UEdGraphPin& InGraphPin) const;

	/** Returns true if the pin has a custom default string format and it is not safe to use ExportText */
	UE_API bool PinHasCustomDefaultFormat(const UEdGraphPin& InGraphPin) const;

	struct FCreateSplitPinNodeParams
	{
		FCreateSplitPinNodeParams(const bool bInTransient)
			: CompilerContext(nullptr)
			, SourceGraph(nullptr)
			, bTransient(bInTransient)
		{}

		FCreateSplitPinNodeParams(class FKismetCompilerContext* InCompilerContext, UEdGraph* InSourceGraph)
			: CompilerContext(InCompilerContext)
			, SourceGraph(InSourceGraph)
			, bTransient(false)
		{}

		FKismetCompilerContext* CompilerContext;
		UEdGraph* SourceGraph;
		bool bTransient;
	};

	/** Helper function to create the expansion node.  
		If the CompilerContext is specified this will be created as an intermediate node */
	UE_API UK2Node* CreateSplitPinNode(UEdGraphPin* Pin, const FCreateSplitPinNodeParams& Params) const;

	/** Reads in a FString and gets the values of the pin defaults for that type. This can be passed to DefaultValueSimpleValidation to validate. OwningObject can be null */
	UE_API virtual void GetPinDefaultValuesFromString(const FEdGraphPinType& PinType, UObject* OwningObject, const FString& NewValue, FString& UseDefaultValue, TObjectPtr<UObject>& UseDefaultObject, FText& UseDefaultText, bool bPreserveTextIdentity = true) const;

	/** Do validation, that doesn't require a knowledge about actual pin */
	UE_API virtual bool DefaultValueSimpleValidation(const FEdGraphPinType& PinType, const FName PinName, const FString& NewDefaultValue, TObjectPtr<UObject> NewDefaultObject, const FText& InText, FString* OutMsg = nullptr) const;

	/** Returns true if the owning node is a function with AutoCreateRefTerm meta data */
	static UE_API bool IsAutoCreateRefTerm(const UEdGraphPin* Pin);

	/** See if a class has any members that are accessible by a blueprint */
	UE_API bool ClassHasBlueprintAccessibleMembers(const UClass* InClass) const;

	/**
	 * Checks to see if the specified graph is a construction script
	 *
	 * @param	TestEdGraph		Graph to test
	 * @return	true if this is a construction script
	 */
	static UE_API bool IsConstructionScript(const UEdGraph* TestEdGraph);

	/**
	 * Checks to see if the specified graph is a composite graph
	 *
	 * @param	TestEdGraph		Graph to test
	 * @return	true if this is a composite graph
	 */
	UE_API bool IsCompositeGraph(const UEdGraph* TestEdGraph) const;

	/**
	 * Checks to see if the specified graph is a const function graph
	 *
	 * @param	TestEdGraph		Graph to test
	 * @param	bOutIsEnforcingConstCorrectness (Optional) Whether or not this graph is enforcing const correctness during compilation
	 * @return	true if this is a const function graph
	 */
	UE_API bool IsConstFunctionGraph(const UEdGraph* TestEdGraph, bool* bOutIsEnforcingConstCorrectness = nullptr) const;

	/**
	 * Checks to see if the specified graph is a static function graph
	 *
	 * @param	TestEdGraph		Graph to test
	 * @return	true if this is a const function graph
	 */
	UE_API bool IsStaticFunctionGraph(const UEdGraph* TestEdGraph) const;

	/**
	 * Checks to see if a pin is an execution pin.
	 *
	 * @param	Pin	The pin to check.
	 * @return	true if it is an execution pin.
	 */
	static inline bool IsExecPin(const UEdGraphPin& Pin)
	{
		return Pin.PinType.PinCategory == PC_Exec;
	}

	/**
	 * Checks to see if a pin is a Self pin (indicating the calling context for the node)
	 *
	 * @param	Pin	The pin to check.
	 * @return	true if it is a Self pin.
	 */
	UE_API virtual bool IsSelfPin(const UEdGraphPin& Pin) const override;

	/**
	 * Checks to see if a pin is a meta-pin (either a Self or Exec pin)
	 *
	 * @param	Pin	The pin to check.
	 * @return	true if it is a Self or Exec pin.
	 */
	inline bool IsMetaPin(const UEdGraphPin& Pin) const
	{
		return IsExecPin(Pin) || IsSelfPin(Pin);
	}

	/** Is given string a delegate category name ? */
	UE_API virtual bool IsDelegateCategory(const FName Category) const override;

	/** Returns whether a pin category is compatible with an Index Wildcard (PC_Wildcard and PSC_Index) */
	inline bool IsIndexWildcardCompatible(const FEdGraphPinType& PinType) const
	{
		return (!PinType.IsContainer()) && 
			(
				PinType.PinCategory == PC_Boolean || 
				PinType.PinCategory == PC_Int || 
				PinType.PinCategory == PC_Byte ||
				(PinType.PinCategory == PC_Wildcard && PinType.PinSubCategory == PSC_Index)
			);
	}

	/**
	 * Searches for the first execution pin with the specified direction on the node
	 *
	 * @param	Node			The node to search.
	 * @param	PinDirection	The required pin direction.
	 *
	 * @return	the first found execution pin with the correct direction or null if there were no matching pins.
	 */
	UEdGraphPin* FindExecutionPin(const UEdGraphNode& Node, EEdGraphPinDirection PinDirection) const
	{
		for (int32 PinIndex = 0; PinIndex < Node.Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Node.Pins[PinIndex];

			if ((Pin->Direction == PinDirection) && IsExecPin(*Pin))
			{
				return Pin;
			}
		}

		return NULL;
	}

	/**
	 * Searches for the first Self pin with the specified direction on the node
	 *
	 * @param	Node			The node to search.
	 * @param	PinDirection	The required pin direction.
	 *
	 * @return	the first found self pin with the correct direction or null if there were no matching pins.
	 */
	UEdGraphPin* FindSelfPin(const UEdGraphNode& Node, EEdGraphPinDirection PinDirection) const
	{
		for (int32 PinIndex = 0; PinIndex < Node.Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Node.Pins[PinIndex];

			if ((Pin->Direction == PinDirection) && IsSelfPin(*Pin))
			{
				return Pin;
			}
		}

		return NULL;
	}

	/** Can Pin be promoted to a variable? */
	UE_API bool CanPromotePinToVariable (const UEdGraphPin& Pin, bool bInToMemberVariable) const;

	/** Can Pin be split in to its component elements */
	UE_API bool CanSplitStructPin(const UEdGraphPin& Pin) const;

	/** Can Pin be recombined back to its original form */
	UE_API bool CanRecombineStructPin(const UEdGraphPin& Pin) const;

	/** 
	 * Helper function for filling out Category, SubCategory, and SubCategoryObject based on a FProperty 
	 * 
	 * @return	true on success, false if the property is unsupported or invalid.
	 */
	static UE_API bool GetPropertyCategoryInfo(const FProperty* TestProperty, FName& OutCategory, FName& OutSubCategory, UObject*& OutSubCategoryObject, bool& bOutIsWeakPointer);

	/**
	 * Convert the type of a FProperty to the corresponding pin type.
	 *
	 * @param		Property	The property to convert.
	 * @param [out]	TypeOut		The resulting pin type.
	 *
	 * @return	true on success, false if the property is unsupported or invalid.
	 */
	UE_API bool ConvertPropertyToPinType(const FProperty* Property, /*out*/ FEdGraphPinType& TypeOut) const;

	/** Returns true if the function has wildcard parameters, e.g. uses runtime type information that may require safe failure handling */
	static UE_API bool HasWildcardParams(const UFunction* Function);

	/** Determines if the specified param property is intended to be used as a wildcard (for custom thunk functions, like in our array library, etc.)*/
	static UE_API bool IsWildcardProperty(const FProperty* ParamProperty);

	/** Flags to indicate different types of blueprint callable functions */
	enum EFunctionType
	{
		FT_Imperative	= 0x01,
		FT_Pure			= 0x02,
		FT_Const		= 0x04,
		FT_Protected	= 0x08,
	};

	/**
	 * Finds the parent function for the specified function, if any
	 *
	 * @param	Function			The function to find a parent function for
	 * @return	The UFunction parentfunction, if any.
	 */
	static UE_API UFunction* GetCallableParentFunction(UFunction* Function);

	/** Whether or not the specified actor is a valid target for bound events and literal references (in the right level, not a builder brush, etc */
	UE_API bool IsActorValidForLevelScriptRefs(const AActor* TestActor, const UBlueprint* Blueprint) const;

	/** 
	 *	Generate a list of replaceable nodes for context menu based on the editor's current selection 
	 *
	 *	@param	Reference to graph node
	 *	@param	Reference to context menu builder
	 */
	UE_API void AddSelectedReplaceableNodes(FToolMenuSection& Section, UBlueprint* Blueprint, const UEdGraphNode* InGraphNode) const;

	/**
	 *	Function to replace current graph node reference object with a new object
	 *
	 *	@param Reference to graph node
	 *	@param Reference to new reference object
	 */
	UE_API void ReplaceSelectedNode(UEdGraphNode* SourceNode, AActor* TargetActor);

	/** Returns whether a function is marked 'override' and doesn't have any out parameters */
	static UE_API bool FunctionCanBePlacedAsEvent(const UFunction* InFunction);

	/** Can this function be called by kismet delegate */
	static UE_API bool FunctionCanBeUsedInDelegate(const UFunction* InFunction);

	/** Can this function be called by kismet code */
	static UE_API bool CanUserKismetCallFunction(const UFunction* Function);

	/** Returns if function has output parameter(s) */
	static UE_API bool HasFunctionAnyOutputParameter(const UFunction* Function);

	enum EDelegateFilterMode
	{
		CannotBeDelegate,
		MustBeDelegate,
		VariablesAndDelegates
	};

	/** Can this variable be accessed by kismet code */
	static UE_API bool CanUserKismetAccessVariable(const FProperty* Property, const UClass* InClass, EDelegateFilterMode FilterMode);

	/** Can this function be overridden by kismet (either placed as event or new function graph created) */
	static UE_API bool CanKismetOverrideFunction(const UFunction* Function);

	/** returns friendly signature name if possible or Removes any mangling to get the unmangled signature name of the function */
	static UE_API FText GetFriendlySignatureName(const UFunction* Function);

	/** Returns true if this enum is safe to be used as a variable in blueprints */
	static UE_API bool IsAllowableBlueprintVariableType(const UEnum* InEnum);

	/** 
	 * Returns true if this class is safe to be used as a variable in blueprints
	 *
	 * @param	bImpliedBlueprintType	If true, the class is implied to be a blueprint type and will be allowed unless it is specifically forbidden
	 */
	static UE_API bool IsAllowableBlueprintVariableType(const UClass* InClass, bool bAssumeBlueprintType = false);

	/**
	 * Returns true if this struct is safe to to be used as a variable in blueprints
	 *
	 * @param	bForInternalUse			If true, the struct is for internal usage so BlueprintInternalUseOnly is allowed
	 */
	static UE_API bool IsAllowableBlueprintVariableType(const UScriptStruct *InStruct, bool bForInternalUse = false);

	static UE_API bool IsPropertyExposedOnSpawn(const FProperty* Property);

	/** returns whether the property is deprecated. */
	static UE_API bool IsPropertyDeprecated(const FProperty* Property);

	/**
	 * Returns a list of parameters for the function that are specified as automatically emitting terms for unconnected ref parameters in the compiler (MD_AutoCreateRefTerm)
	 *
	 * @param	Function				The function to check for auto-emitted ref terms on
	 * @param	AutoEmitParameterNames	(out) Returns an array of param names that should be auto-emitted if nothing is connected
	 */
	static UE_API void GetAutoEmitTermParameters(const UFunction* Function, TArray<FString>& AutoEmitParameterNames);

	/**
	 * Determine if a function has a parameter of a specific type.
	 *
	 * @param	InFunction	  	The function to search.
	 * @param	InGraph			The graph that you're looking to call the function from (some functions hide different pins depending on the graph they're in)
	 * @param	DesiredPinType	The type that at least one function parameter needs to be.
	 * @param	bWantOutput   	The direction that the parameter needs to be.
	 *
	 * @return	true if at least one parameter is of the correct type and direction.
	 */
	UE_API bool FunctionHasParamOfType(const UFunction* InFunction, UEdGraph const* InGraph, const FEdGraphPinType& DesiredPinType, bool bWantOutput) const;

	/**
	 * Add the specified flags to the function entry node of the graph, to make sure they get compiled in to the generated function
	 *
	 * @param	CurrentGraph	The graph of the function to modify the flags for
	 * @param	ExtraFlags		The flags to add to the function entry node
	 */
	UE_API void AddExtraFunctionFlags(const UEdGraph* CurrentGraph, int32 ExtraFlags) const;

	/**
	 * Marks the function entry of a graph as editable via function editor or not-editable
	 *
	 * @param	CurrentGraph	The graph of the function to modify the entry node for
	 * @param	bNewEditable	Whether or not the function entry for the graph should be editable via the function editor
	 */
	UE_API void MarkFunctionEntryAsEditable(const UEdGraph* CurrentGraph, bool bNewEditable) const;

	/** 
	 * Populate new macro graph with entry and possibly return node
	 * 
	 * @param	Graph			Graph to add the function terminators to
	 * @param	ContextClass	If specified, the graph terminators will use this class to search for context for signatures (i.e. interface functions)
	 */
	UE_API virtual void CreateMacroGraphTerminators(UEdGraph& Graph, UClass* Class) const;

	/** 
	 * Populate new function graph with entry and possibly return node
	 * 
	 * @param	Graph			Graph to add the function terminators to
	 * @param	ContextClass	If specified, the graph terminators will use this class to search for context for signatures (i.e. interface functions)
	 */
	UE_API virtual void CreateFunctionGraphTerminators(UEdGraph& Graph, UClass* Class) const;

	/**
	 * Populate new function graph with entry and possibly return node
	 * 
	 * @param	Graph			Graph to add the function terminators to
	 * @param	FunctionSignature	The function signature to mimic when creating the inputs and outputs for the function.
	 */
	UE_API virtual void CreateFunctionGraphTerminators(UEdGraph& Graph, const UFunction* FunctionSignature) const;

	/**
	 * Converts the type of a property into a fully qualified string (e.g., object'ObjectName').
	 *
	 * @param	Property	The property to convert into a string.
	 *
	 * @return	The converted type string.
	 */
	static UE_API FText TypeToText(const FProperty* const Property);

	/**
	* Converts a terminal type into a fully qualified FText (e.g., object'ObjectName').
	* Primarily used as a helper when converting containers to TypeToText.
	*
	* @param	Category					The category to convert into a FText.
	* @param	SubCategory					The subcategory to convert into FText
	* @param	SubCategoryObject			The SubcategoryObject to convert into FText
	* @param	bIsWeakPtr					Whether the type is a WeakPtr
	*
	* @return	The converted type text.
	*/
	static UE_API FText TerminalTypeToText(const FName Category, const FName SubCategory, UObject* SubCategoryObject, bool bIsWeakPtr);

	/**
	 * Converts a pin type into a fully qualified FText (e.g., object'ObjectName').
	 *
	 * @param	Type	The type to convert into a FText.
	 *
	 * @return	The converted type text.
	 */
	static UE_API FText TypeToText(const FEdGraphPinType& Type);

	/**
	 * Returns the FText to use for a given schema category
	 *
	 * @param	Category	The category to convert into a FText.
	 * @param	bForMenu	Indicates if this is for display in tooltips or menu
	 *
	 * @return	The text to display for the category.
	 */
	static UE_API FText GetCategoryText(const FName Category, const bool bForMenu = false);

	/**
	 * Returns the FText to use for a given schema category and subcategory
	 *
	 * @param	Category	The category to convert into a FText.
	 * @param	SubCategory	The subcategory to convert into a FText.
	 * @param	bForMenu	Indicates if this is for display in tooltips or menu
	 *
	 * @return	The text to display for the category.
	 */
	static UE_API FText GetCategoryText(FName Category, FName SubCategory, bool bForMenu = false);

	/**
	 * Get the type tree for all of the property types valid for this schema
	 *
	 * @param	TypeTree		The array that will contain the type tree hierarchy for this schema upon returning
	 * @param	TypeTreeFilter	ETypeTreeFilter flags that determine how the TypeTree is populated.
	 */
	UE_API virtual void GetVariableTypeTree(TArray< TSharedPtr<FPinTypeTreeInfo> >& TypeTree, ETypeTreeFilter TypeTreeFilter = ETypeTreeFilter::None) const;

	/**
	 * Returns whether or not the specified type has valid subtypes available
	 *
	 * @param	Type	The type to check for subtypes
	 */
	UE_API bool DoesTypeHaveSubtypes( const FName Category ) const;

	/**
	 * Returns true if the types and directions of two pins are schema compatible. Handles
	 * outputting a more derived type to an input pin expecting a less derived type.
	 *
	 * @param	PinA		  	The pin a.
	 * @param	PinB		  	The pin b.
	 * @param	CallingContext	(optional) The calling context (required to properly evaluate pins of type Self)
	 * @param	bIgnoreArray	(optional) Whether or not to ignore differences between array and non-array types
	 *
	 * @return	true if the pin types and directions are compatible.
	 */
	UE_API virtual bool ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext = NULL, bool bIgnoreArray = false) const override;

	/**
	 * Returns the connection response for connecting PinA to PinB, which have already been determined to be compatible
	 * types with a compatible direction.  InputPin and OutputPin are PinA and PinB or vis versa, indicating their direction.
	 *
	 * @param	PinA		  	The pin a.
	 * @param	PinB		  	The pin b.
	 * @param	InputPin	  	Either PinA or PinB, depending on which one is the input.
	 * @param	OutputPin	  	Either PinA or PinB, depending on which one is the output.
	 *
	 * @return	The message and action to take on trying to make this connection.
	 */
	UE_API virtual const FPinConnectionResponse DetermineConnectionResponseOfCompatibleTypedPins(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const;

	/**
	 * Returns true if the two pin types are schema compatible.  Handles outputting a more derived
	 * type to an input pin expecting a less derived type.
	 *
	 * @param	Output		  	The output type.
	 * @param	Input		  	The input type.
	 * @param	CallingContext	(optional) The calling context (required to properly evaluate pins of type Self)
	 * @param	bIgnoreArray	(optional) Whether or not to ignore differences between array and non-array types
	 *
	 * @return	true if the pin types are compatible.
	 */
	UE_API virtual bool ArePinTypesCompatible(const FEdGraphPinType& Output, const FEdGraphPinType& Input, const UClass* CallingContext = NULL, bool bIgnoreArray = false) const;

	/**
	 * Returns true if the types are schema Equivalent. 
	 *
	 * @param	PinA		  	The type of Pin A.
	 * @param	PinB		  	The type of Pin B.
	 *
	 * @return	true if the pin types and directions are compatible.
	 */
	UE_API virtual bool ArePinTypesEquivalent(const FEdGraphPinType& PinA, const FEdGraphPinType& PinB) const;

	/** Sets the autogenerated default value for a pin, optionally using the passed in function and parameter. This will also reset the current default value to the autogenerated one */
	UE_API virtual void SetPinAutogeneratedDefaultValue(UEdGraphPin* Pin, const FString& NewValue) const;

	/** Sets the autogenerated default value for a pin using the default for that type. This will also reset the current default value to the autogenerated one */
	UE_API virtual void SetPinAutogeneratedDefaultValueBasedOnType(UEdGraphPin* Pin) const;

	/** Sets the pin defaults, but not autogenerated defaults, at pin construction time. This is like TrySetDefaultValue but does not do validation or callbacks */
	UE_API virtual void SetPinDefaultValueAtConstruction(UEdGraphPin* Pin, const FString& DefaultValueString) const;

	/** Call to let blueprint and UI know that parameters have changed for a function/macro/etc */
	UE_API virtual void HandleParameterDefaultValueChanged(UK2Node* TargetNode) const;

	/** Given a function and property, return the default value */
	static UE_API bool FindFunctionParameterDefaultValue(const UFunction* Function, const FProperty* Param, FString& OutString);

	/** Utility that makes sure existing connections are valid, breaking any that are now illegal. */
	static UE_API void ValidateExistingConnections(UEdGraphPin* Pin);

	/** Find a 'set value by name' function for the specified pin, if it exists */
	static UE_API UFunction* FindSetVariableByNameFunction(const FEdGraphPinType& PinType);

	/** Find an appropriate function to call to perform an automatic cast operation */
	struct FSearchForAutocastFunctionResults
	{
		FName TargetFunction;
		UClass* FunctionOwner = nullptr;
	};
	[[nodiscard]] UE_API virtual TOptional<FSearchForAutocastFunctionResults> SearchForAutocastFunction(const FEdGraphPinType& OutputPinType, const FEdGraphPinType& InputPinType) const;
	
	/** Find an appropriate node that can convert from one pin type to another (not a cast; e.g. "MakeLiteralArray" node) */
	struct FFindSpecializedConversionNodeResults
	{
		class UK2Node* TargetNode = nullptr;
	};
	[[nodiscard]] UE_API virtual TOptional<FFindSpecializedConversionNodeResults> FindSpecializedConversionNode(const FEdGraphPinType& OutputPinType, const UEdGraphPin& InputPin, bool bCreateNode) const;

	/** Create menu for variable get/set nodes which refer to a variable which does not exist. */
	UE_API void GetNonExistentVariableMenu(FToolMenuSection& Section, const UEdGraphNode* InGraphNode, UBlueprint* OwnerBlueprint) const;

	/**
	 * Create menu for variable get/set nodes which allows for the replacement of variables
	 *
	 * @param InGraphNode					Variable node to replace
	 * @param InOwnerBlueprint				The owning Blueprint of the variable
	 * @param InMenuBuilder					MenuBuilder to place the menu items into
	 * @param bInReplaceExistingVariable	TRUE if replacing an existing variable, will keep the variable from appearing on the list
	 */
	UE_API void GetReplaceVariableMenu(FToolMenuSection& Section, const UEdGraphNode* InGraphNode, UBlueprint* InOwnerBlueprint, bool bInReplaceExistingVariable = false) const;

	// Calculates an average position between the nodes owning the two specified pins
	static UE_API FVector2D CalculateAveragePositionBetweenNodes(UEdGraphPin* InputPin, UEdGraphPin* OutputPin);

	// Tries to connect any pins with matching types and directions from the conversion node to the specified input and output pins
	UE_API void AutowireConversionNode(UEdGraphPin* InputPin, UEdGraphPin* OutputPin, UEdGraphNode* ConversionNode) const;

	/** Calculates an estimated height for the specified node */
	static UE_API float EstimateNodeHeight( UEdGraphNode* Node );

	/** Calculates an estimated width for the specified node */
	static UE_API float EstimateNodeWidth( const UEdGraphNode* Node );

	/** 
	 * Checks if the graph supports impure functions
	 *
	 * @param InGraph		Graph to check
	 *
	 * @return				True if the graph supports impure functions
	 */
	UE_API bool DoesGraphSupportImpureFunctions(const UEdGraph* InGraph) const;

	/** 
	 * Checks if the graph is marked as thread safe
	 * @param InGraph		Graph to check
	 * @return			True if the graph is marked theead safe
	 */
	UE_API bool IsGraphMarkedThreadSafe(const UEdGraph* InGraph) const;
	
	/**
	 * Checks to see if the passed in function is valid in the graph for the current class
	 *
	 * @param	InClass  			Class being checked to see if the function is valid for
	 * @param	InFunction			Function being checked
	 * @param	InDestGraph			Graph we will be using action for (may be NULL)
	 * @param	InFunctionTypes		Combination of EFunctionType to indicate types of functions accepted
	 * @param	bInCalledForEach	Call for each element in an array (a node accepts array)
	 * @param	OutReason			Allows callers to receive a localized string containing more detail when the function is determined to be invalid (optional)
	 */
	UE_API bool CanFunctionBeUsedInGraph(const UClass* InClass, const UFunction* InFunction, const UEdGraph* InDestGraph, uint32 InFunctionTypes, bool bInCalledForEach, FText* OutReason = nullptr) const;

	/**
	 * Makes connections into/or out of the gateway node, connect directly to the associated networks on the opposite side of the tunnel
	 * When done, none of the pins on the gateway node will be connected to anything.
	 * Requires both this gateway node and it's associated node to be in the same graph already (post-merging)
	 *
	 * @param InGatewayNode			The function or tunnel node
	 * @param InEntryNode			The entry node in the inner graph
	 * @param InResultNode			The result node in the inner graph
	 *
	 * @return						Returns TRUE if successful
	 */
	UE_API bool CollapseGatewayNode(UK2Node* InNode, UEdGraphNode* InEntryNode, UEdGraphNode* InResultNode, class FKismetCompilerContext* CompilerContext = nullptr, TSet<UEdGraphNode*>* OutExpandedNodes = nullptr) const;

	/** 
	 * Connects all of the linked pins from PinA to all of the linked pins from PinB, removing
	 * both PinA and PinB from being linked to anything else
	 * Requires the nodes that own the pins to be in the same graph already (post-merging)
	 */
	UE_API void CombineTwoPinNetsAndRemoveOldPins(UEdGraphPin* InPinA, UEdGraphPin* InPinB) const;

	/**
	 * Make links from all data pins from InOutputNode output to InInputNode input.
	 */
	UE_API void LinkDataPinFromOutputToInput(UEdGraphNode* InOutputNode, UEdGraphNode* InInputNode) const;

	/** Moves all connections from the old node to the new one. Returns true and destroys OldNode on success. Fails if it cannot find a mapping from an old pin. */
	UE_API bool ReplaceOldNodeWithNew(UEdGraphNode* OldNode, UEdGraphNode* NewNode, const TMap<FName, FName>& OldPinToNewPinMap) const;

	/** Convert a deprecated node into a function call node, called from per-node ConvertDeprecatedNode */
	UE_API UK2Node* ConvertDeprecatedNodeToFunctionCall(UK2Node* OldNode, UFunction* NewFunction, TMap<FName, FName>& OldPinToNewPinMap, UEdGraph* Graph) const;

	/** some inherited schemas don't want anim-notify actions listed, so this is an easy way to check that */
	virtual bool DoesSupportAnimNotifyActions() const { return true; }

	///////////////////////////////////////////////////////////////////////////////////
	// NonExistent Variables: Broken Get/Set Nodes where the variable is does not exist 

	/** Create the variable that the broken node refers to */
	static UE_API void OnCreateNonExistentVariable(class UK2Node_Variable* Variable, UBlueprint* OwnerBlueprint);

	/** Create the local variable that the broken node refers to */
	static UE_API void OnCreateNonExistentLocalVariable(class UK2Node_Variable* Variable, UBlueprint* OwnerBlueprint);

	/** Replace the variable that a variable node refers to when the variable it refers to does not exist */
	static UE_API void OnReplaceVariableForVariableNode(class UK2Node_Variable* Variable, UBlueprint* OwnerBlueprint, FName VariableName, bool bIsSelfMember);

	/** Create sub menu that shows all possible variables that can be used to replace the existing variable reference */
	static UE_API void GetReplaceVariableMenu(UToolMenu* Menu, class UK2Node_Variable* Variable, UBlueprint* OwnerBlueprint, bool bReplaceExistingVariable = false);

	/** Function called when the owning module is shut down */ 
	static UE_API void Shutdown();
private:

	static TSharedPtr<IAssetReferenceFilter> MakeAssetReferenceFilter(const UBlueprint* Blueprint);

	/**
	 * Returns true if the specified function has any out parameters
	 * @param [in] Function	The function to check for out parameters
	 * @return true if there are out parameters, else false
	 */
	UE_API bool DoesFunctionHaveOutParameters( const UFunction* Function ) const;
	
	UE_API bool ShouldShowPanelContextMenuForIncompatibleConnectionsImpl() const override;

	static UE_API const UScriptStruct* VectorStruct;
	static UE_API const UScriptStruct* Vector3fStruct;
	static UE_API const UScriptStruct* Vector2DStruct;
	static UE_API const UScriptStruct* RotatorStruct;
	static UE_API const UScriptStruct* TransformStruct;
	static UE_API const UScriptStruct* LinearColorStruct;
	static UE_API const UScriptStruct* ColorStruct;
};

#undef UE_API
