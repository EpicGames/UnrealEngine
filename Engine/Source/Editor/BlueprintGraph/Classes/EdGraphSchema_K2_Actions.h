// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "EdGraphSchema_K2_Actions.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UBlueprint;
class UEdGraph;
class UEdGraphNode;
class UK2Node;

/*******************************************************************************
* FEdGraphSchemaAction_K2NewNode
*******************************************************************************/

enum class EK2NewNodeFlags
{
	None = 0x0,

	SelectNewNode = 0x1,
	GotoNewNode = 0x2,
};
ENUM_CLASS_FLAGS(EK2NewNodeFlags)

/** Action to add a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2NewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2NewNode"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** Template of node we want to create */
	UPROPERTY()
	TObjectPtr<class UK2Node> NodeTemplate;

	UPROPERTY()
	bool bGotoNode;

	FEdGraphSchemaAction_K2NewNode() 
		: FEdGraphSchemaAction()
		, NodeTemplate(nullptr)
		, bGotoNode(false)
	{}

	FEdGraphSchemaAction_K2NewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, FText InKeywords = FText())
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, MoveTemp(InKeywords))
		, NodeTemplate(nullptr)
		, bGotoNode(false)
	{}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FEdGraphSchemaAction interface

	template <typename NodeType>
	static NodeType* SpawnNodeFromTemplate(class UEdGraph* ParentGraph, NodeType* InTemplateNode, const FVector2D Location, bool bSelectNewNode = true)
	{
		EK2NewNodeFlags Options = EK2NewNodeFlags::None;
		if (bSelectNewNode)
		{
			Options |= EK2NewNodeFlags::SelectNewNode;
		}
		return Cast<NodeType>(CreateNode(ParentGraph, TArrayView<UEdGraphPin*>(), Location, InTemplateNode, Options));
	}

	template <typename NodeType>
	static NodeType* SpawnNode(class UEdGraph* ParentGraph, const FVector2D Location, EK2NewNodeFlags Options, TFunctionRef<void(NodeType*)> InitializerFn)
	{
		UClass* NodeClass = NodeType::StaticClass();
		return Cast<NodeType>(CreateNode(
			ParentGraph,
			TArrayView<UEdGraphPin*>(),
			Location,
			[NodeClass](UEdGraph* InParentGraph)->UK2Node*
			{
				return NewObject<UK2Node>(InParentGraph, NodeClass);
			},
			[InitializerFn](UK2Node* NewNode)
			{
				InitializerFn(CastChecked<NodeType>(NewNode));
			},
			Options));
	}

	template <typename NodeType>
	static NodeType* SpawnNode(class UEdGraph* ParentGraph, const FVector2D Location, EK2NewNodeFlags Options)
	{
		return SpawnNode<NodeType>(ParentGraph, Location, Options, [](NodeType*){});
	}

	static UE_API UEdGraphNode* CreateNode(UEdGraph* ParentGraph, TArrayView<UEdGraphPin*> FromPins, const FVector2D Location, UK2Node* NodeTemplate, EK2NewNodeFlags Options);
	static UE_API UEdGraphNode* CreateNode(UEdGraph* ParentGraph, TArrayView<UEdGraphPin*> FromPins, const FVector2D Location, TFunctionRef<UK2Node*(UEdGraph*)> ConstructionFn, TFunctionRef<void(UK2Node*)> InitializerFn, EK2NewNodeFlags Options);
};

/*******************************************************************************
* FEdGraphSchemaAction_K2ViewNode
*******************************************************************************/

/** Action to view a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2ViewNode : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2ViewNode"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** node we want to view */
	UPROPERTY()
	TObjectPtr<const UK2Node> NodePtr;

	FEdGraphSchemaAction_K2ViewNode() 
		: FEdGraphSchemaAction()
		, NodePtr(nullptr)
	{}

	FEdGraphSchemaAction_K2ViewNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, NodePtr(nullptr)
	{}

	// FEdGraphSchemaAction interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface

};

/*******************************************************************************
* FEdGraphSchemaAction_K2AssignDelegate
*******************************************************************************/

/** Action to add a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AssignDelegate : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()

	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AssignDelegate"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2AssignDelegate() 
		:FEdGraphSchemaAction_K2NewNode()
	{}

	FEdGraphSchemaAction_K2AssignDelegate(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping) 
	{}

	// FEdGraphSchemaAction interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface

	static UE_API UEdGraphNode* AssignDelegate(class UK2Node* NodeTemplate, class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, bool bSelectNewNode);
};

/*******************************************************************************
* FEdGraphSchemaAction_EventFromFunction
*******************************************************************************/

/** Action to add a node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_EventFromFunction : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_EventFromFunction"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	UPROPERTY()
	TObjectPtr<class UFunction> SignatureFunction;

	FEdGraphSchemaAction_EventFromFunction() 
		:FEdGraphSchemaAction()
		, SignatureFunction(nullptr)
	{}

	FEdGraphSchemaAction_EventFromFunction(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, SignatureFunction(nullptr)
	{}

	// FEdGraphSchemaAction interface
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode = true) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2AddComponent
*******************************************************************************/

/** Action to add an 'add component' node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AddComponent : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AddComponent"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** Class of component we want to add */
	UPROPERTY()
	TSubclassOf<class UActorComponent> ComponentClass;

	/** Option asset to assign to newly created component */
	UPROPERTY()
	TObjectPtr<class UObject> ComponentAsset;

	FEdGraphSchemaAction_K2AddComponent()
		: FEdGraphSchemaAction_K2NewNode()
		, ComponentClass(NULL)
		, ComponentAsset(NULL)
	{}

	FEdGraphSchemaAction_K2AddComponent(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, ComponentClass(NULL)
		, ComponentAsset(NULL)
	{}

	// FEdGraphSchemaAction interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2AddEvent
*******************************************************************************/

/** Action to add a 'event' node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AddEvent : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AddEvent"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2AddEvent()
		: FEdGraphSchemaAction_K2NewNode()
	{}

	FEdGraphSchemaAction_K2AddEvent(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface

	/** */
	BLUEPRINTGRAPH_API bool EventHasAlreadyBeenPlaced(UBlueprint const* Blueprint, class UK2Node_Event const** FoundEventOut = NULL) const;
};

/*******************************************************************************
* FEdGraphSchemaAction_K2AddCustomEvent
*******************************************************************************/

/** Action to add a 'custom event' node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AddCustomEvent : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AddCustomEvent"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2AddCustomEvent()
		: FEdGraphSchemaAction_K2NewNode()
	{}

	FEdGraphSchemaAction_K2AddCustomEvent(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	BLUEPRINTGRAPH_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2AddCallOnActor
*******************************************************************************/

/** Action to add a 'call function on actor(s)' set of nodes to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AddCallOnActor : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AddCallOnActor"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** Pointer to actors in level we want to call function on */
	UPROPERTY()
	TArray<TObjectPtr<class AActor>> LevelActors;

	FEdGraphSchemaAction_K2AddCallOnActor()
		: FEdGraphSchemaAction_K2NewNode()
	{}

	FEdGraphSchemaAction_K2AddCallOnActor(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// End of FEdGraphSchemaAction interface	
};

/*******************************************************************************
* FEdGraphSchemaAction_K2AddComment
*******************************************************************************/

/** Action to add a 'comment' node to the graph */
USTRUCT()
struct FEdGraphSchemaAction_K2AddComment : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2AddComment"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2AddComment()
		: FEdGraphSchemaAction(FText(), NSLOCTEXT("K2AddComment", "AddComment", "Add Comment..."), NSLOCTEXT("K2AddComment", "AddComment_Tooltip", "Create a resizable comment box."), 0)
	{
	}

	FEdGraphSchemaAction_K2AddComment(FText InDescription, FText InToolTip)
		: FEdGraphSchemaAction(FText(), MoveTemp(InDescription), MoveTemp(InToolTip), 0)
	{
	}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	BLUEPRINTGRAPH_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2TargetNode
*******************************************************************************/

/** Action to target a specific node on graph */
USTRUCT()
struct FEdGraphSchemaAction_K2TargetNode : public FEdGraphSchemaAction_K2NewNode
{
	GENERATED_USTRUCT_BODY()


	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2TargetNode"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2TargetNode()
		: FEdGraphSchemaAction_K2NewNode()
	{}

	FEdGraphSchemaAction_K2TargetNode(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction_K2NewNode(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2PasteHere
*******************************************************************************/

/** Action to paste at this location on graph*/
USTRUCT()
struct FEdGraphSchemaAction_K2PasteHere : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2PasteHere"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_K2PasteHere()
		: FEdGraphSchemaAction()
	{}

	FEdGraphSchemaAction_K2PasteHere (FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
	{}

	// FEdGraphSchemaAction interface
	using FEdGraphSchemaAction::PerformAction; // Prevent hiding of deprecated base class function with FVector2D
	UE_API virtual UEdGraphNode* PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode = true) override;
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2Enum
*******************************************************************************/

/** Reference to an enumeration (only used in 'docked' palette) */
USTRUCT()
struct FEdGraphSchemaAction_K2Enum : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2Enum"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	TObjectPtr<UEnum> Enum;

	void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		if( Enum )
		{
			Collector.AddReferencedObject(Enum);
		}
	}

	FName GetPathName() const
	{
		return Enum ? FName(*Enum->GetPathName()) : NAME_None;
	}

	FEdGraphSchemaAction_K2Enum() 
		: FEdGraphSchemaAction()
		, Enum(nullptr)
	{}

	FEdGraphSchemaAction_K2Enum(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping)
		, Enum(nullptr)
	{}
};

/*******************************************************************************
* FEdGraphSchemaAction_BlueprintVariableBase
*******************************************************************************/

/** Reference to a variable (only used in 'My Blueprints' but used for member variables, local variables, delegates, etc...) */
USTRUCT()
struct FEdGraphSchemaAction_BlueprintVariableBase : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

private:
	/** Name of function or class */
	FName VarName;

	/** The struct that owns this item */
	TWeakObjectPtr<UObject> VariableSource;

	/** TRUE if the variable's type is boolean */
	bool bIsVarBool;

public:
	void SetVariableInfo(const FName& InVarName, const UObject* InOwningScope, bool bInIsVarBool)
	{
		VarName = InVarName;
		bIsVarBool = bInIsVarBool;

		check(InOwningScope);
		VariableSource = MakeWeakObjectPtr(const_cast<UObject*>(InOwningScope));
	}

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_BlueprintVariableBase"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	FEdGraphSchemaAction_BlueprintVariableBase()
		: FEdGraphSchemaAction()
	{}

	FEdGraphSchemaAction_BlueprintVariableBase(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, FText(), InSectionID)
	{}

	FName GetVariableName() const
	{
		return VarName;
	}

	FString GetFriendlyVariableName() const
	{
		return FName::NameToDisplayString( VarName.ToString(), bIsVarBool );
	}

	virtual UClass* GetVariableClass() const
	{
		return Cast<UClass>(GetVariableScope());
	}

	UObject* GetVariableScope() const
	{
		return VariableSource.Get();
	}

	virtual FProperty* GetProperty() const
	{
		if (UStruct* Scope = Cast<UStruct>(GetVariableScope()))
		{
			return FindFProperty<FProperty>(Scope, VarName);
		}
		return nullptr;
	}

	virtual FEdGraphPinType GetPinType() const{ return FEdGraphPinType();}

	virtual void ChangeVariableType(const FEdGraphPinType& NewPinType){}

	virtual void RenameVariable(const FName& NewName) { VarName = NewName; }

	virtual bool IsValidName(const FName& NewName, FText& OutErrorMessage) const { return true; }

	virtual void DeleteVariable() {}

	virtual bool IsVariableUsed() { return false; }
	
	// FEdGraphSchemaAction interface
	UE_API virtual void MovePersistentItemToCategory(const FText& NewCategoryName) override;
	UE_API virtual int32 GetReorderIndexInContainer() const override;
	UE_API virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) override;
	UE_API virtual FEdGraphSchemaActionDefiningObject GetPersistentItemDefiningObject() const override;
	virtual bool IsAVariable() const { return true; }
	// End of FEdGraphSchemaAction interface

	UE_API UBlueprint* GetSourceBlueprint() const;
};


/*******************************************************************************
* FEdGraphSchemaAction_K2Var
*******************************************************************************/

/** Reference to a variable (only used in 'docked' palette) */
USTRUCT()
struct FEdGraphSchemaAction_K2Var : public FEdGraphSchemaAction_BlueprintVariableBase
{
	GENERATED_USTRUCT_BODY()

public:
	// Simple type info
	static FName StaticGetTypeId() { static FName Type("FEdGraphSchemaAction_K2Var"); return Type; }
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }

	FEdGraphSchemaAction_K2Var()
		: FEdGraphSchemaAction_BlueprintVariableBase()
	{}

	FEdGraphSchemaAction_K2Var(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID)
		: FEdGraphSchemaAction_BlueprintVariableBase(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InSectionID)
	{}

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId() || InType == FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId();
	}
};

/*******************************************************************************
* FEdGraphSchemaAction_K2LocalVar
*******************************************************************************/

/** Reference to a local variable (only used in 'docked' palette) */
USTRUCT()
struct FEdGraphSchemaAction_K2LocalVar : public FEdGraphSchemaAction_BlueprintVariableBase
{
	GENERATED_USTRUCT_BODY()

public:
	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2LocalVar"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 
	UE_API virtual int32 GetReorderIndexInContainer() const override;

	FEdGraphSchemaAction_K2LocalVar() 
		: FEdGraphSchemaAction_BlueprintVariableBase()
	{}

	FEdGraphSchemaAction_K2LocalVar(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID)
		: FEdGraphSchemaAction_BlueprintVariableBase(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InSectionID)
	{}

	virtual bool IsA(const FName& InType) const override
    {
    	return InType == GetTypeId() || InType == FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId();
    }

	virtual UClass* GetVariableClass() const override
	{
		UObject* VariableScope = GetVariableScope();
		return VariableScope ? VariableScope->GetTypedOuter<UClass>() : nullptr;
	}
};

/*******************************************************************************
* FEdGraphSchemaAction_K2Graph
*******************************************************************************/

/** The graph type that the schema is */
UENUM()
namespace EEdGraphSchemaAction_K2Graph
{
	enum Type : int
	{
		Graph,
		Subgraph,
		Function,
		Interface,
		Macro,
		MAX
	};
}

/** Reference to a function, macro, event graph, or timeline (only used in 'docked' palette) */
USTRUCT()
struct FEdGraphSchemaAction_K2Graph : public FEdGraphSchemaAction
{
	GENERATED_USTRUCT_BODY()

	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2Graph"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** Name of function or class */
	FName FuncName;

	/** The type of graph that action is */
	EEdGraphSchemaAction_K2Graph::Type GraphType;

	/** The associated editor graph for this schema */
	UEdGraph* EdGraph;

	FEdGraphSchemaAction_K2Graph() 
		: FEdGraphSchemaAction()
	{}

	FEdGraphSchemaAction_K2Graph(EEdGraphSchemaAction_K2Graph::Type InType, FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID = 0)
		: FEdGraphSchemaAction(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, FText(), InSectionID)
		, GraphType(InType)
		, EdGraph(nullptr)
	{}

	// FEdGraphSchemaAction interface
	virtual bool IsParentable() const override { return true; }
	UE_API virtual void MovePersistentItemToCategory(const FText& NewCategoryName) override;
	UE_API virtual int32 GetReorderIndexInContainer() const override;
	UE_API virtual bool ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction) override;
	UE_API virtual FEdGraphSchemaActionDefiningObject GetPersistentItemDefiningObject() const override;
	// End of FEdGraphSchemaAction interface

	UE_API UFunction* GetFunction() const;
	UE_API UBlueprint* GetSourceBlueprint() const;
};

/*******************************************************************************
* FEdGraphSchemaAction_K2Event
*******************************************************************************/

/**
* A reference to a specific event (living inside a Blueprint graph)... intended
* to be used the 'docked' palette only.
*/
USTRUCT()
struct FEdGraphSchemaAction_K2Event : public FEdGraphSchemaAction_K2TargetNode
{
	GENERATED_USTRUCT_BODY()

	/**
	 * An empty default constructor (stubbed out for arrays and other containers 
	 * that need to allocate default instances of this struct)
	 */
	FEdGraphSchemaAction_K2Event() 
		: FEdGraphSchemaAction_K2TargetNode()
	{}

	/**
	 * The primary constructor, used to customize the Super FEdGraphSchemaAction.
	 * 
	 * @param   Category		The tree parent header (or path, delimited by '|') you want to sort this action under.
	 * @param   MenuDescription	The string you want displayed in the tree, corresponding to this action.
	 * @param   Tooltip			A string to display when hovering over this action entry.
	 * @param   Grouping		Used to override list ordering (actions with the same number get grouped together, higher numbers get sorted first).
	 */
	FEdGraphSchemaAction_K2Event(FText Category, FText MenuDescription, FText Tooltip, int32 const Grouping)
		: FEdGraphSchemaAction_K2TargetNode(MoveTemp(Category), MoveTemp(MenuDescription), MoveTemp(Tooltip), Grouping)
	{}

	/**
	 * Provides a set identifier for all FEdGraphSchemaAction_K2Event actions.
	 * 
	 * @return A static identifier for actions of this type.
	 */
	static FName StaticGetTypeId() 
	{
		static FName Type("FEdGraphSchemaAction_K2Event"); 
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	virtual bool IsParentable() const override { return true; }
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2InputAction
*******************************************************************************/

/**
* A reference to a specific event (living inside a Blueprint graph)... intended
* to be used the 'docked' palette only.
*/
USTRUCT()
struct FEdGraphSchemaAction_K2InputAction : public FEdGraphSchemaAction_K2TargetNode
{
	GENERATED_USTRUCT_BODY()

	/**
	 * An empty default constructor (stubbed out for arrays and other containers
	 * that need to allocate default instances of this struct)
	 */
	FEdGraphSchemaAction_K2InputAction()
		: FEdGraphSchemaAction_K2TargetNode()
	{}

	/**
	* The primary constructor, used to customize the Super FEdGraphSchemaAction.
	*
	* @param   Category		The tree parent header (or path, delimited by '|') you want to sort this action under.
	* @param   MenuDescription	The string you want displayed in the tree, corresponding to this action.
	* @param   Tooltip			A string to display when hovering over this action entry.
	* @param   Grouping		Used to override list ordering (actions with the same number get grouped together, higher numbers get sorted first).
	*/
	FEdGraphSchemaAction_K2InputAction(FText Category, FText MenuDescription, FText Tooltip, int32 const Grouping)
		: FEdGraphSchemaAction_K2TargetNode(MoveTemp(Category), MoveTemp(MenuDescription), MoveTemp(Tooltip), Grouping)
	{}

	/**
	* Provides a set identifier for all FEdGraphSchemaAction_K2InputAction actions.
	*
	* @return A static identifier for actions of this type.
	*/
	static FName StaticGetTypeId()
	{
		static FName Type("FEdGraphSchemaAction_K2InputAction");
		return Type;
	}

	// FEdGraphSchemaAction interface
	virtual FName GetTypeId() const override { return StaticGetTypeId(); }
	virtual bool IsParentable() const override { return true; }
	// End of FEdGraphSchemaAction interface
};

/*******************************************************************************
* FEdGraphSchemaAction_K2Delegate
*******************************************************************************/

/** Reference to a delegate */
USTRUCT()
struct FEdGraphSchemaAction_K2Delegate : public FEdGraphSchemaAction_BlueprintVariableBase
{
	GENERATED_USTRUCT_BODY()

public:		
	// Simple type info
	static FName StaticGetTypeId() {static FName Type("FEdGraphSchemaAction_K2Delegate"); return Type;}
	virtual FName GetTypeId() const override { return StaticGetTypeId(); } 

	/** The associated editor graph for this schema */
	UEdGraph* EdGraph;

	FEdGraphSchemaAction_K2Delegate() 
		: FEdGraphSchemaAction_BlueprintVariableBase()
		, EdGraph(nullptr)
	{}

	FEdGraphSchemaAction_K2Delegate(FText InNodeCategory, FText InMenuDesc, FText InToolTip, const int32 InGrouping, const int32 InSectionID)
		: FEdGraphSchemaAction_BlueprintVariableBase(MoveTemp(InNodeCategory), MoveTemp(InMenuDesc), MoveTemp(InToolTip), InGrouping, InSectionID)
		, EdGraph(nullptr)
	{}

	FName GetDelegateName() const
	{
		return GetVariableName();
	}

	UClass* GetDelegateClass() const
	{
		return GetVariableClass();
	}

	FMulticastDelegateProperty* GetDelegateProperty() const
	{
		return FindFProperty<FMulticastDelegateProperty>(GetVariableClass(), GetVariableName());
	}

	virtual bool IsA(const FName& InType) const override
	{
		return InType == GetTypeId() || InType == FEdGraphSchemaAction_BlueprintVariableBase::StaticGetTypeId();
	}
};


#undef UE_API
