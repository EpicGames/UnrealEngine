// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeSignature.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/NotNull.h"
#include "Templates/SubclassOf.h"
#include "UObject/LinkerLoad.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class AActor;
class FArchive;
class FBlueprintActionDatabaseRegistrar;
class FCompilerResultsLog;
class FKismetCompilerContext;
class FProperty;
class UActorComponent;
class UBlueprint;
class UClass;
class UDynamicBlueprintBinding;
class UEdGraph;
class UEdGraphPin;
class UEdGraphSchema;
class UFunction;
class UK2Node;
class UStruct;
struct FMemberReference;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnUserDefinedPinRenamed, UK2Node*, FName, FName);

/** Helper structure to cache old data for optional pins so the data can be restored during reconstruction */
struct FOldOptionalPinSettings
{
	/** TRUE if optional pin was previously visible */
	uint8 bOldVisibility:1;
	/** TRUE if the optional pin's override value was previously enabled */
	uint8 bIsOldOverrideEnabled:1;
	/** TRUE if the optional pin's value was previously editable */
	uint8 bIsOldSetValuePinVisible:1;
	/** TRUE if the optional pin's override value was previously editable */
	uint8 bIsOldOverridePinVisible:1;

	FOldOptionalPinSettings(bool bInOldVisibility, bool bInIsOldOverrideEnabled, bool bInIsOldSetValuePinVisible, bool bInIsOldOverridePinVisible)
		: bOldVisibility(bInOldVisibility)
		, bIsOldOverrideEnabled(bInIsOldOverrideEnabled)
		, bIsOldSetValuePinVisible(bInIsOldSetValuePinVisible)
		, bIsOldOverridePinVisible(bInIsOldOverridePinVisible)
	{}
};

USTRUCT()
struct FOptionalPinFromProperty
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category= OptionalPin)
	FName PropertyName;

	UPROPERTY(EditAnywhere, Category= OptionalPin)
	FString PropertyFriendlyName;

	UPROPERTY(EditAnywhere, Category= OptionalPin)
	FText PropertyTooltip;

	UPROPERTY(EditAnywhere, Category=OptionalPin)
	FName CategoryName;

	UPROPERTY(EditAnywhere, Category=OptionalPin)
	uint8 bShowPin:1;

	UPROPERTY(EditAnywhere, Category=OptionalPin)
	uint8 bCanToggleVisibility:1;

	UPROPERTY(EditAnywhere, Category=OptionalPin)
	uint8 bPropertyIsCustomized:1;
	
	UPROPERTY(EditAnywhere, Category=OptionalPin)
	uint8 bHasOverridePin:1;

	UPROPERTY(EditAnywhere, Category=OptionalPin)
	uint8 bIsMarkedForAdvancedDisplay:1;

	/** TRUE if the override value is enabled for use */
	UPROPERTY(EditAnywhere, Category = OptionalPin)
	uint8 bIsOverrideEnabled:1;

	/** TRUE if the override value should be set through this pin */
	UPROPERTY(EditAnywhere, Category = OptionalPin)
	uint8 bIsSetValuePinVisible:1;

	/** TRUE if the override pin is visible */
	UPROPERTY(EditAnywhere, Category = OptionalPin)
	uint8 bIsOverridePinVisible:1;

	FOptionalPinFromProperty()
		: bShowPin(false)
		, bCanToggleVisibility(false)
		, bPropertyIsCustomized(false)
		, bHasOverridePin(false)
		, bIsMarkedForAdvancedDisplay(false)
		, bIsOverrideEnabled(true)
		, bIsSetValuePinVisible(true)
		, bIsOverridePinVisible(true)
	{
	}

	FOptionalPinFromProperty(FName InPropertyName, bool bInShowPin, bool bInCanToggleVisibility, const FString& InFriendlyName, const FText& InTooltip, bool bInPropertyIsCustomized, FName InCategoryName, bool bInHasOverridePin)
		: PropertyName(InPropertyName)
		, PropertyFriendlyName(InFriendlyName)
#if WITH_EDITORONLY_DATA
		, PropertyTooltip(InTooltip)
#endif
		, CategoryName(InCategoryName)
		, bShowPin(bInShowPin)
		, bCanToggleVisibility(bInCanToggleVisibility)
		, bPropertyIsCustomized(bInPropertyIsCustomized)
		, bHasOverridePin(bInHasOverridePin)
		, bIsMarkedForAdvancedDisplay(false)
		, bIsOverrideEnabled(true)
		, bIsSetValuePinVisible(true)
		, bIsOverridePinVisible(true)
	{
	}
};

// Manager to build or refresh a list of optional pins
struct FOptionalPinManager
{
public:
	virtual ~FOptionalPinManager() = default;

	/** Should the specified property be displayed by default */
	UE_API virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const;

	/** Can this property be managed as an optional pin (with the ability to be shown or hidden) */
	UE_API virtual bool CanTreatPropertyAsOptional(FProperty* TestProperty) const;

	/** 
	 * Reconstructs the specified property array using the SourceStruct
	 * @param [in,out] Properties	The property array
	 * @param SourceStruct			The source structure to update the properties array from
	 * @param bDefaultVisibility
	 */
	UE_API void RebuildPropertyList(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct);

	/**
	 * Creates a pin for each visible property on the specified node
	 * @param Properties	The property array
	 * @param SourceStruct	The source structure to update the properties array from
	 * @param Direction		Direction of pins to create
	 * @param TargetNode	Node to generate pins for
	 * @param StructBasePtr	Struct that is iterated over, to represent the current version of the data
	 * @param DefaultsPtr	Struct that is iterated over, to represent the default version of the data, used for delta serialization
	 */
	UE_API void CreateVisiblePins(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct, EEdGraphPinDirection Direction, class UK2Node* TargetNode, uint8* StructBasePtr = nullptr, uint8* DefaultsPtr = nullptr);

	// Customize automatically created pins if desired
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property = NULL) const {}

	/** Helper function to make consistent behavior between nodes that use optional pins */
	static UE_API void CacheShownPins(const TArray<FOptionalPinFromProperty>& OptionalPins, TArray<FName>& OldShownPins);

	/** Helper function to make consistent behavior between nodes that use optional pins */
	static UE_API void EvaluateOldShownPins(const TArray<FOptionalPinFromProperty>& OptionalPins, TArray<FName>& OldShownPins, UK2Node* Node);


protected:
	virtual void PostInitNewPin(UEdGraphPin* Pin, FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const {}
	virtual void PostRemovedOldPin(FOptionalPinFromProperty& Record, int32 ArrayIndex, FProperty* Property, uint8* PropertyAddress, uint8* DefaultPropertyAddress) const {}
	UE_API void RebuildProperty(FProperty* TestProperty, FName CategoryName, TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct, TMap<FName, FOldOptionalPinSettings>& OldSettings);
};

enum ERenamePinResult
{
	ERenamePinResult_Success,
	ERenamePinResult_NoSuchPin,
	ERenamePinResult_NameCollision
};

/**
 * Abstract base class of all blueprint graph nodes.
 */
UCLASS(abstract, MinimalAPI)
class UK2Node : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

public:
	// UObject interface
	BLUEPRINTGRAPH_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	BLUEPRINTGRAPH_API virtual void PostLoad() override;
	BLUEPRINTGRAPH_API virtual void Serialize(FArchive& Ar) override;
	BLUEPRINTGRAPH_API virtual bool HasNonEditorOnlyReferences() const override;
	// End of UObject interface

	// UEdGraphNode interface
	BLUEPRINTGRAPH_API virtual void ReconstructNode() override;
	BLUEPRINTGRAPH_API virtual FLinearColor GetNodeTitleColor() const override;
	BLUEPRINTGRAPH_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	BLUEPRINTGRAPH_API void PinConnectionListChanged(UEdGraphPin* Pin) override;
	BLUEPRINTGRAPH_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	BLUEPRINTGRAPH_API virtual bool CanJumpToDefinition() const override;
	BLUEPRINTGRAPH_API virtual void JumpToDefinition() const override;
	BLUEPRINTGRAPH_API virtual FString GetDocumentationLink() const override;
	BLUEPRINTGRAPH_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	virtual bool ShowPaletteIconOnNode() const override { return true; }
	BLUEPRINTGRAPH_API virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	BLUEPRINTGRAPH_API virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;
	BLUEPRINTGRAPH_API virtual bool IsInDevelopmentMode() const override;
	BLUEPRINTGRAPH_API virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	BLUEPRINTGRAPH_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	// End of UEdGraphNode interface

	// K2Node interface

	/**
	 * Reallocate pins during reconstruction; by default ignores the old pins and calls AllocateDefaultPins()
	 * If you override this to create additional pins you likely need to call RestoreSplitPins to restore any
	 * pins that have been split (e.g. a vector pin split into its components)
	 */
	BLUEPRINTGRAPH_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins);

	/** Returns whether this node is considered 'pure' by the compiler */
	virtual bool IsNodePure() const { return false; }

	/** Returns whether this node can have breakpoints placed on it in the debugger */
	BLUEPRINTGRAPH_API virtual bool CanPlaceBreakpoints() const;

	/** Return whether to draw this node as an entry */
	virtual bool DrawNodeAsEntry() const { return false; }

	/** Return whether to draw this node as an entry */
	virtual bool DrawNodeAsExit() const { return false; }

	/** Return whether to draw this node as a small variable node */
	virtual bool DrawNodeAsVariable() const { return false; }

	/** Should draw compact */
	virtual bool ShouldDrawCompact() const { return false; }

	/** Return title if drawing this node in 'compact' mode */
	virtual FText GetCompactNodeTitle() const { return GetNodeTitle(ENodeTitleType::FullTitle); }

	/** */
	BLUEPRINTGRAPH_API virtual FText GetToolTipHeading() const;

	/**
	 * Determine if the node of this type should be filtered in the actions menu
	 */
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) { return false; }

	/** Return whether the node's properties display in the blueprint details panel */
	virtual bool ShouldShowNodeProperties() const { return false; }

	/** Return whether the node's execution pins should support the insert execution pin action */
	virtual bool CanEverInsertExecutionPin() const { return false; }

	/** Return whether the node's execution pins should support the remove execution pin action */
	virtual bool CanEverRemoveExecutionPin() const { return false; }

	/** Called when the connection list of one of the pins of this node is changed in the editor, after the pin has had it's literal cleared */
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) {}

	/**
	 * Creates the pins required for a function entry or exit node (only inputs as outputs or outputs as inputs respectively)
	 *
	 * @param	Function	The function being implemented that the entry or exit nodes are for.
	 * @param	bForFunctionEntry	True indicates a function entry node, false indicates an exit node.
	 *
	 * @return	true on success.
	 */
	bool CreatePinsForFunctionEntryExit(const UFunction* Function, bool bForFunctionEntry);

	/** Expands a node while compiling, which may add additional nodes or delete this node */
	BLUEPRINTGRAPH_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph);

	/** Clears out any cached data that needs to be regenerated after a structural blueprint change */
	virtual void ClearCachedBlueprintData(UBlueprint* Blueprint) {}

	/** Performs a node-specific deprecation fixup, which may delete this node and replace it with another one */
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) {}

	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const { return nullptr; }

	BLUEPRINTGRAPH_API UK2Node* ExpandSplitPin(class FKismetCompilerContext* CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* Pin);

	/**
	 * Query if this is a node that is safe to ignore (e.g., a comment node or other non-structural annotation that can be pruned with no warnings).
	 *
	 * @return	true if node safe to ignore.
	 */
	virtual bool IsNodeSafeToIgnore() const { return false; }

	/** Query if this node should be kept as part of the root set during the pruning stage of compilation. Prevents the node being pruned as isolated. */
	virtual bool IsNodeRootSet() const { return false; }

	/** Called at the end of ReconstructNode, allows node specific work to be performed */
	BLUEPRINTGRAPH_API virtual void PostReconstructNode();

	/** Return true if adding/removing this node requires calling MarkBlueprintAsStructurallyModified on the Blueprint */
	virtual bool NodeCausesStructuralBlueprintChange() const { return false; }

	/** 
	 * Return true if this node has a valid blueprint outer, or false otherwise.
	 * This is useful for checking validity of the node for UI list refreshes, 
	 * which may operate on stale nodes for a frame until the list is refreshed.
	 */
	BLUEPRINTGRAPH_API bool HasValidBlueprint() const;

	/** Get the Blueprint object to which this node belongs. This will return null if HasValidBlueprint is false */
	BLUEPRINTGRAPH_API UBlueprint* GetBlueprint() const;

	/** Get the Blueprint object to which this node belongs. This will assert if HasValidBlueprint is false */
	BLUEPRINTGRAPH_API TNotNull<UBlueprint*> GetBlueprintChecked() const;

	/** Get the input execution pin of this node (if one exists)*/
	BLUEPRINTGRAPH_API UEdGraphPin* GetExecPin() const;

	/** Get the output then pin of this node (if one exists)*/
	BLUEPRINTGRAPH_API UEdGraphPin* GetThenPin() const;

	/**
	 * If this node references an actor in the level that should be selectable by "Find Actors In Level," this will return a reference to that actor
	 *
	 * @return	Reference to an actor corresponding to this node, or NULL if no actors are referenced
	 */
	virtual AActor* GetReferencedLevelActor() const { return nullptr; }

	// Can this node be created under the specified schema
	BLUEPRINTGRAPH_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;

	// Returns which dynamic binding class (if any) to use for this node
	virtual UClass* GetDynamicBindingClass() const { return NULL; }

	// Puts information about this node into the dynamic binding object
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const { };

	// Renames a user defined pin and broadcasts the result to any OnUserDefinedPinRenamed listeners on success (if bTest is false)
	BLUEPRINTGRAPH_API ERenamePinResult RenameUserDefinedPin(const FName OldName, const FName NewName, bool bTest = false);

	/** Retrieves a delegate that is called when a user-defined pin on this node is renamed */
	BLUEPRINTGRAPH_API FOnUserDefinedPinRenamed& OnUserDefinedPinRenamed();

	/**
	 * Handles inserting the node between the FromPin and what the FromPin was original connected to
	 *
	 * @param FromPin			The pin this node is being spawned from
	 * @param NewLinkPin		The new pin the FromPin will connect to
	 * @param OutNodeList		Any nodes that are modified will get added to this list for notification purposes
	 */
	void InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList);

	/** @return true if this function can be called on multiple contexts at once */
	virtual bool AllowMultipleSelfs(bool bInputAsArray) const { return false; }

	/** @return name of brush for special icon in upper-right corner */
	virtual FName GetCornerIcon() const { return FName(); }

	/** @return true, is pins cannot be connected due to node's inner logic, put message for user in OutReason */
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const { return false; }

	/** This function if used for nodes that needs CDO for validation (Called before expansion)*/
	BLUEPRINTGRAPH_API virtual void EarlyValidation(FCompilerResultsLog& MessageLog) const;

	/** This function returns an arbitrary number of attributes that describe this node for analytics events */
	BLUEPRINTGRAPH_API virtual void GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const;

	/** Called before compilation begins, giving a blueprint time to force the linker to load data */
	virtual void PreloadRequiredAssets() { }

	/** 
	 * Replacement for GetMenuEntries(). Override to add specific 
	 * UBlueprintNodeSpawners pertaining to the sub-class type. Serves as an 
	 * extensible way for new nodes, and game module nodes to add themselves to
	 * context menus.
	 *
	 * @param  ActionListOut	The list to be populated with new spawners.
	 */
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const {}

	/**
	 * Override to provide a default category for specific node types to be 
	 * listed under.
	 *
	 * @return A localized category string (or an empty string if you want this node listed at the menu's root).
	 */
	virtual FText GetMenuCategory() const { return FText::GetEmpty(); }

	/**
	 * Retrieves a unique identifier for this node type. Built from the node's 
	 * class, as well as other signature items (like functions for CallFunction
	 * nodes).
	 *
	 * NOTE: This is not the same as a node identification GUID, two node 
	 *       instances can have the same signature (if both call the same 
	 *       function, etc.).
	 * 
	 * @return A signature struct, discerning this node from others.
	 */
	virtual FBlueprintNodeSignature GetSignature() const { return FBlueprintNodeSignature(GetClass()); }

	enum EBaseNodeRefreshPriority
	{
		Low_UsesDependentWildcard = 100,
		Low_ReceivesDelegateSignature = 150,
		Normal = 200,
	};

	virtual int32 GetNodeRefreshPriority() const { return EBaseNodeRefreshPriority::Normal; }

	BLUEPRINTGRAPH_API bool DoesWildcardPinAcceptContainer(const UEdGraphPin* Pin);
	virtual bool DoesInputWildcardPinAcceptArray(const UEdGraphPin* Pin) const { return true; }
	virtual bool DoesOutputWildcardPinAcceptContainer(const UEdGraphPin* Pin) const { return true; }

	/** Handle when a variable is renamed in the Blueprint Palette */
	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) {}

	/** Return whether this node references the specified variable, give the supplied scope. Used when variable types are changed. */
	virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const { return false; }

	/** Handle when a function is renamed in the Blueprint Palette */
	virtual void HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName) {}
	
	/** Return whether this node references the specified function, identified by a name and guid pair **/
	virtual bool ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const { return false; };

	/** 
	 * Replace any member references of source with replacement 
	 * @param	InBlueprint		The blueprint that the source reference is from
	 * @param	InReplacementBlueprint	The blueprint that the replacement reference is from
	 * @param	InSource		The reference to be replaced
	 * @param	InReplacement		The reference to replace with
	 */
	virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) {}

	/** Helper function for ExpandNode(), allowing other contexts to call pin expansion alone */
	BLUEPRINTGRAPH_API void ExpandSplitPins(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph);

	/** Hook for allowing derived types to determine if they should be treated as a latent node by macros */
	virtual bool IsLatentForMacros() const { return false; }

protected:

	enum ERedirectType
	{
		/* The pins do not match */
		ERedirectType_None,
		/* The pins match by name or redirect and have the same type (or we're ok with the mismatched type) */
		ERedirectType_Name,
		/* The pins match via a redirect and the value needs to also be redirected */
		ERedirectType_Value,
		/* The pins differ by type and have a default value*/
		ERedirectType_DefaultValue,
	};

	// Handles the actual reconstruction (copying data, links, name, etc...) from two pins that have already been matched together
	BLUEPRINTGRAPH_API void ReconstructSinglePin(UEdGraphPin* NewPin, UEdGraphPin* OldPin, ERedirectType RedirectType);

	// Helper function to rewire old pins to new pins during node reconstruction (or other regeneration of pins)
	BLUEPRINTGRAPH_API void RewireOldPinsToNewPins(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins, TMap<UEdGraphPin*, UEdGraphPin*>* NewPinToOldPin);

	// Helper function to properly destroy a set of pins
	BLUEPRINTGRAPH_API void DestroyPinList(TArray<UEdGraphPin*>& InPins);

	/** Whether or not two pins match for purposes of reconnection after reconstruction.  This allows pins that may have had their names changed via reconstruction to be matched to their old values on a node-by-node basis, if needed*/
	BLUEPRINTGRAPH_API virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const;

	/** Determines what the possible redirect pin names are **/
	BLUEPRINTGRAPH_API virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const;

	// Implementation function that renames an existing pin on the node. Does not broadcast notifications.
	BLUEPRINTGRAPH_API virtual ERenamePinResult RenameUserDefinedPinImpl(const FName OldName, const FName NewName, bool bTest);

	/** Attempt to broadcast an event for a user-defined pin being renamed on this node */
	BLUEPRINTGRAPH_API void BroadcastUserDefinedPinRenamed(FName OldName, FName NewName);

	/** 
	 * Searches ParamRedirect Map and find if there is matching new param
	 * 
	 * returns the redirect type
	 */
	BLUEPRINTGRAPH_API ERedirectType ShouldRedirectParam(const TArray<FString>& OldPinNames, FName& NewPinName, const UK2Node* NewPinNode) const;

	// Helper function to restore Split Pins after ReallocatePinsDuringReconstruction, call after recreating all pins to restore split pin state
	BLUEPRINTGRAPH_API void RestoreSplitPins(TArray<UEdGraphPin*>& OldPins);

	/** 
	 * Sends a message to the owning blueprint's CurrentMessageLog, if there is one available.  Otherwise, defaults to logging to the normal channels.
	 * Should use this for node actions that happen during compilation!
	 */
	template<typename... ArgTypes>
	void Message_Note(const FString& Message, ArgTypes... Args) const
	{
		UBlueprint* OwningBP = GetBlueprint();
		if (OwningBP)
		{
			OwningBP->Message_Note(Message, Forward<ArgTypes>(Args)...);
		}
		else
		{
			UE_LOGF(LogBlueprint, Log, "%ls", *Message);
		}
	}

	template<typename... ArgTypes>
	void Message_Warn(const FString& Message, ArgTypes... Args) const
	{
		UBlueprint* OwningBP = GetBlueprint();
		if (OwningBP)
		{
			OwningBP->Message_Warn(Message, Forward<ArgTypes>(Args)...);
		}
		else
		{
			UE_LOGF(LogBlueprint, Warning, "%ls", *Message);
		}
	}

	template<typename... ArgTypes>
	void Message_Error(const FString& Message, ArgTypes... Args) const
	{
		UBlueprint* OwningBP = GetBlueprint();
		if (OwningBP)
		{
			OwningBP->Message_Error(Message, Forward<ArgTypes>(Args)...);
		}
		else
		{
			UE_LOGF(LogBlueprint, Error, "%ls", *Message);
		}
	}

	friend class FKismetCompilerContext;


protected:
	// Ensures the specified object is preloaded.  ReferencedObject can be NULL.
	void PreloadObject(UObject* ReferencedObject)
	{
		if (ReferencedObject)
		{
			ReferencedObject->ConditionalPreload();
		}
	}

	/** Handle backwards compatible fixes on load */
	BLUEPRINTGRAPH_API virtual void FixupPinDefaultValues();

	/** Fixes up structure/soft object ref pins, on both save and load */
	BLUEPRINTGRAPH_API virtual void FixupPinStringDataReferences(FArchive* SavingArchive);

private:

	/** 
	 * Utility function to write messages about orphan nodes in to the compiler log.
	 * bStore indicates whether to write immediately to the log, or to store as a potential message to be committed once node pruning has completed
	 */
	void ValidateOrphanPins(FCompilerResultsLog& MessageLog, bool bStore) const;
	void ValidateOrphanPin(UEdGraphPin* Pin, FCompilerResultsLog& MessageLog, bool bStore) const;

	/**
	 * Checks that pins are type compatible with their links.
	 * If there's a type mismatch (eg: due to a underlying type change), then we'll automatically
	 * insert a conversion node into the graph if one exists between the two types.
	 */
	void ValidateLinkedPinTypes(UEdGraphPin* OutputPin, FCompilerResultsLog& MessageLog) const;

	/**
	 * If there's a type mismatch between old and new pins during rewiring *and* the old pin has a default value,
	 * then we'll need a way to keep the old default value and convert it to the new type.
	 * This function achieves that by inserting both a literal and a conversion node.
	 * The literal node will use the original default value.
	 * 
	 * Returns true if the conversion was successful.
	 */
	bool TryInsertDefaultValueConversionNode(const UEdGraphPin& OldPin, UEdGraphPin& NewPin) const;

	/** A fast pin rewiring function that does not handle split pins or orphaned pins, but safely aborts if any are found */
	bool TryRewireOldPinsToNewPinsFast(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins, TMap<UEdGraphPin*, UEdGraphPin*>* NewPinToOldPin);
public:

	/** Util to get the generated class from a node. */
	BLUEPRINTGRAPH_API UClass* GetBlueprintClassFromNode() const;
};

#undef UE_API
