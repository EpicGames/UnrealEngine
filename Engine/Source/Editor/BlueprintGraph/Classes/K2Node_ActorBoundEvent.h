// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_Event.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_ActorBoundEvent.generated.h"

class AActor;
class FArchive;
class FKismetCompilerContext;
class FMulticastDelegateProperty;
class FNodeHandlingFunctor;
class UClass;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_ActorBoundEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	/** Delegate property name that this event is associated with */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Delegate property's owner class that this event is associated with */
	UPROPERTY()
	TObjectPtr<UClass> DelegateOwnerClass;

	/** The event that this event is bound to */
	UPROPERTY()
	TObjectPtr<class AActor> EventOwner;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void ReconstructNode() override;
	virtual void DestroyNode() override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual AActor* GetReferencedLevelActor() const override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual FNodeHandlingFunctor* CreateNodeHandler(FKismetCompilerContext& CompilerContext) const override;
	//~ End K2Node Interface

	virtual bool IsUsedByAuthorityOnlyDelegate() const override;

	/**
	* Initialized the members of the node, given the specified owner and delegate property.  This will fill out all the required members for the event, such as CustomFunctionName
	*
	* @param InEventOwner			The target for this bound event
	* @param InDelegateProperty	The multicast delegate property associated with the event, which will have a delegate added to it in the level script actor, matching its signature
	*/
	BLUEPRINTGRAPH_API void InitializeActorBoundEventParams(AActor* InEventOwner, const FMulticastDelegateProperty* InDelegateProperty);

	/** Return the delegate property that this event is bound to */
	BLUEPRINTGRAPH_API FMulticastDelegateProperty* GetTargetDelegateProperty() const;
	BLUEPRINTGRAPH_API FMulticastDelegateProperty* GetTargetDelegatePropertyFromSkel() const;

	/** Gets the proper display name for the property */
	BLUEPRINTGRAPH_API FText GetTargetDelegateDisplayName() const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};

