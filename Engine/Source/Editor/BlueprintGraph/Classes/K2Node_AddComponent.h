// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AddComponent.generated.h"

class FArchive;
class UActorComponent;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_AddComponent : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	/** Prefix used for component template object name. */
	BLUEPRINTGRAPH_API static const FString ComponentTemplateNamePrefix;

	UPROPERTY()
	uint32 bHasExposedVariable:1;

	/** The blueprint name we came from, so we can lookup the template after a paste */
	UPROPERTY()
	FString TemplateBlueprint;

	UPROPERTY()
	TObjectPtr<UClass> TemplateType;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void DestroyNode() override;
	virtual void PrepareForCopying() override;
	virtual void PostPasteNode() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	virtual void ReconstructNode() override;
	virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual void PostReconstructNode() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End UK2Node Interface

	BLUEPRINTGRAPH_API void AllocateDefaultPinsWithoutExposedVariables();
	BLUEPRINTGRAPH_API void AllocatePinsForExposedVariables();

	UEdGraphPin* GetTemplateNamePinChecked() const
	{
		UEdGraphPin* FoundPin = GetTemplateNamePin();
		check(FoundPin != NULL);
		return FoundPin;
	}

	UEdGraphPin* GetRelativeTransformPin() const
	{
		return FindPinChecked(NAME_RelativeTransform);
	}

	UEdGraphPin* GetManualAttachmentPin() const
	{
		return FindPinChecked(NAME_ManualAttachment);
	}

	/** Tries to get a template object from this node. */
	BLUEPRINTGRAPH_API UActorComponent* GetTemplateFromNode() const;

	/** Helper method used to generate a new, unique component template name. */
	FName MakeNewComponentTemplateName(UObject* InOuter, UClass* InComponentClass);

	/** Helper method used to instantiate a new component template after duplication. */
	BLUEPRINTGRAPH_API void MakeNewComponentTemplate();

	/** Static name of function to call */
	static BLUEPRINTGRAPH_API FName GetAddComponentFunctionName();

protected:
	//~ UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface

private: 
	UEdGraphPin* GetTemplateNamePin() const
	{
		return FindPin(TEXT("TemplateName"));
	}

	UClass* GetSpawnedType() const;	

	static const FName NAME_RelativeTransform;
	static const FName NAME_ManualAttachment;
};



