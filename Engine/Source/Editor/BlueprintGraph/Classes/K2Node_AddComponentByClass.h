// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "K2Node_ConstructObjectFromClass.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AddComponentByClass.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FKismetCompilerContext;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

/**
 * Implementation of K2Node for creating a component based on a selected or passed in class
 */
UCLASS(MinimalAPI)
class UK2Node_AddComponentByClass : public UK2Node_ConstructObjectFromClass
{
	GENERATED_BODY()

public:
	UE_API UK2Node_AddComponentByClass(const FObjectInitializer& ObjectInitializer);
	
	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node_ConstructObjectFromClass Interface
	UE_API virtual void CreatePinsForClass(UClass* InClass, TArray<UEdGraphPin*>* OutClassPins) override;

protected:
	UE_API virtual FText GetBaseNodeTitle() const override;
	UE_API virtual FText GetDefaultNodeTitle() const override;
	UE_API virtual FText GetNodeTitleFormat() const override;
	UE_API virtual UClass* GetClassPinBaseClass() const override;
	UE_API virtual bool IsSpawnVarPin(UEdGraphPin* Pin) const override;
	//~ End UK2Node_ConstructObjectFromClass Interface

	UE_API UEdGraphPin* GetRelativeTransformPin() const;
	UE_API UEdGraphPin* GetManualAttachmentPin() const;

	/** Returns true if the currently selected or linked class is known to be a scene component */
	UE_API bool IsSceneComponent() const;

	/** Utility function to set whether the scene component specific pins are hidden or not */
	UE_API void SetSceneComponentPinsHidden(bool bHidden);
};

#undef UE_API
