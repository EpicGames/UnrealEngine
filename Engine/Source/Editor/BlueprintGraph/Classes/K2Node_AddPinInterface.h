// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AddPinInterface.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UObject;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UK2Node_AddPinInterface : public UInterface
{
	GENERATED_BODY()
};

class UEdGraphPin;

/**
* Interface for adding the small "Add Pin" symbol to a node in the bottom right hand side. 
* Implementing this interface will provide the API needed to get the UI up and running, but 
* the actual pin creation/naming is up to the specific node. 
* 
* @see UK2Node_CommutativeAssociativeBinaryOperator, UK2Node_DoOnceMultiInput
*/
class IK2Node_AddPinInterface
{
	GENERATED_BODY()

public:

	/**
	* Determine the maximum number of additional input pins this node can have.
	* The node is limited by the number of letters in the alphabet for display
	* purposes
	* 
	* @return constexpr int32
	*/
	static constexpr int32 GetMaxInputPinsNum()
	{
		return (TCHAR('Z') - TCHAR('A'));
	}

	/**
	* GetNameForAdditionalPin
	*
	* @param PinIndex		The additional pin index to get the name of
	*
	* @return FName			Name of this additional pin
	*/
	static UE_API FName GetNameForAdditionalPin(int32 PinIndex);

	/**
	* Add an additional input pin to this node
	*/
	virtual void AddInputPin() = 0;

	/**
	* Determines if a pin can be added to this node. Typically used within   
	* UK2Node::GetNodeContextMenuActions to determine what options there should be. 
	*
	* @return bool	True if a new pin can be added, false if it cannot. 
	*/
	virtual bool CanAddPin() const { return true; }

	/**
	* Attempt to remove the given pin from this node. Should fail if the 
	* given pin is not an additional pin to the node
	*
	* @param Pin		The pin to remove
	*/
	virtual void RemoveInputPin(UEdGraphPin* Pin) { } 

	/**
	* Returns true if the given pin can be removed from this node.
	*
	* @param Pin		The pin to test 
	*
	* @return bool		True if the pin can be removed, false otherwise.
	*/
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const { return true; };
};

#undef UE_API
