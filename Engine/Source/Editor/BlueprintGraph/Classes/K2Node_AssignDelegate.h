// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_AddDelegate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AssignDelegate.generated.h"

class UEdGraph;
class UObject;

/**
 * Modeled after FEdGraphSchemaAction_K2AssignDelegate for the newer Blueprint
 * menu system. Acts simply as a UK2Node_AddDelegate with an attached custom-
 * event node (spawned in PostPlacedNewNode).
 */
UCLASS(MinimalAPI)
class UK2Node_AssignDelegate : public UK2Node_AddDelegate
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual void PostPlacedNewNode() override;
	// End of UEdGraphNode interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedListTitle;
};
