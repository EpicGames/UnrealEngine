// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AssignmentStatement.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FName;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_AssignmentStatement : public UK2Node
{
	GENERATED_UCLASS_BODY()


	// Name of the Variable pin for this node
	static FName VariablePinName;
	// Name of the Value pin for this node
	static FName ValuePinName;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void PostReconstructNode() override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface

	/** Get the Variable input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetVariablePin() const;
	/** Get the Value input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetValuePin() const;
};

