// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_BaseAsyncTask.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_AsyncAction.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class UObject;

/** !!! The proxy object should have RF_StrongRefOnFrame flag. !!! */

UCLASS(MinimalAPI)
class UK2Node_AsyncAction : public UK2Node_BaseAsyncTask
{
	GENERATED_UCLASS_BODY()
	
	// UK2Node interface
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	/** Initialize the async task from a known function **/
	UE_API void InitializeProxyFromFunction(const UFunction* ProxyFunction);
};

#undef UE_API
