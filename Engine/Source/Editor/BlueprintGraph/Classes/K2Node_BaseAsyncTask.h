// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_BaseAsyncTask.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class FMulticastDelegateProperty;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UEdGraphSchema_K2;
class UFunction;
class UK2Node_CallFunction;
class UK2Node_CustomEvent;
class UK2Node_TemporaryVariable;
class UObject;

/** struct to remap pins for Async Tasks.
 * a single K2 node is shared by many proxy classes.
 * This allows redirecting pins by name per proxy class.
 * Add entries similar to this one in Engine.ini:
 * +K2AsyncTaskPinRedirects=(ProxyClassName="AbilityTask_PlayMontageAndWait", OldPinName="OnComplete", NewPinName="OnBlendOut")
 */

struct FAsyncTaskPinRedirectMapInfo
{
	TMap<FName, TArray<UClass*> > OldPinToProxyClassMap;
};

/**
 * Base class used for a blueprint node that calls a function and provides asynchronous exec pins when the function is complete.
 * The created proxy object should have RF_StrongRefOnFrame flag.
 */
UCLASS(MinimalAPI, Abstract)
class UK2Node_BaseAsyncTask : public UK2Node
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	UE_API virtual bool IsLatentForMacros() const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FName GetCornerIcon() const override;
	UE_API virtual FText GetToolTipHeading() const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const override;
	UE_API virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	// End of UK2Node interface

	/** Returns the primary function that this node will call */
	UE_API UFunction* GetFactoryFunction() const;

protected:
	/**
	 * If a the DefaultToSelf pin exists then it needs an actual connection to get properly casted
	 * during compilation.
	 *
	 * @param CompilerContext			The current compiler context used during expansion
	 * @param SourceGraph				The graph to place the expanded self node on
	 * @param IntermediateProxyNode		The spawned intermediate proxy node that has a DefaultToSelfPin
	 *
	 * @return	True if a successful connection was made to an intermediate node
	 *			or if one was not necessary. False if a connection was failed.
	 */
	UE_API bool ExpandDefaultToSelfPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node_CallFunction* IntermediateProxyNode);
	
	// The name of the function to call to create a proxy object
	UPROPERTY()
	FName ProxyFactoryFunctionName;

	// The class containing the proxy object functions
	UPROPERTY()
	TObjectPtr<UClass> ProxyFactoryClass;

	// The type of proxy object that will be created
	UPROPERTY()
	TObjectPtr<UClass> ProxyClass;

	// The name of the 'go' function on the proxy object that will be called after delegates are in place, can be NAME_None
	UPROPERTY()
	FName ProxyActivateFunctionName;

	struct FBaseAsyncTaskHelper
	{
		struct FOutputPinAndLocalVariable
		{
			UEdGraphPin* OutputPin;
			UK2Node_TemporaryVariable* TempVar;

			FOutputPinAndLocalVariable(UEdGraphPin* Pin, UK2Node_TemporaryVariable* Var) : OutputPin(Pin), TempVar(Var) {}
		};

		static UE_API bool ValidDataPin(const UEdGraphPin* Pin, EEdGraphPinDirection Direction);
		static UE_API bool CreateDelegateForNewFunction(UEdGraphPin* DelegateInputPin, FName FunctionName, UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext);
		static UE_API bool CopyEventSignature(UK2Node_CustomEvent* CENode, UFunction* Function, const UEdGraphSchema_K2* Schema);
		static UE_API bool HandleDelegateImplementation(
			FMulticastDelegateProperty* CurrentProperty, const TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable>& VariableOutputs,
			UEdGraphPin* ProxyObjectPin, UEdGraphPin*& InOutLastThenPin, UEdGraphPin*& OutLastActivatedThenPin,
			UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext);

		static UE_API const FName GetAsyncTaskProxyName();
	};

	// Pin Redirector support
	static UE_API TMap<FName, FAsyncTaskPinRedirectMapInfo> AsyncTaskPinRedirectMap;
	static UE_API bool bAsyncTaskPinRedirectMapInitialized;

	/** Expand out the logic to handle the delegate output pins */
	UE_API virtual bool HandleDelegates(
		const TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable>& VariableOutputs, UEdGraphPin* ProxyObjectPin,
		UEdGraphPin*& InOutLastThenPin, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext);

private:
	/** Invalidates current pin tool tips, so that they will be refreshed before being displayed: */
	void InvalidatePinTooltips() { bPinTooltipsValid = false; }

	/**
	* Creates hover text for the specified pin.
	*
	* @param   Pin				The pin you want hover text for (should belong to this node)
	*/
	UE_API void GeneratePinTooltip(UEdGraphPin& Pin) const;

	/** Flag used to track validity of pin tooltips, when tooltips are invalid they will be refreshed before being displayed */
	mutable bool bPinTooltipsValid;
};

#undef UE_API
