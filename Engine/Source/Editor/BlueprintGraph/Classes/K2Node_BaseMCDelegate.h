// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "K2Node.h"
#include "Engine/MemberReference.h"
#include "UObject/UObjectHash.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "K2Node_BaseMCDelegate.generated.h"

class UEdGraph;

UCLASS(MinimalAPI, abstract)
class UK2Node_BaseMCDelegate : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Reference to delegate */
	UPROPERTY()
	FMemberReference	DelegateReference;

public:
	
	// UK2Node interface
	virtual bool IsNodePure() const override { return false; }
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual bool AllowMultipleSelfs(bool bInputAsArray) const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual bool HasDeprecatedReference() const override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	// End of UK2Node interface

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	// End of UEdGraphNode interface

	void SetFromProperty(const FProperty* Property, bool bSelfContext, UClass* OwnerClass)
	{
		DelegateReference.SetFromField<FProperty>(Property, bSelfContext, OwnerClass);
	}

	FProperty* GetProperty() const
	{
		return DelegateReference.ResolveMember<FMulticastDelegateProperty>(GetBlueprintClassFromNode());
	}

	FName GetPropertyName() const
	{
		return DelegateReference.GetMemberName();
	}

	FText GetPropertyDisplayName() const
	{
		FProperty* Prop = GetProperty();
		return (Prop ? Prop->GetDisplayNameText() : FText::FromName(GetPropertyName()));
	}

	BLUEPRINTGRAPH_API UFunction* GetDelegateSignature(bool bForceNotFromSkelClass = false) const;

	BLUEPRINTGRAPH_API UEdGraphPin* GetDelegatePin() const;

	/** Is the delegate BlueprintAuthorityOnly */
	BLUEPRINTGRAPH_API bool IsAuthorityOnly() const;

protected:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};
