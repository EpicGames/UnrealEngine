// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "NodeDependingOnEnumInterface.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_BitmaskLiteral.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class FName;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_BitmaskLiteral : public UK2Node, public INodeDependingOnEnumInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UEnum> BitflagsEnum;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void ReconstructNode() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface
	
	//~ Begin UK2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

	//~ Begin INodeDependingOnEnumInterface
	virtual class UEnum* GetEnum() const override { return BitflagsEnum; }
	virtual void ReloadEnum(class UEnum* InEnum) override;
	virtual bool ShouldBeReconstructedAfterEnumChanged() const override { return true; }
	//~ End INodeDependingOnEnumInterface

	BLUEPRINTGRAPH_API static const FName& GetBitmaskInputPinName();

protected:
	/** Internal helper method used to validate the current enum type */
	void ValidateBitflagsEnumType();
};

