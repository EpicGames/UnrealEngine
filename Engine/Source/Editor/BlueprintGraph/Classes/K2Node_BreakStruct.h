// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_StructMemberGet.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_BreakStruct.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class FObjectPreSaveContext;
class UEdGraph;
class UEdGraphPin;
class UObject;
class UScriptStruct;

UCLASS(MinimalAPI)
class UK2Node_BreakStruct : public UK2Node_StructMemberGet
{
	GENERATED_UCLASS_BODY()

	/** Helper property to handle upgrades from an old system of displaying pins for
	*	the override values that properties referenced as a conditional of being set in a struct */
	UPROPERTY()
	bool bMadeAfterOverridePinRemoval;

	/** 
	 * Returns false if:
	 *   1. The Struct has a 'native break' method
	 * Returns true if:
	 *   1. The Struct is tagged as BlueprintType
	 *   and
	 *   2. The Struct has any property that is tagged as CPF_BlueprintVisible
	 */
	static bool CanBeBroken(const UScriptStruct* Struct, bool bForInternalUse = false);

	/** Can this struct be used as a split pin */
	static bool CanBeSplit(const UScriptStruct* Struct) { return CanBeBroken(Struct); }

	// UObject interface
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void PreloadRequiredAssets() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void PostPlacedNewNode() override;
	//~ End  UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return false; }
	virtual bool IsNodePure() const override { return true; }
	virtual bool DrawNodeAsVariable() const override { return false; }
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override;
	//~ End K2Node Interface

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

