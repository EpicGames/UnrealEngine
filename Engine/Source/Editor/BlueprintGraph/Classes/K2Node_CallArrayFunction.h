// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallArrayFunction.generated.h"

class FProperty;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;

struct FArrayPropertyPinCombo
{
	UEdGraphPin* ArrayPin;
	UEdGraphPin* ArrayPropPin;

	FArrayPropertyPinCombo()
		: ArrayPin(NULL)
		, ArrayPropPin(NULL)
	{}
};

UCLASS(MinimalAPI)
class UK2Node_CallArrayFunction : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void PostReconstructNode() override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual bool DoesInputWildcardPinAcceptArray(const UEdGraphPin* Pin) const override { return false; }
	virtual bool DoesOutputWildcardPinAcceptContainer(const UEdGraphPin* Pin) const override { return false; }
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override;
	//~ End UK2Node Interface

	/** Retrieves the target pin for the function */
	BLUEPRINTGRAPH_API UEdGraphPin* GetTargetArrayPin() const;

	/**
	 * Retrieves the array pins and their property pins as a combo-struct
	 *
	 * @param OutArrayPinInfo		The pins and their property pins will be added to this array
	 */
	BLUEPRINTGRAPH_API void GetArrayPins(TArray< FArrayPropertyPinCombo >& OutArrayPinInfo ) const;

	/**
	 * Checks if the passed in property is a wildcard property
	 *
	 * @param InArrayFunction		The array function to check
	 * @param InProperty			Property to examine to see if it is marked in metadata as a wildcard
	 *
	 * @return						TRUE if the property is a wildcard.
	 */
	BLUEPRINTGRAPH_API static bool IsWildcardProperty(UFunction* InArrayFunction, const FProperty* InProperty);

	
	/**
	* Gather all pins on this node that are dependent on the array type, including the Array target pin. 
	* 
	* @param OutPins		Array of pins to populate
	*/
	void GetArrayTypeDependentPins(TArray<UEdGraphPin*>& OutPins) const;

	void PropagateArrayTypeInfo(const UEdGraphPin* SourcePin);
};

