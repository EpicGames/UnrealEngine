// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallDataTableFunction.generated.h"

class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CallDataTableFunction : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphNode Interface
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface
};
