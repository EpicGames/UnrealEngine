// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallDelegate.generated.h"

UCLASS(MinimalAPI)
class UK2Node_CallDelegate : public UK2Node_BaseMCDelegate
{
	GENERATED_UCLASS_BODY()

public:

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual FName GetCornerIcon() const override;
	// End of UK2Node interface

	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	bool CreatePinsForFunctionInputs(const UFunction* Function);
};
