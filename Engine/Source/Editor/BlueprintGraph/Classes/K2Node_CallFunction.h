// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallFunction.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FKismetCompilerContext;
class FProperty;
class SWidget;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UENUM()
enum class ENodePurityOverride : int8
{
	Unset = 0,
	Pure,
	Impure
};

UCLASS(MinimalAPI)
class UK2Node_CallFunction : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Indicates that the bound function defaults to a pure state */
	UPROPERTY()
	uint32 bDefaultsToPureFunc:1;
	
	UE_DEPRECATED(5.5, "bIsPureFunc is deprecated. Use bDefaultsToPureFunc or IsNodePure instead.")
	uint32 bIsPureFunc:1;

	/** Indicates that during compile we want to create multiple exec pins from an enum param */
	UPROPERTY()
	uint32 bWantsEnumToExecExpansion:1;

	UE_DEPRECATED(5.5, "bIsConstFunc is deprecated. Check for FUNC_Const on FunctionReference.ResolveMember<UFunction>() instead.")
	uint32 bIsConstFunc:1;

	UE_DEPRECATED(5.5, "bIsInterfaceCall is deprecated. Check for CLASS_Interface on FunctionReference.GetMemberParentClass() instead.")
	uint32 bIsInterfaceCall:1;

	/** The function to call */
	UPROPERTY()
	FMemberReference FunctionReference;

private:
	/** The name of the function to call */
	UPROPERTY()
	FName CallFunctionName_DEPRECATED;

	/** The class that the function is from. */
	UPROPERTY()
	TSubclassOf<class UObject> CallFunctionClass_DEPRECATED;

protected:
	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;

private:
	/** Flag used to track validity of pin tooltips, when tooltips are invalid they will be refreshed before being displayed */
	mutable bool bPinTooltipsValid;

	TArray<UEdGraphPin*> ExpandAsEnumPins;

public:

	// UObject interface
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FString GetDescriptiveCompiledName() const override;
	UE_API virtual bool HasDeprecatedReference() const override;
	UE_API virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual FString GetDocumentationLink() const override;
	UE_API virtual FString GetDocumentationExcerptName() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual TSharedPtr<SWidget> CreateNodeImage() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	UE_API virtual bool IsLatentForMacros() const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual bool IsNodePure() const override;
	UE_API virtual void PostReconstructNode() override;
	UE_API virtual bool ShouldDrawCompact() const override;
	UE_API virtual FText GetCompactNodeTitle() const override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual bool ShouldShowNodeProperties() const override;
	UE_API virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const override;
	UE_API virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FName GetCornerIcon() const override;
	UE_API virtual FText GetToolTipHeading() const override;
	UE_API virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	// End of UK2Node interface

	/** Returns the UFunction that this class is pointing to */
	UE_API UFunction* GetTargetFunction() const;

	/** Get the return value pin */
	UE_API UEdGraphPin* GetReturnValuePin() const;

	/** @return	true if the function is a latent operation */
	UE_API bool IsLatentFunction() const;

	/** @return true if this function can be called on multiple contexts at once */
	UE_API virtual bool AllowMultipleSelfs(bool bInputAsArray) const override;

	/**
	 * Creates a self pin for the graph, taking into account the scope of the function call
	 *
	 * @param	Function	The function to be called by the Node.
	 *
	 * @return	Pointer to the pin that was created
	 */
	UE_API virtual UEdGraphPin* CreateSelfPin(const UFunction* Function);

	/**
	 * Creates all of the pins required to call a particular UFunction.
	 *
	 * @param	Function	The function to be called by the Node.
	 *
	 * @return	true on success.
	 */
	UE_API bool CreatePinsForFunctionCall(const UFunction* Function);

	/** Create exec pins for this function. May be multiple is using 'expand enum as execs' */
	UE_API void CreateExecPinsForFunctionCall(const UFunction* Function);

	/** Gets the name of the referenced function */
	UE_API FName GetFunctionName() const;

	virtual void PostParameterPinCreated(UEdGraphPin *Pin) {}

	UE_DEPRECATED(5.5, "Moved to ObjectTools::GetUserFacingFunctionName.")
	static UE_API FText GetUserFacingFunctionName(const UFunction* Function, ENodeTitleType::Type NodeTitleType = ENodeTitleType::EditableTitle);

	/** Set up a pins tooltip from a function's tooltip */
	static UE_API void GeneratePinTooltipFromFunction(UEdGraphPin& Pin, const UFunction* Function);

	UE_DEPRECATED(5.5, "Moved to ObjectTools::GetDefaultTooltipForFunction.")
	static UE_API FString GetDefaultTooltipForFunction(const UFunction* Function);

	/** Get default category for this function in action menu */
	static UE_API FText GetDefaultCategoryForFunction(const UFunction* Function, const FText& BaseCategory);
	/** Get keywords for this function in the action menu */
	static UE_API FText GetKeywordsForFunction(const UFunction* Function);
	/** Should be drawn compact for this function */
	static UE_API bool ShouldDrawCompact(const UFunction* Function);
	/** Get the compact name for this function */
	static UE_API FString GetCompactNodeTitle(const UFunction* Function);

	/** Get the text to use to explain the context for this function (used on node title) */
	UE_API virtual FText GetFunctionContextString() const;

	/** Set properties of this node from a supplied function (does not save ref to function) */
	UE_API virtual void SetFromFunction(const UFunction* Function);

	static UE_API void CallForEachElementInArrayExpansion(UK2Node* Node, UEdGraphPin* MultiSelf, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph);

	static UE_API UEdGraphPin* InnerHandleAutoCreateRef(UK2Node* Node, UEdGraphPin* Pin, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, bool bForceAssignment);

	/**
	 * Returns the graph for this function, if available. In cases of calling an event, it will be the ubergraph for the event
	 *
	 * @param OutGraphNode		If this function calls an event, this param is the event node, otherwise it is NULL
	 */
	UE_API UEdGraph* GetFunctionGraph(const UEdGraphNode*& OutGraphNode) const;

	/** Checks if the property is marked as "CustomStructureParam" */
	static UE_API bool IsStructureWildcardProperty(const UFunction* InFunction, const FName PropertyName);

	/** returns true if InProperty should be treated as a wildcard (e.g. due to SetParam markup) */
	static UE_API bool IsWildcardProperty(const UFunction* InFunction, const FProperty* InProperty);

	/** Used to determine the result of AllowMultipleSelfs() (without having a node instance) */
	static UE_API bool CanFunctionSupportMultipleTargets(UFunction const* InFunction);

	/** Checks if the input function can be called in the input object with respect to editor-only/runtime mismatch*/
	static UE_API bool CanEditorOnlyFunctionBeCalled(const UFunction* InFunction, const UObject* InObject);

	/** */
	static UE_API FSlateIcon GetPaletteIconForFunction(UFunction const* Function, FLinearColor& OutColor);

	static UE_API void GetExpandEnumPinNames(const UFunction* Function, TArray<FName>& EnumNamesToCheck);

private: 
	/* Looks at function metadata and properties to determine if this node should be using enum to exec expansion */
	UE_API void DetermineWantsEnumToExecExpansion(const UFunction* Function);

	/**
	 * Creates hover text for the specified pin.
	 * 
	 * @param   Pin				The pin you want hover text for (should belong to this node)
	 */
	UE_API void GeneratePinTooltip(UEdGraphPin& Pin) const;

	/**
	 * Connect Execute and Then pins for functions, which became pure.
	 */
	UE_API bool ReconnectPureExecPins(TArray<UEdGraphPin*>& OldPins);

	/** Conforms container pins */
	UE_API void ConformContainerPins();

	UPROPERTY()
	ENodePurityOverride NodePurityOverride;

	UE_API bool AreExecPinsVisible() const;
	UE_API bool FunctionHasOutputs() const;
	UE_API void ToggleNodePurityOverride();

protected:

	/** Invalidates current pin tool tips, so that they will be refreshed before being displayed: */
	UE_API void InvalidatePinTooltips();

	/** Helper function to ensure function is called in our context */
	UE_API virtual void FixupSelfMemberContext();

	/** By default, pure nodes can be toggled. Return false if you don't want your node to support toggling. */
	UE_API virtual bool CanToggleNodePurity() const;

	/** Adds this function to the suppressed deprecation warnings list for this project */
	UE_API void SuppressDeprecationWarning() const;

	/** Helper function for searching a UFunction for the names of requires pins/params: */
	static UE_API TSet<FName> GetRequiredParamNames(const UFunction* ForFunction);

	/** Routine for validating that all UPARAM(Required) parmas have a connection: */
	UE_API void ValidateRequiredPins(const UFunction* Function, class FCompilerResultsLog& MessageLog) const;

	/** Helper function to find UFunction entries from the skeleton class, use with caution.. */
	UE_API UFunction* GetTargetFunctionFromSkeletonClass() const;
};

#undef UE_API
