// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Engine/MemberReference.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallFunctionOnMember.generated.h"

class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CallFunctionOnMember : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	/** Reference to member variable to call function on */
	UPROPERTY()
	FMemberReference MemberVariableToCallOn;

	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;

	//~ Begin UK2Node_CallFunction Interface
	virtual UEdGraphPin* CreateSelfPin(const UFunction* Function) override;
	virtual FText GetFunctionContextString() const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;

protected:
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction Interface
};

