// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallMaterialParameterCollectionFunction.generated.h"

class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CallMaterialParameterCollectionFunction : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphNode Interface
	virtual void PreloadRequiredAssets() override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End EdGraphNode Interface
};
