// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CallParentFunction.generated.h"

class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CallParentFunction : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual void PostPlacedNewNode() override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	//~ End EdGraphNode Interface

	virtual void SetFromFunction(const UFunction* Function) override;

protected:

	//~ Begin K2Node_CallFunction Interface
	virtual void FixupSelfMemberContext() override;
	//~ End K2Node_CallFunction Interface
};

