// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "NodeDependingOnEnumInterface.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CastByteToEnum.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class FNodeHandlingFunctor;
class FString;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_CastByteToEnum : public UK2Node, public INodeDependingOnEnumInterface
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<UEnum> Enum;

	/* if true, the node returns always a valid value */
	UPROPERTY()
	bool bSafe;

	static const FName ByteInputPinName;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetCompactNodeTitle() const override;
	virtual bool ShouldDrawCompact() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	FNodeHandlingFunctor* CreateNodeHandler(FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

	// INodeDependingOnEnumInterface
	virtual class UEnum* GetEnum() const override { return Enum; }
	virtual void ReloadEnum(class UEnum* InEnum) override;
	virtual bool ShouldBeReconstructedAfterEnumChanged() const override { return false; }
	// End of INodeDependingOnEnumInterface

	virtual FName GetFunctionName() const;

private:
	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;
};

