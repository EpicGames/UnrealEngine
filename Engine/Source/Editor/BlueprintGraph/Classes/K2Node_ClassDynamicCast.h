// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_DynamicCast.h"
#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_ClassDynamicCast.generated.h"

class FString;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_ClassDynamicCast : public UK2Node_DynamicCast
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override { return UK2Node::IsConnectionDisallowed(MyPin, OtherPin, OutReason); }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override { UK2Node::NotifyPinConnectionListChanged(Pin); }
	//~ End UK2Node Interface

	//~ Begin UK2Node_DynamicCast Interface
	virtual UEdGraphPin* GetCastSourcePin() const override;
	virtual UEdGraphPin* GetBoolSuccessPin() const override;
	//~ End UK2Node_DynamicCast Interface
};

