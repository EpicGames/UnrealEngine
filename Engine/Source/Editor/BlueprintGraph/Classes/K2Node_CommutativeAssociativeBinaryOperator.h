// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "K2Node_AddPinInterface.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CommutativeAssociativeBinaryOperator.generated.h"

class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CommutativeAssociativeBinaryOperator : public UK2Node_CallFunction, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

	/** The number of additional input pins to generate for this node (2 base pins are not included) */
	UPROPERTY()
	int32 NumAdditionalInputs;

private:
	const static int32 BinaryOperatorInputsNum = 2;

	FEdGraphPinType GetType() const;

	void AddInputPinInner(int32 AdditionalPinIndex);

protected: 
	bool CanRemovePin(const UEdGraphPin* Pin) const override;

public:
	BLUEPRINTGRAPH_API UEdGraphPin* FindOutPin() const;
	BLUEPRINTGRAPH_API UEdGraphPin* FindSelfPin() const;

	/** Get TRUE input type (self, etc.. are skipped) */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInputPin(int32 InputPinIndex);

	/** Returns the number of additional input pins that this node has */
	int32 GetNumberOfAdditionalInputs() const { return NumAdditionalInputs; }

	virtual void RemoveInputPin(UEdGraphPin* Pin) override;

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	// End of UK2Node interface

	// IK2Node_AddPinInterface interface
	BLUEPRINTGRAPH_API virtual void AddInputPin() override;
	virtual bool CanAddPin() const override;
	// End of IK2Node_AddPinInterface interface
};
