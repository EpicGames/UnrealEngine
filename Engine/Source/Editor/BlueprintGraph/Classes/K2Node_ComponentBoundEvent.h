// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_Event.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "K2Node_ComponentBoundEvent.generated.h"

class FArchive;
class FMulticastDelegateProperty;
class FObjectProperty;
class UBlueprint;
class UClass;
class UDynamicBlueprintBinding;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_ComponentBoundEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	/** Delegate property name that this event is associated with */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Delegate property's owner class that this event is associated with */
	UPROPERTY()
	TObjectPtr<UClass> DelegateOwnerClass;

	/** Name of property in Blueprint class that pointer to component we want to bind to */
	UPROPERTY()
	FName ComponentPropertyName;

	//~ Begin UObject Interface
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void ReconstructNode() override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual bool HasDeprecatedReference() const override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End K2Node Interface

	virtual bool IsUsedByAuthorityOnlyDelegate() const override;

	/** Return the delegate property that this event is bound to */
	BLUEPRINTGRAPH_API FMulticastDelegateProperty* GetTargetDelegateProperty() const;

	/** Gets the proper display name for the property */
	BLUEPRINTGRAPH_API FText GetTargetDelegateDisplayName() const;

	BLUEPRINTGRAPH_API void InitializeComponentBoundEventParams(FObjectProperty const* InComponentProperty, const FMulticastDelegateProperty* InDelegateProperty);

public:

	FName GetComponentPropertyName() const { return ComponentPropertyName; }

private:

	/** Returns true if there is an FObjectProperty on this blueprint with a name that matches ComponentPropertyName */
	bool IsDelegateValid() const;

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};

