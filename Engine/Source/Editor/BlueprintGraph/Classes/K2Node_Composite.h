// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_Tunnel.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Composite.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class INameValidatorInterface;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_Composite : public UK2Node_Tunnel
{
	GENERATED_UCLASS_BODY()

	// The graph that this composite node is representing
	UPROPERTY()
	TObjectPtr<class UEdGraph> BoundGraph;

	//~ Begin UObject Interface
	UE_API virtual void PostEditUndo() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void DestroyNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual TArray<UEdGraph*> GetSubGraphs() const override { return TArray<UEdGraph*>( { BoundGraph } ); }
	UE_API virtual bool CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsExit() const override { return false; }
	virtual bool DrawNodeAsEntry() const override { return false; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	//~ End UK2Node Interface

	// Get the entry/exit nodes inside this collapsed graph
	UE_API UK2Node_Tunnel* GetEntryNode() const;
	UE_API UK2Node_Tunnel* GetExitNode() const;

protected:
	/** Fixes up the input and output sink when needed, useful after PostEditUndo which changes which graph these nodes point to */
	UE_API void FixupInputAndOutputSink();

private:
	/** Rename the BoundGraph to a unique name
		- this is a special case rename: RenameGraphCloseToName() assumes that we are only concurrned with the immediate Outer() scope,
		  but in the case of the BoundGraph we must also look into the Outer of the composite node(a Graph), and make sure no graphs in its SubGraph
		  array are already using the name.  */
	UE_API void RenameBoundGraphCloseToName(const FString& Name);

	/** Determine if the name already used by another graph in composite nodes chain */
	UE_API bool IsCompositeNameAvailable( const FString& NewName );

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};



#undef UE_API
