// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_ConstructObjectFromClass.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI, abstract)
class UK2Node_ConstructObjectFromClass : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetKeywords() const override;
	UE_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	UE_API virtual void PinConnectionListChanged(UEdGraphPin* Pin);
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	UE_API virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

	/** Create new pins to show properties on archetype */
	UE_API virtual void CreatePinsForClass(UClass* InClass, TArray<UEdGraphPin*>* OutClassPins = nullptr);

	/** See if this is a spawn variable pin, or a 'default' pin */
	UE_API virtual bool IsSpawnVarPin(UEdGraphPin* Pin) const;

	/** Get the blueprint input pin */	
	UE_API UEdGraphPin* GetClassPin(const TArray<UEdGraphPin*>* InPinsToSearch=NULL) const;
	/** Get the world context input pin, can return NULL */
	UE_API UEdGraphPin* GetWorldContextPin() const;
	/** Get the result output pin */
	UE_API UEdGraphPin* GetResultPin() const;
	/** Get the result input pin */
	UE_API UEdGraphPin* GetOuterPin() const;

	/** Get the class that we are going to spawn, if it's defined as default value */
	UE_API UClass* GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch=NULL) const;

	/** Returns if the node uses World Object Context input */
	UE_API virtual bool UseWorldContext() const;

	/** Returns if the node uses Outer input */
	virtual bool UseOuter() const { return false; }

protected:
	/** Gets the node for use in lists and menus */
	UE_API virtual FText GetBaseNodeTitle() const;
	/** Gets the default node title when no class is selected */
	UE_API virtual FText GetDefaultNodeTitle() const;
	/** Gets the node title when a class has been selected. */
	UE_API virtual FText GetNodeTitleFormat() const;
	/** Gets base class to use for the 'class' pin.  UObject by default. */
	UE_API virtual UClass* GetClassPinBaseClass() const;

	/**
	 * Takes the specified "MutatablePin" and sets its 'PinToolTip' field (according
	 * to the specified description)
	 * 
	 * @param   MutatablePin	The pin you want to set tool-tip text on
	 * @param   PinDescription	A string describing the pin's purpose
	 */
	UE_API void SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const;

	/** Refresh pins when class was changed */
	UE_API void OnClassPinChanged();

	/** Tooltip text for this node. */
	FText NodeTooltip;

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};

#undef UE_API
