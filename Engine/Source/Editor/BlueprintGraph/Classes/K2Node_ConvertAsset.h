// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_ConvertAsset.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

/** This node converts between hard and soft references, for both objects and classes. The name is historical from when they were called asset IDs */
UCLASS(MinimalAPI)
class UK2Node_ConvertAsset : public UK2Node
{
	GENERATED_BODY()
public:

	/** Returns the class for the object being converted */
	UClass* GetTargetClass() const;
	/** True if this is converting a class, false for object or unknown */
	bool IsClassType() const;
	/** True if this is going from hard to soft, false for opposite or unknown */
	bool IsConvertToSoft() const;

	/** Pin Accessors */
	UEdGraphPin* GetInputPin() const;
	UEdGraphPin* GetOutputPin() const;

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetKeywords() const override;
	virtual FText GetTooltipText() const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void PostReconstructNode() override;
	virtual bool IsNodePure() const override { return true; }
	virtual bool ShouldDrawCompact() const override { return true; }
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual bool DoesInputWildcardPinAcceptArray(const UEdGraphPin* Pin) const override { return false; }
	virtual bool DoesOutputWildcardPinAcceptContainer(const UEdGraphPin* Pin) const override { return false; }

	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FText GetCompactNodeTitle() const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	// End of UK2Node interface

protected:
	void RefreshPinTypes();
};
