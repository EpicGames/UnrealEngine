// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Copy.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;

UCLASS(MinimalAPI, meta=(Keywords = "Duplicate"))
class UK2Node_Copy : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void PostReconstructNode() override;
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override { return false; }
	//~ End UK2Node Interface

	/** Get the input reference pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInputReferencePin() const;
	/** Get the return value pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetCopyResultPin() const;

protected:
	/** Propagates pin type to the between the input and output pins */
	void PropagatePinType(FEdGraphPinType& InType);
};

