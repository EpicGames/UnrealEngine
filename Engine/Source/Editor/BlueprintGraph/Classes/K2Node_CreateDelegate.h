// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CreateDelegate.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_CreateDelegate : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (BlueprintSearchable = "true"))
	FName SelectedFunctionName;

	UPROPERTY()
	FGuid SelectedFunctionGuid;

public:
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual void NodeConnectionListChanged() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	virtual void FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodePure() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_ReceivesDelegateSignature; }
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	// End of UK2Node interface

	bool IsValid(FString* OutMsg = nullptr, bool bDontUseSkeletalClassForSelf = false) const;

	/** Set new Function name (Without notifying about the change) */
	BLUEPRINTGRAPH_API void SetFunction(FName Name);

	BLUEPRINTGRAPH_API UFunction* GetDelegateSignature() const;
	BLUEPRINTGRAPH_API UClass* GetScopeClass(bool bDontUseSkeletalClassForSelf = false) const;

	BLUEPRINTGRAPH_API FName GetFunctionName() const;
	BLUEPRINTGRAPH_API UEdGraphPin* GetDelegateOutPin() const;
	BLUEPRINTGRAPH_API UEdGraphPin* GetObjectInPin() const;
 
	BLUEPRINTGRAPH_API void HandleAnyChange(bool bForceModify = false);
	BLUEPRINTGRAPH_API void HandleAnyChangeWithoutNotifying();

	BLUEPRINTGRAPH_API void ValidationAfterFunctionsAreCreated(class FCompilerResultsLog& MessageLog, bool bFullCompile) const;

	// return Graph and Blueprint, when they should be notified about change. It allows to call BroadcastChanged only once per blueprint.
	BLUEPRINTGRAPH_API void HandleAnyChange(UEdGraph*& OutGraph, UBlueprint*& OutBlueprint);
};
