// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Event.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_CustomEvent.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class INameValidatorInterface;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;
struct FEdGraphPinType;
struct FKismetUserDeclaredFunctionMetadata;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_CustomEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	/** Optional message to display when the event is deprecated */
	UPROPERTY()
	FString DeprecationMessage;

	/** Specifies that usage of this event has been deprecated */
	UPROPERTY()
	bool bIsDeprecated;

	/** Specifies that the event can be triggered in Editor */
	UPROPERTY()
	bool bCallInEditor;

	virtual bool IsEditable() const override;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void ReconstructNode() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual FText GetKeywords() const override;
	virtual bool HasDeprecatedReference() const override { return bIsDeprecated; }
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void FixupPinStringDataReferences(FArchive* SavingArchive) override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool ShouldUseConstRefParams() const override { return true; }
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue) override;
	//~ Begin UK2Node_EditablePinBase Interface

	virtual bool IsUsedByAuthorityOnlyDelegate() const override;

	// Rename this custom event to have unique name
	BLUEPRINTGRAPH_API void RenameCustomEventCloseToName(int32 StartIndex = 1);

	BLUEPRINTGRAPH_API static UK2Node_CustomEvent* CreateFromFunction(FVector2D GraphPosition, UEdGraph* ParentGraph, const FString& Name, const UFunction* Function, bool bSelectNewNode = true);

	/**
	 * Discernible from the base UK2Node_Event's bOverrideFunction field. This 
	 * checks to see if this UK2Node_CustomEvent overrides another CustomEvent
	 * declared in a parent blueprint.
	 * 
	 * @return True if this CustomEvent's name matches another CustomEvent's name, declared in a parent (false if not).
	 */
	BLUEPRINTGRAPH_API bool IsOverride() const;

	/**
	 * If a CustomEvent overrides another CustomEvent, then it inherits the 
	 * super's net flags. This method does that work for you, either returning
	 * the super function's flags, or this node's flags (if it's not an override).
	 * 
	 * @return If this CustomEvent is an override, then this is the super's net flags, otherwise it's from the FunctionFlags set on this node.
	 */
	BLUEPRINTGRAPH_API uint32 GetNetFlags() const;

	/**
	* Updates the Signature of this event. Empties current pins and adds the new given parameters.
	* Does nothing if DelegateSignature is nullptr
	* 
	* @param DelegateSignature	The new signature for this function to have
	*/
	BLUEPRINTGRAPH_API void SetDelegateSignature(const UFunction* DelegateSignature);

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;

	/** Custom event metadata that can be used for adding custom keywords */
	UPROPERTY()
	FKismetUserDeclaredFunctionMetadata MetaData;

public:

	FKismetUserDeclaredFunctionMetadata& GetUserDefinedMetaData()
	{
		return MetaData;
	}
};