// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_DeadClass.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UK2Node_DeadClass : public UK2Node
{
	GENERATED_UCLASS_BODY()
};
