// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_DelegateSet.generated.h"

class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_DelegateSet : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Delegate property name that this event is associated with on the target */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Class that the delegate property is defined in */
	UPROPERTY()
	TSubclassOf<class UObject>  DelegatePropertyClass;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override { return FColor(216,88,88); }
	//~ End UEdGraphNode Interface


	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsEntry() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End UK2Node Interface

	// @todo document
	BLUEPRINTGRAPH_API UEdGraphPin* GetDelegateOwner() const;

	// @todo document
	FName GetDelegateTargetEntryPointName() const
	{
		const FString TargetName = GetName() + TEXT("_") + DelegatePropertyName.ToString() + TEXT("_EP");
		return FName(*TargetName);
	}

	// @todo document
	BLUEPRINTGRAPH_API UFunction* GetDelegateSignature();
	BLUEPRINTGRAPH_API UFunction* GetDelegateSignature() const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

