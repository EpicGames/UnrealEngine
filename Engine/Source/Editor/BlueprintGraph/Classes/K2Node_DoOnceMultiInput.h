// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_DoOnceMultiInput.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_DoOnceMultiInput : public UK2Node, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

	/** The number of additional input pins to generate for this node (2 base pins are not included) */
	UPROPERTY()
	int32 NumAdditionalInputs;

	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;

	/** Reference to the integer that contains */
	UPROPERTY(transient)
	TObjectPtr<class UK2Node_TemporaryVariable> DataNode;

	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;

private:

	const static int32 NumBaseInputs = 1;

	static FText GetNameForPin(int32 PinIndex, bool In);

	FEdGraphPinType GetInType() const;
	FEdGraphPinType GetOutType() const;

	void AddPinsInner(int32 AdditionalPinIndex);
	bool CanRemovePin(const UEdGraphPin* Pin) const override;
public:
	BLUEPRINTGRAPH_API UEdGraphPin* FindOutPin() const;
	BLUEPRINTGRAPH_API UEdGraphPin* FindSelfPin() const;

	/** Get TRUE input type (self, etc.. are skipped) */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInputPin(int32 InputPinIndex);
	BLUEPRINTGRAPH_API UEdGraphPin* GetOutputPin(int32 InputPinIndex);

	virtual void RemoveInputPin(UEdGraphPin* Pin) override;

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	// End of UK2Node interface

	// IK2Node_AddPinInterface interface
	BLUEPRINTGRAPH_API virtual void AddInputPin() override;
	virtual bool CanAddPin() const override;
	// End of IK2Node_AddPinInterface interface
};
