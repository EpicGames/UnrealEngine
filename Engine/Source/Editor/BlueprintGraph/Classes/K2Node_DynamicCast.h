// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeSignature.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_DynamicCast.generated.h"

class FString;
class UEdGraphPin;
class UObject;

UENUM()
enum class EPureState : uint8
{
	Pure,
	Impure,
	UseDefault
};

UCLASS(MinimalAPI)
class UK2Node_DynamicCast : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** The type that the input should try to be cast to */
	UPROPERTY()
	TSubclassOf<class UObject>  TargetType;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar);
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual FText GetMenuCategory() const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	virtual bool IsNodePure() const override { return PureState == EPureState::Pure; }
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool IsActionFilteredOut(const class FBlueprintActionFilter& Filter) override;
	//~ End UK2Node Interface

	/** Get the 'valid cast' exec pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetValidCastPin() const;

	/** Get the 'invalid cast' exec pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInvalidCastPin() const;

	/** Get the cast result pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetCastResultPin() const;

	/** Get the input object to be casted pin */
	BLUEPRINTGRAPH_API virtual UEdGraphPin* GetCastSourcePin() const;

	/** Get the boolean output pin that signifies a successful/failed cast. */
	BLUEPRINTGRAPH_API virtual UEdGraphPin* GetBoolSuccessPin() const;

	/**
	 * Will change the node's purity, and reallocate pins accordingly (adding/
	 * removing exec pins).
	 * 
	 * @param  bNewPurity  Whether to configure node as pure.
	 */
	BLUEPRINTGRAPH_API void SetPurity(bool bNewPurity);

protected:
	/** Determine purity and create exec pins if needed */
	void CreateExecPins();

	/** Create success output signal pin (visible only if pure) */
	void CreateSuccessPin();

	/** Flips the node's purity (adding/removing exec pins as needed). */
	void TogglePurity();

	/** Update exec pins when converting from impure to pure. */
	bool ReconnectPureExecPins(TArray<UEdGraphPin*>& OldPins);

	/** Sets the pure state to Pure or Impure if it's using the default one */
	void InitPureState();
	
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;

	/** [DEPRECATED] This member is no longer in use. */
	UE_DEPRECATED(5.3, "Use IsNodePure() instead.")
	bool bIsPureCast;

private:

	/** Internal state used to determine purity */
	UPROPERTY()
	EPureState PureState;

	/** [DEPRECATED] Receives the old value on load if previously saved */
	UPROPERTY()
	bool bIsPureCast_DEPRECATED;
};

