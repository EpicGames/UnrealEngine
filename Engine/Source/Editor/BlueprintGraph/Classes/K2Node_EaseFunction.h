// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_EaseFunction.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UEdGraph;
class UObject;
struct FLinearColor;

/////////////////////////////////////////////////////
// UK2Node_EaseFunction

USTRUCT()
struct FCustomPin
{
	GENERATED_USTRUCT_BODY()

	/** Name of the pin within the current node */
	UPROPERTY()
	FName PinName;

	/** Name of the pin of the call function */
	UPROPERTY()
	FName CallFuncPinName;

	/** If TRUE this is a custom pin that does come from the type of wildcard pin connected */
	UPROPERTY()
	bool bValuePin = false;
};

UCLASS(MinimalAPI)
class UK2Node_EaseFunction : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Contex) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PostReconstructNode() override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool IsNodePure() const override { return true; }
	//~ End UK2Node Interface.

private:
	/** Reset the A, B and Result pins to its initial wildcard state */
	void ResetToWildcards();
	
	/** returns the EaseFunc pin */
	UEdGraphPin* GetEaseFuncPin() const;

	/** Returns TRUE if the selected curve can feature customization, FALSE otherwise */
	bool CanCustomizeCurve() const
	{
		const auto EaseFuncPin = GetEaseFuncPin();
		return EaseFuncPin && EaseFuncPin->LinkedTo.Num() == 0;
	}

	/**
	* Notification from the editor that the user wants to change the PinType on a selected pin
	*
	* @param Pin	The pin the user wants to change the type for
	*/
	void ChangePinType(UEdGraphPin* Pin);

	/** Generates the available custom pins for the current ease function if applicable */
	void RefreshPinVisibility();

	/** Update MyPin with data from another pin. Return FALSE in case nothing was changed, TRUE otherwise. */
	bool UpdatePin(UEdGraphPin* MyPin, UEdGraphPin* OtherPin);

	/**
	* Takes the specified "MutatablePin" and sets its 'PinToolTip' field (according
	* to the specified description)
	*
	* @param   MutatablePin	The pin you want to set tool-tip text on
	* @param   PinDescription	A string describing the pin's purpose
	*/
	void SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const;

	/** Called right after the wildcard pins has been assigned a value */
	void GenerateExtraPins();

	/** Name of the kismet ease function to be called */
	UPROPERTY()
	FName EaseFunctionName;

	/** Tooltip text for this node. */
	FText NodeTooltip;

	/** The "EaseFunc" input pin, used to enable/disable easing customization */
	UEdGraphPin* CachedEaseFuncPin;
};
