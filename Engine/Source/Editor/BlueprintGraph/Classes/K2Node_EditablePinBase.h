// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_EditablePinBase.generated.h"

class FArchive;
class FFeedbackContext;
class FOutputDevice;
class UFunction;
class UObject;

USTRUCT()
struct FUserPinInfo
{
	GENERATED_USTRUCT_BODY()

	FUserPinInfo()
		: DesiredPinDirection(EGPD_MAX)
	{}

	/** Constructs a FUserPinInfo to match an existing Pin */
	explicit FUserPinInfo(const UEdGraphPin& InPin);

	/** The name of the pin, as defined by the user */
	UPROPERTY()
	FName PinName;

	/** Type info for the pin */
	UPROPERTY()
	struct FEdGraphPinType PinType;

	/** Desired direction for the pin. The direction will be forced to work with the node if necessary */
	UPROPERTY()
	TEnumAsByte<EEdGraphPinDirection> DesiredPinDirection;

	/** The default value of the pin */
	UPROPERTY()
	FString PinDefaultValue;

	friend FArchive& operator <<(FArchive& Ar, FUserPinInfo& Info);

};

// This structure describes metadata associated with a user declared function or macro
// It will be turned into regular metadata during compilation
USTRUCT()
struct FKismetUserDeclaredFunctionMetadata
{
	GENERATED_BODY()

	UPROPERTY()
	FText ToolTip;

	UPROPERTY()
	FText Category;

	UPROPERTY()
	FText Keywords;

	UPROPERTY()
	FText CompactNodeTitle;

	UPROPERTY()
	FLinearColor InstanceTitleColor;

	UPROPERTY()
	FString DeprecationMessage;

	UPROPERTY()
	bool bIsDeprecated;

	UPROPERTY()
	bool bCallInEditor;

	UPROPERTY()
	bool bThreadSafe;

	UPROPERTY()
	bool bIsUnsafeDuringActorConstruction;

	/** Cached value for whether or not the graph has latent functions, positive for TRUE, zero for FALSE, and INDEX_None for undetermined */
	UPROPERTY(transient)
	int8 HasLatentFunctions;

public:
	FKismetUserDeclaredFunctionMetadata()
		: InstanceTitleColor(FLinearColor::White)
		, bIsDeprecated(false)
		, bCallInEditor(false)
		, bThreadSafe(false)
		, bIsUnsafeDuringActorConstruction(false)
		, HasLatentFunctions(INDEX_NONE)
	{
	}

	/** Set a metadata value on the function */
	BLUEPRINTGRAPH_API void SetMetaData(FName Key, FString&& Value);
	/** Set a metadata value on the function */
	BLUEPRINTGRAPH_API void SetMetaData(FName Key, const FStringView Value);
	/** Gets a metadata value on the function; asserts if the value isn't present. */
	BLUEPRINTGRAPH_API const FString& GetMetaData(FName Key) const;
	/** Clear metadata value on the function */
	BLUEPRINTGRAPH_API void RemoveMetaData(FName Key);
	/** Checks if there is metadata for a key */
	BLUEPRINTGRAPH_API bool HasMetaData(FName Key) const;
	/** Gets all metadata associated with this function */
	BLUEPRINTGRAPH_API const TMap<FName, FString>& GetMetaDataMap() const;

private:
	UPROPERTY()
	TMap<FName, FString> MetaDataMap;
};

UCLASS(abstract, MinimalAPI)
class UK2Node_EditablePinBase : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Whether or not this entry node should be user-editable with the function editor */
	UPROPERTY()
	uint32 bIsEditable:1;

	/** Pins defined by the user */
	TArray< TSharedPtr<FUserPinInfo> >UserDefinedPins;

	virtual bool IsEditable() const { return bIsEditable; }

	// UObject interface
	BLUEPRINTGRAPH_API virtual void Serialize(FArchive& Ar) override;
	BLUEPRINTGRAPH_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	BLUEPRINTGRAPH_API virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	BLUEPRINTGRAPH_API virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	// End of UObject interface

	// UEdGraphNode interface
	BLUEPRINTGRAPH_API virtual void AllocateDefaultPins() override;
	BLUEPRINTGRAPH_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool ShouldShowNodeProperties() const override { return bIsEditable; }
	// End of UK2Node interface

	/**
	 * Queries if a user defined pin of the passed type can be constructed on this node. Node will return false by default and must opt into this functionality
	 *
	 * @param			InPinType			The type info for the pin to create
	 * @param			OutErrorMessage		Only filled with an error if there is pin add support but there is an error with the pin type
	 * @return			TRUE if a user defined pin can be constructed
	 */
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) { return false; }

	/**
	 * Creates a UserPinInfo from the specified information, and also adds a pin based on that description to the node
	 *
	 * @param	InPinName				Name of the pin to create
	 * @param	InPinType				The type info for the pin to create
	 * @param	InDesiredDirection		Desired direction of the pin, will auto-correct if the direction is not allowed on the pin.
	 */
	BLUEPRINTGRAPH_API UEdGraphPin* CreateUserDefinedPin(const FName InPinName, const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, bool bUseUniqueName = true);

	/**
	 * Removes a pin from the user-defined array, and removes the pin with the same name from the Pins array
	 *
	 * @param	PinToRemove	Shared pointer to the pin to remove from the UserDefinedPins array.  Corresponding pin in the Pins array will also be removed
	 */
	BLUEPRINTGRAPH_API void RemoveUserDefinedPin( TSharedPtr<FUserPinInfo> PinToRemove );

	/**
	 * Removes from the user-defined array, and removes the pin with the same name from the Pins array
	 *
	 * @param	PinName name of pin to remove
	 */
	BLUEPRINTGRAPH_API void RemoveUserDefinedPinByName(const FName PinName);

	/**
	* Check if a pin with this name exists in the user defined pin set
	* 
	* @param	PinName name of pin check existence of
	* @return	True if a user defined pin with this name exists
	*/
	BLUEPRINTGRAPH_API bool UserDefinedPinExists(const FName PinName) const;

	/**
	 * Creates a new pin on the node from the specified user pin info.
	 * Must be overridden so each type of node can ensure that the pin is created in the proper direction, etc
	 * 
	 * @param	NewPinInfo		Shared pointer to the struct containing the info for this pin
	 */
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) { return nullptr; }

	/** Modifies the default value of an existing pin on the node, this will update both the UserPinInfo and the linked editor pin */
	BLUEPRINTGRAPH_API virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue);

	/** Copies default value data from the graph pins to the user pins, returns true if any were modified */
	BLUEPRINTGRAPH_API virtual bool UpdateUserDefinedPinDefaultValues();

	/**
	 * Creates function pins that are user defined based on a function signature.
	 */
	BLUEPRINTGRAPH_API virtual bool CreateUserDefinedPinsForFunctionEntryExit(const UFunction* Function, bool bForFunctionEntry);

	/**
	 * Can this node have execution wires added or removed?
	 */
	virtual bool CanModifyExecutionWires() { return false; }

	/**
	 * Can this node have pass-by-reference parameters?
	 */
	virtual bool CanUseRefParams() const { return false; }

	/**
	 * Should this node require 'const' for pass-by-reference parameters?
	 */
	virtual bool ShouldUseConstRefParams() const { return false;  }

private:
	/** Internal function that just updates the UEdGraphPin, separate to avoid recursive update calls */
	bool UpdateEdGraphPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, FString& NewDefaultValue);
};

