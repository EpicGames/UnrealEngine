// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_EnumEquality.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FName;
class FString;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI, meta=(Keywords = "=="))
class UK2Node_EnumEquality : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
 	virtual void PostReconstructNode() override;
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual bool ShouldDrawCompact() const override { return true; }
	virtual FText GetCompactNodeTitle() const override { return NSLOCTEXT("K2Node", "EqualEqual", "=="); }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	//~ End UK2Node Interface

	/** Get the return value pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetReturnValuePin() const;
	/** Get the first input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInput1Pin() const;
	/** Get the second input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetInput2Pin() const;

	/** Gets the name and class of the EqualEqual_ByteByte function */
	BLUEPRINTGRAPH_API virtual void GetConditionalFunction(FName& FunctionName, UClass** FunctionClass) const;
};

