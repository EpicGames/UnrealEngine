// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_EnumEquality.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_EnumInequality.generated.h"

class FName;
class UClass;
class UObject;

UCLASS(MinimalAPI, meta=(Keywords = "!="))
class UK2Node_EnumInequality : public UK2Node_EnumEquality
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetCompactNodeTitle() const override { return NSLOCTEXT("K2Node", "NotEqual", "!="); }
	//~ End UK2Node Interface

	/** Gets the name and class of the EqualEqual_ByteByte function */
	BLUEPRINTGRAPH_API virtual void GetConditionalFunction(FName& FunctionName, UClass** FunctionClass) const override;
};
