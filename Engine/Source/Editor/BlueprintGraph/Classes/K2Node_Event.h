// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_EventNodeInterface.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Event.generated.h"

class FArchive;
class FKismetCompilerContext;
class FNodeHandlingFunctor;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;
struct FEdGraphSchemaAction;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_Event : public UK2Node_EditablePinBase, public IK2Node_EventNodeInterface
{
	GENERATED_UCLASS_BODY()
	BLUEPRINTGRAPH_API static const FName DelegateOutputName;

	/** Name of function signature that this event implements */
	UPROPERTY()
	FName EventSignatureName_DEPRECATED;

	/** Class that the function signature is from. */
	UPROPERTY()
	TSubclassOf<class UObject> EventSignatureClass_DEPRECATED;

	/** Reference for the function this event is linked to */
	UPROPERTY()
	FMemberReference EventReference;

	/** If true, we are actually overriding this function, not making a new event with a signature that matches */
	UPROPERTY()
	uint32 bOverrideFunction:1;

	/** If true, this event is internal machinery, and should not be marked BlueprintCallable */
	UPROPERTY()
	uint32 bInternalEvent:1;

	/** If this is not an override, allow user to specify a name for the function created by this entry point */
	UPROPERTY()
	FName CustomFunctionName;

	/** Additional function flags to apply to this function */
	UPROPERTY()
	uint32 FunctionFlags;

	//~ Begin UObject Interface
	BLUEPRINTGRAPH_API virtual void Serialize(FArchive& Ar) override;
	BLUEPRINTGRAPH_API virtual void PostLoad() override;
	BLUEPRINTGRAPH_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	BLUEPRINTGRAPH_API virtual void AllocateDefaultPins() override;
	BLUEPRINTGRAPH_API virtual FText GetTooltipText() const override;
	BLUEPRINTGRAPH_API virtual FText GetKeywords() const override;	
	BLUEPRINTGRAPH_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	BLUEPRINTGRAPH_API virtual FLinearColor GetNodeTitleColor() const override;
	BLUEPRINTGRAPH_API virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	BLUEPRINTGRAPH_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	BLUEPRINTGRAPH_API virtual FName GetCornerIcon() const override;
	BLUEPRINTGRAPH_API virtual bool HasDeprecatedReference() const override;
	BLUEPRINTGRAPH_API virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	BLUEPRINTGRAPH_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	BLUEPRINTGRAPH_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	BLUEPRINTGRAPH_API virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;
	BLUEPRINTGRAPH_API virtual void FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	BLUEPRINTGRAPH_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	BLUEPRINTGRAPH_API virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsEntry() const override { return true; }
	BLUEPRINTGRAPH_API virtual bool NodeCausesStructuralBlueprintChange() const override;
	BLUEPRINTGRAPH_API virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const override;
	BLUEPRINTGRAPH_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	BLUEPRINTGRAPH_API virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	BLUEPRINTGRAPH_API virtual void PostReconstructNode() override;
	BLUEPRINTGRAPH_API virtual FString GetDocumentationLink() const override;
	BLUEPRINTGRAPH_API virtual FString GetDocumentationExcerptName() const override;
	BLUEPRINTGRAPH_API virtual FNodeHandlingFunctor* CreateNodeHandler(FKismetCompilerContext& CompilerContext) const override;
	BLUEPRINTGRAPH_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	BLUEPRINTGRAPH_API virtual FText GetToolTipHeading() const override;
	BLUEPRINTGRAPH_API virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	BLUEPRINTGRAPH_API virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

	//~ Begin IK2Node_EventNodeInterface Interface.
	BLUEPRINTGRAPH_API virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) override;
	//~ End IK2Node_EventNodeInterface Interface.

	/** Checks whether the parameters for this event node are compatible with the specified function entry node */
	BLUEPRINTGRAPH_API virtual bool IsFunctionEntryCompatible(const class UK2Node_FunctionEntry* EntryNode) const;
	/** Checks if this event node is implementing an interface event */
	BLUEPRINTGRAPH_API bool IsInterfaceEventNode() const;

	BLUEPRINTGRAPH_API UFunction* FindEventSignatureFunction() const;
	BLUEPRINTGRAPH_API void UpdateDelegatePin(bool bSilent = false);
	BLUEPRINTGRAPH_API FName GetFunctionName() const;
	virtual bool IsUsedByAuthorityOnlyDelegate() const { return false; }
	BLUEPRINTGRAPH_API virtual bool IsCosmeticTickEvent() const;

	/** Returns localized string describing replication settings. 
	 *		Calling - whether this function is being called ("sending") or showing implementation ("receiving"). Determined whether we output "Replicated To Server" or "Replicated From Client".
	 */
	static FText GetLocalizedNetString(uint32 NetFlags, bool Calling);

	/** Helper function to identify if two Event nodes are the same */
	static BLUEPRINTGRAPH_API bool AreEventNodesIdentical(const UK2Node_Event* InNodeA, const UK2Node_Event* InNodeB);

	/** Accessor to the delegate pin **/
	BLUEPRINTGRAPH_API UEdGraphPin* GetDelegatePin() const;

protected:
	void FixupEventReference(bool bForce = false);

private:
	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;
};

