// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "K2Node_EventNodeInterface.generated.h"

struct FEdGraphSchemaAction;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UK2Node_EventNodeInterface : public UInterface
{
	GENERATED_BODY()
};

class IK2Node_EventNodeInterface
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) = 0;
};
