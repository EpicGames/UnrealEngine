// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_ExecutionSequence.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UObject;

UCLASS(MinimalAPI, meta = (Keywords = "sequence"))
class UK2Node_ExecutionSequence : public UK2Node, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool CanEverInsertExecutionPin() const override { return true; }
	virtual bool CanEverRemoveExecutionPin() const override { return true; }
	//~ End UK2Node Interface

	// IK2Node_AddPinInterface interface
	virtual void AddInputPin() override;
	// End of IK2Node_AddPinInterface interface

	//~ Begin K2Node_ExecutionSequence Interface

	/** Gets a unique pin name, the next in the sequence */
	FName GetUniquePinName();

	/**
	 * Inserts a new execution pin, before the specified execution pin, into an execution node
	 *
	 * @param	PinToInsertBefore	The pin to insert a new pin before on the node
	 */
	BLUEPRINTGRAPH_API void InsertPinIntoExecutionNode(UEdGraphPin* PinToInsertBefore, EPinInsertPosition Position);

	/**
	 * Removes the specified execution pin from an execution node
	 *
	 * @param	TargetPin	The pin to remove from the node
	 */
	BLUEPRINTGRAPH_API void RemovePinFromExecutionNode(UEdGraphPin* TargetPin);

	/** Whether an execution pin can be removed from the node or not */
	BLUEPRINTGRAPH_API bool CanRemoveExecutionPin() const;

	// @todo document
	BLUEPRINTGRAPH_API UEdGraphPin* GetThenPinGivenIndex(int32 Index);

private:
	// Returns the exec output pin name for a given 0-based index
	virtual FName GetPinNameGivenIndex(int32 Index) const;
};

