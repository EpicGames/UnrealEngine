// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "K2Node_ExternalGraphInterface.generated.h"

class UEdGraph;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UK2Node_ExternalGraphInterface : public UInterface
{
	GENERATED_BODY()
};

class IK2Node_ExternalGraphInterface
{
	GENERATED_BODY()

public:
	// Get external graphs to display for this node
	virtual TArray<UEdGraph*> GetExternalGraphs() const = 0;
};
