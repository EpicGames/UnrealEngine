// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_FormatText.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_FormatText : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetPinDisplayName(const UEdGraphPin* Pin) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool IsNodePure() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface.

public:
	/** Adds a new pin to the node */
	BLUEPRINTGRAPH_API void AddArgumentPin();

	/** Returns the number of arguments currently available in the node */
	int32 GetArgumentCount() const { return PinNames.Num(); }

	/**
	 * Returns argument name based on argument index
	 *
	 * @param InIndex		The argument's index to find the name for
	 * @return				Returns the argument's name if available
	 */
	BLUEPRINTGRAPH_API FText GetArgumentName(int32 InIndex) const;

	/** Removes the argument at a given index */
	BLUEPRINTGRAPH_API void RemoveArgument(int32 InIndex);

	/**
	 * Sets an argument name
	 *
	 * @param InIndex		The argument's index to find the name for
	 * @param InName		Name to set the argument to
	 */
	BLUEPRINTGRAPH_API void SetArgumentName(int32 InIndex, FName InName);

	/** Swaps two arguments by index */
	BLUEPRINTGRAPH_API void SwapArguments(int32 InIndexA, int32 InIndexB);

	/** returns Format pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetFormatPin() const;

	/** Returns TRUE if the arguments are allowed to be edited */
	bool CanEditArguments() const { return GetFormatPin()->LinkedTo.Num() > 0; }

	/**
	 * Finds an argument pin by name, checking strings in a strict, case sensitive fashion
	 *
	 * @param InPinName		The pin name to check for
	 * @return				NULL if the pin was not found, otherwise the found pin.
	 */
	BLUEPRINTGRAPH_API UEdGraphPin* FindArgumentPin(const FName InPinName) const;

private:
	/** Synchronize the type of the given argument pin with the type its connected to, or reset it to a wildcard pin if there's no connection */
	void SynchronizeArgumentPinType(UEdGraphPin* Pin);

	/** Returns a unique pin name to use for a pin */
	FName GetUniquePinName();

private:
	/** When adding arguments to the node, their names are placed here and are generated as pins during construction */
	UPROPERTY()
	TArray<FName> PinNames;

	/** The "Format" input pin, always available on the node */
	UEdGraphPin* CachedFormatPin;

	/** Tooltip text for this node. */
	FText NodeTooltip;
};

