// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Text.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_FunctionTerminator.h"
#include "KismetCompilerMisc.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/StructOnScope.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_FunctionEntry.generated.h"

class FArchive;
class FObjectPreSaveContext;
class FStructOnScope;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;
class UObject;
class UStruct;
struct FBPVariableDescription;
struct FEdGraphPinType;

UCLASS(MinimalAPI)
class UK2Node_FunctionEntry : public UK2Node_FunctionTerminator
{
	GENERATED_UCLASS_BODY()

	/** If specified, the function that is created for this entry point will have this name.  Otherwise, it will have the function signature's name */
	UPROPERTY()
	FName CustomGeneratedFunctionName;

	/** Function metadata */
	UPROPERTY()
	struct FKismetUserDeclaredFunctionMetadata MetaData;

	/** Array of local variables to be added to generated function */
	UPROPERTY()
	TArray<FBPVariableDescription> LocalVariables;

	/** Whether or not to enforce const-correctness for const function overrides */
	UPROPERTY()
	bool bEnforceConstCorrectness;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual bool GetCanRenameNode() const override;
	virtual bool CanUserDeleteNode() const override { return false; }
	virtual bool HasDeprecatedReference() const override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	virtual FText GetTooltipText() const override;
	virtual void FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results) override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* InGraph) const override;
	virtual void PostPasteNode() override;
	virtual void AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsEntry() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void PostReconstructNode() override;
	virtual void ClearCachedBlueprintData(UBlueprint* Blueprint) override;
	virtual void FixupPinStringDataReferences(FArchive* SavingArchive) override;
	virtual ERenamePinResult RenameUserDefinedPinImpl(FName OldName, FName NewName, bool bTest) override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanUseRefParams() const override { return true; }
	virtual bool ShouldUseConstRefParams() const override;
	virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue) override;
	//~ End UK2Node_EditablePinBase Interface

	//~ Begin K2Node_FunctionTerminator Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	//~ End K2Node_FunctionTerminator Interface

	/** Gets the UFunction and function variable cache structure that should be used for serialization fixups for local variables. If bForceRefresh is true it will always recreate the cache */
	BLUEPRINTGRAPH_API TSharedPtr<FStructOnScope> GetFunctionVariableCache(bool bForceRefresh = false);

	/** Copies data from the local variable defaults into the variable cache */
	BLUEPRINTGRAPH_API bool RefreshFunctionVariableCache();

	/** Handles updating loaded default values, by going default string into variable cache and back, if bForceRefresh it will happen even if the cache is already setup */
	BLUEPRINTGRAPH_API bool UpdateLoadedDefaultValues(bool bForceRefresh = false);

	// Removes an output pin from the node
	BLUEPRINTGRAPH_API void RemoveOutputPin(UEdGraphPin* PinToRemove);

	// Returns pin for the automatically added WorldContext parameter (used only by BlueprintFunctionLibrary).
	BLUEPRINTGRAPH_API UEdGraphPin* GetAutoWorldContextPin() const;

	/** Retrieves the function flags from the UFunction that this function entry node represents */
	BLUEPRINTGRAPH_API int32 GetFunctionFlags() const;

	/** Retrieves the extra flags set on this node. */
	int32 GetExtraFlags() const
	{
		return ExtraFlags;
	}

	/** Set the extra flags on this node */
	void SetExtraFlags(int32 InFlags)
	{
		ExtraFlags = (InFlags & ~FUNC_Native);
	}

	void AddExtraFlags(int32 InFlags)
	{
		ExtraFlags |= InFlags;
	}

	void ClearExtraFlags(int32 InFlags)
	{
		ExtraFlags &= ~InFlags;
	}
	
	/** Used to safely check whether the passed in flag is set. */
	bool HasAnyExtraFlags(int32 FlagsToCheck) const
	{
		return (ExtraFlags & FlagsToCheck) != 0 || FlagsToCheck == ~0;
	}

	/** Used to safely check whether all of the passed in flags are set. */
	bool HasAllExtraFlags(int32 FlagsToCheck) const
	{
		return ((ExtraFlags & FlagsToCheck) == FlagsToCheck);
	}

protected:
	/** Copies data from any local variables matching properties in VariableStruct into the VariableStructData */
	BLUEPRINTGRAPH_API bool UpdateVariableStructFromDefaults(const UStruct* VariableStruct, uint8* VariableStructData);

	/** Copies data from VariableStruct into the local variables */
	BLUEPRINTGRAPH_API bool UpdateDefaultsFromVariableStruct(const UStruct* VariableStruct, uint8* VariableStructData);

	/** Any extra flags that the function may need */
	UPROPERTY()
	int32 ExtraFlags;

	/** Holds a an in-memory representation of the UFunction struct, used to fixup local and user variables */
	TSharedPtr<FStructOnScope> FunctionVariableCache;

	/** True if we've updated the default values on this node at least once */
	bool bUpdatedDefaultValuesOnLoad;
};

