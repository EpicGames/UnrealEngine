// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_FunctionTerminator.h"
#include "KismetCompilerMisc.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_FunctionResult.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;
struct FUserPinInfo;

UCLASS(MinimalAPI)
class UK2Node_FunctionResult : public UK2Node_FunctionTerminator
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsExit() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const;
	virtual void PostPlacedNewNode();
	virtual void PostPasteNode() override;
	virtual bool CanUserDeleteNode() const override;
	virtual void FixupPinStringDataReferences(FArchive* SavingArchive) override;
	virtual ERenamePinResult RenameUserDefinedPinImpl(FName OldName, FName NewName, bool bTest) override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	//~ End UK2Node_EditablePinBase Interface

	//~ Begin K2Node_FunctionTerminator Interface
	virtual void PromoteFromInterfaceOverride(bool bIsPrimaryTerminator = true);
	//~ End K2Node_FunctionTerminator Interface

	BLUEPRINTGRAPH_API TArray<UK2Node_FunctionResult*> GetAllResultNodes() const;

protected:
	void SyncWithEntryNode();
	void SyncWithPrimaryResultNode();
	void MakePinsEditable();
};

