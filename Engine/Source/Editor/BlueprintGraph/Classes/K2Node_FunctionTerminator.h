// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/MemberReference.h"
#include "K2Node_EditablePinBase.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_FunctionTerminator.generated.h"

class FArchive;
class FText;
class UFunction;
class UObject;
struct FEdGraphPinType;

UCLASS(abstract, MinimalAPI)
class UK2Node_FunctionTerminator : public UK2Node_EditablePinBase
{
	GENERATED_UCLASS_BODY()

	/** Reference to the function signature. This is only resolvable by default if this is an inherited function */
	UPROPERTY()
	FMemberReference FunctionReference;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;
	virtual FName CreateUniquePinName(FName SourcePinName) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostPasteNode() override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	//~ End UK2Node_EditablePinBase Interface

	/** Promotes the node from being a part of an interface override to a full function that allows for parameter and result pin additions */
	BLUEPRINTGRAPH_API virtual void PromoteFromInterfaceOverride(bool bIsPrimaryTerminator = true);

	/** Returns the UFunction that this node actually represents, this will work for both inherited and newly created functions */
	BLUEPRINTGRAPH_API UFunction* FindSignatureFunction() const;

private:
	/** (DEPRECATED) Function signature class. Replaced by the 'FunctionReference' property. */
	UPROPERTY()
	TSubclassOf<class UObject> SignatureClass_DEPRECATED;

	/** (DEPRECATED) Function signature name. Replaced by the 'FunctionReference' property. */
	UPROPERTY()
	FName SignatureName_DEPRECATED;
};

