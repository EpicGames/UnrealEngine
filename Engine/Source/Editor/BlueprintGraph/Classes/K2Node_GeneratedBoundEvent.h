// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_Event.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "K2Node_GeneratedBoundEvent.generated.h"

class FArchive;
class FMulticastDelegateProperty;
class FObjectProperty;
class UBlueprint;
class UClass;
class UDynamicBlueprintBinding;
class UEdGraph;
class UObject;

/**
 * Node used during generation of ubergraph pages.
 * This node should never be used at editor time, so various editor related features are not implemented
 */
UCLASS(MinimalAPI)
class UK2Node_GeneratedBoundEvent : public UK2Node_Event
{
	GENERATED_BODY()

public:

	/** Delegate property name that this event is associated with */
	UPROPERTY()
	FName DelegatePropertyName;

	/** Delegate property's owner class that this event is associated with */
	UPROPERTY()
	TObjectPtr<UClass> DelegateOwnerClass;

	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ReconstructNode() override;
	//~ End UEdGraphNode Interface
	
	//~ Begin K2Node Interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End K2Node Interface

	//~ Begin K2Node_Event Interface
	virtual bool IsUsedByAuthorityOnlyDelegate() const override;
	//~ End K2Node_Event Interface

	/** Return the delegate property that this event is bound to */
	BLUEPRINTGRAPH_API FMulticastDelegateProperty* GetTargetDelegateProperty() const;

	/** Gets the proper display name for the property */
	BLUEPRINTGRAPH_API FText GetTargetDelegateDisplayName() const;

	/** Initialize this node with the provided delegate */
	BLUEPRINTGRAPH_API void InitializeGeneratedBoundEventParams(const FMulticastDelegateProperty* InDelegateProperty);

private:

	/** Returns true if there is a delegate on this blueprint with a name that matches ComponentPropertyName */
	bool IsDelegateValid() const;
};

