// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_ConstructObjectFromClass.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GenericCreateObject.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FKismetCompilerContext;
class UEdGraph;
class UK2Node_CallFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_GenericCreateObject : public UK2Node_ConstructObjectFromClass
{
	GENERATED_BODY()

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface.

	virtual bool UseWorldContext() const override { return false; }
	virtual bool UseOuter() const override { return true; }

	/**
	 * attaches a self node to the self pin of 'this' if the CallCreateNode function has DefaultToSelf in it's metadata
	 *
	 * @param	CompilerContext		the context to expand in - likely passed from ExpandNode
	 * @param	SourceGraph			the graph to expand in - likely passed from ExpandNode
	 * @param	CallCreateNode		the CallFunction node that 'this' is imitating
	 *
	 * @return	true on success.
	 */
	UE_API bool ExpandDefaultToSelfPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node_CallFunction* CallCreateNode);
};

#undef UE_API
