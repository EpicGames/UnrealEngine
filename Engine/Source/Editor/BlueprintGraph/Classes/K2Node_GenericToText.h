// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"

#include "K2Node_GenericToText.generated.h"


UCLASS(MinimalAPI)
class UK2Node_GenericToText : public UK2Node
{
	GENERATED_BODY()

	//~ Begin UK2Node Interface.
	virtual void AllocateDefaultPins() override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual void NodeConnectionListChanged() override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void PostReconstructNode() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsNodePure() const override
	{
		return true;
	}
	virtual bool NodeCausesStructuralBlueprintChange() const override
	{
		return true;
	}

	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface.

private:
	/** Synchronize the type of the given argument pin with the type its connected to, or reset it to a wildcard pin if there's no connection */
	void SynchronizeArgumentPinType();
};
