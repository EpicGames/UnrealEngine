// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeSignature.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetArrayItem.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class SWidget;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;
struct FLinearColor;

UCLASS(MinimalAPI, Category = "Utilities|Array", meta=(Keywords = "array"))
class UK2Node_GetArrayItem : public UK2Node
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual bool IsNodePure() const override { return true; }
	virtual void AllocateDefaultPins() override;
	virtual void PostReconstructNode() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual TSharedPtr<SWidget> CreateNodeImage() const override;
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool ShouldDrawCompact() const { return true; }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual FText GetMenuCategory() const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	// End of UK2Node interface

	/** Helper function to return the array pin */
	UEdGraphPin* GetTargetArrayPin() const { return Pins[0]; }

	/** Helper function to return the index pin */
	UEdGraphPin* GetIndexPin() const { return Pins[1]; }

	/** Helper function to return the result pin */
	UEdGraphPin* GetResultPin() const { return Pins[2]; }

	/** Sets weather we want the array item returned as a reference or by value (as a copy) */
	void SetDesiredReturnType(bool bAsReference);

private:
	/** Flips the node's desired 'ReturnByRef' setting & refreshes the node if necessary */
	void ToggleReturnPin();
	/** Propagates pin type to the Array pin and the Result pin */
	void PropagatePinType(FEdGraphPinType& InType);
	/** Determines if this node wants to return by ref, and if it is able to do so */
	bool IsSetToReturnRef() const;
	
	UPROPERTY()
	bool bReturnByRefDesired;
};
