// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetClassDefaults.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FName;
class FProperty;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_GetClassDefaults : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual void PostPlacedNewNode() override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End UK2Node Interface

public:
	/** Finds and returns the class input pin from the current set of pins. */
	UEdGraphPin* FindClassPin() const
	{
		return FindClassPin(Pins);
	}

	/** Retrieves the current input class type. */
	UClass* GetInputClass() const
	{
		return GetInputClass(FindClassPin());
	}

	void OnBlueprintClassModified(UBlueprint* TargetBlueprint);

protected:
	/**
	 * Finds and returns the class input pin.
	 *
	 * @param FromPins	A list of pins to search.
	 */
	BLUEPRINTGRAPH_API UEdGraphPin* FindClassPin(const TArray<UEdGraphPin*>& FromPins) const;

	/**
	* Determines the input class type from the given pin.
	*
	* @param FromPin	Input class pin.
	*/
	BLUEPRINTGRAPH_API UClass* GetInputClass(const UEdGraphPin* FromPin) const;

	/**
	 * Creates the full set of output pins (properties) from the given input class.
	 *
	 * @param InClass	Input class type.
	 */
	void CreateOutputPins(UClass* InClass);

	/** Will be called whenever the class pin selector changes its value. */
	void OnClassPinChanged();

private:
	/** Class pin name */
	static FName ClassPinName;

	/** Blueprint that we subscribed OnBlueprintChangedDelegate and OnBlueprintCompiledDelegate to */
	UPROPERTY(Transient)
	TObjectPtr<UBlueprint> BlueprintSubscribedTo;

	/** Blueprint.OnChanged delegate handle */
	FDelegateHandle OnBlueprintChangedDelegate;

	/** Blueprint.OnCompiled delegate handle */
	FDelegateHandle OnBlueprintCompiledDelegate;

	/** Output pin visibility control */
	UPROPERTY(EditAnywhere, Category=PinOptions, EditFixedSize)
	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	TArray<FName> OldShownPins;

	/** Whether or not to exclude object container properties */
	UPROPERTY()
	bool bExcludeObjectContainers;

	/** Whether or not to exclude object array properties (deprecated) */
	UPROPERTY()
	bool bExcludeObjectArrays_DEPRECATED;
};
