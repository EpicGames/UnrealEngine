// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetDataTableRow.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FString;
class UDataTable;
class UEdGraph;
class UEdGraphPin;
class UObject;
class UScriptStruct;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_GetDataTableRow : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void PostReconstructNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	UE_API virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void PreloadRequiredAssets() override;
	UE_API virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	//~ End UK2Node Interface

	/** Get the return type of our struct */
	UE_API UScriptStruct* GetReturnTypeForStruct();

	/** Get the Data Table input pin */
	UE_API UEdGraphPin* GetDataTablePin(const TArray<UEdGraphPin*>* InPinsToSearch=NULL) const;
	/** Get the spawn transform input pin */	
	UE_API UEdGraphPin* GetRowNamePin() const;
    /** Get the exec output pin for when the row was not found */
	UE_API UEdGraphPin* GetRowNotFoundPin() const;
	/** Get the result output pin */
	UE_API UEdGraphPin* GetResultPin() const;

	/** Get the type of the TableRow to return */
	UE_API UScriptStruct* GetDataTableRowStructType() const;

	UE_API void OnDataTableRowListChanged(const UDataTable* DataTable);
private:
	/**
	 * Takes the specified "MutatablePin" and sets its 'PinToolTip' field (according
	 * to the specified description)
	 * 
	 * @param   MutatablePin	The pin you want to set tool-tip text on
	 * @param   PinDescription	A string describing the pin's purpose
	 */
	UE_API void SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const;

	/** Set the return type of our struct */
	UE_API void SetReturnTypeForStruct(UScriptStruct* InClass);
	/** Queries for the authoritative return type, then modifies the return pin to match */
	UE_API void RefreshOutputPinType();
	/** Triggers a refresh which will update the node's widget; aimed at updating the dropdown menu for the RowName input */
	UE_API void RefreshRowNameOptions();

	/** Tooltip text for this node. */
	FText NodeTooltip;

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};

#undef UE_API
