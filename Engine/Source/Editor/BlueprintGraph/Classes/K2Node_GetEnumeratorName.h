// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetEnumeratorName.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FString;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_GetEnumeratorName : public UK2Node
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetCompactNodeTitle() const override;
	virtual bool ShouldDrawCompact() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	virtual void PostReconstructNode() override;
	virtual void NodeConnectionListChanged() override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface

protected:

	static FName EnumeratorPinName;

private:

	UEnum* GetConnectedEnum() const;
	virtual FName GetFunctionName() const;
};
