// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_GetEnumeratorName.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetEnumeratorNameAsString.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_GetEnumeratorNameAsString : public UK2Node_GetEnumeratorName
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool ShouldDrawCompact() const override { return false; }
	//~ End UK2Node Interface

private:

	virtual FName GetFunctionName() const override;
};

