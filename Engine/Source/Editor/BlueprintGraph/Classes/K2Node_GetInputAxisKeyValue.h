// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeSignature.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetInputAxisKeyValue.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UClass;
class UDynamicBlueprintBinding;
class UEdGraph;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI, meta=(Keywords = "Get"))
class UK2Node_GetInputAxisKeyValue : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FKey InputAxisKey;

	// Prevents actors with lower priority from handling this input
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bConsumeInput:1;

	// Should the binding gather input even when the game is paused
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bExecuteWhenPaused:1;

	//~ Begin EdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	//~ End UK2Node Interface
	
	void Initialize(const FKey AxisKey);

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};
