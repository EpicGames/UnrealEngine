// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "K2Node_GetInputAxisKeyValue.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetInputVectorAxisValue.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UClass;
class UDynamicBlueprintBinding;
class UObject;

UCLASS(MinimalAPI, meta=(Keywords = "Get"))
class UK2Node_GetInputVectorAxisValue : public UK2Node_GetInputAxisKeyValue
{
	GENERATED_UCLASS_BODY()

	//~ Begin EdGraphNode Interface
	virtual FText GetTooltipText() const override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UK2Node Interface
	
	void Initialize(const FKey AxisKey);
};
