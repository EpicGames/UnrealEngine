// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_GetSubsystem.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class FString;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;
class USubsystem;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_GetSubsystem : public UK2Node
{
	GENERATED_BODY()
public:

	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API void Initialize(UClass* NodeClass);
	//~Begin UEdGraphNode interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual FText GetTooltipText() const override;
	//~End UEdGraphNode interface.

	virtual bool IsNodePure() const { return true; }
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual void GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const override;
	UE_API virtual FText GetMenuCategory() const override;

	/** Get the blueprint input pin */
	UE_API UEdGraphPin* GetClassPin(const TArray< UEdGraphPin* >* InPinsToSearch = nullptr) const;
	/** Get the world context input pin, can return NULL */
	UE_API UEdGraphPin* GetWorldContextPin() const;
	/** Get the result output pin */
	UE_API UEdGraphPin* GetResultPin() const;

	virtual bool ShouldDrawCompact() const override { return true; }

protected:
	UPROPERTY()
	TSubclassOf<USubsystem> CustomClass;
};

UCLASS()
class UK2Node_GetSubsystemFromPC : public UK2Node_GetSubsystem
{
	GENERATED_BODY()
public:

	//~Begin UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FText GetTooltipText() const override;
	//~End UEdGraphNode interface.

	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;

	/** Get the world context input pin, can return NULL */
	UEdGraphPin* GetPlayerControllerPin() const;

};

UCLASS()
class UK2Node_GetEngineSubsystem : public UK2Node_GetSubsystem
{
	GENERATED_BODY()
public:

	//~Begin UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FText GetTooltipText() const override;
	//~End UEdGraphNode interface.

	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	
};

UCLASS()
class UK2Node_GetEditorSubsystem : public UK2Node_GetSubsystem
{
	GENERATED_BODY()
public:

	//~Begin UEdGraphNode interface.
	virtual void AllocateDefaultPins() override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FText GetTooltipText() const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~End UEdGraphNode interface.

	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;

	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;

};

#undef UE_API
