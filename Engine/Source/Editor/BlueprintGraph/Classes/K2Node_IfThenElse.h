// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_IfThenElse.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI, meta=(Keywords = "if bool branch"))
class UK2Node_IfThenElse : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface.
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	//~ End K2Node Interface.

	/** Get the return value pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetElsePin() const;
	/** Get the condition pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetConditionPin() const;
};

