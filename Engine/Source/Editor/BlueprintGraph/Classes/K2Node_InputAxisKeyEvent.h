// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeSignature.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "K2Node_Event.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputAxisKeyEvent.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class UClass;
class UDynamicBlueprintBinding;
class UEdGraph;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_InputAxisKeyEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FKey AxisKey;

	// Prevents actors with lower priority from handling this input
	UPROPERTY(EditAnywhere, Category = "Input")
	uint32 bConsumeInput : 1;

	// Should the binding execute even when the game is paused
	UPROPERTY(EditAnywhere, Category = "Input")
	uint32 bExecuteWhenPaused : 1;

	// Should any bindings to this event in parent classes be removed
	UPROPERTY(EditAnywhere, Category = "Input")
	uint32 bOverrideParentBinding : 1;

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin EdGraphNode Interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	//~ End EdGraphNode Interface

	//~ Begin UK2Node Interface
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool ShouldShowNodeProperties() const override{ return true; }
	UE_API virtual UClass* GetDynamicBindingClass() const override;
	UE_API virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FBlueprintNodeSignature GetSignature() const override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End UK2Node Interface

	UE_API void Initialize(const FKey AxisKey);

private:
	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;
};

#undef UE_API
