// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintNodeSignature.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_EventNodeInterface.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputKey.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FEdGraphSchemaAction;

UCLASS(MinimalAPI)
class UK2Node_InputKey : public UK2Node, public IK2Node_EventNodeInterface
{
	GENERATED_UCLASS_BODY()

	// The key that is bound
	UPROPERTY(EditAnywhere, Category="Input")
	FKey InputKey;

	// Prevents actors with lower priority from handling this input
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bConsumeInput:1;

	// Should the binding execute even when the game is paused
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bExecuteWhenPaused:1;

	// Should any bindings to this event in parent classes be removed
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bOverrideParentBinding:1;

	// Does this binding require the control key on PC or the command key on Mac to be held
	UPROPERTY(EditAnywhere, Category="Modifier")
	uint32 bControl:1;

	// Does this binding require the alt key to be held
	UPROPERTY(EditAnywhere, Category="Modifier")
	uint32 bAlt:1;

	// Does this binding require the shift key to be held
	UPROPERTY(EditAnywhere, Category="Modifier")
	uint32 bShift:1;

	// Does this binding require the windows key on PC or the control key on Mac to be held
	UPROPERTY(EditAnywhere, Category="Modifier")
	uint32 bCommand:1;

	//~ Begin UObject Interface
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UK2Node Interface.
	virtual bool ShouldShowNodeProperties() const override { return true; }
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	UE_API virtual FBlueprintNodeSignature GetSignature() const override;
	//~ End UK2Node Interface

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool IsCompatibleWithGraph(UEdGraph const* Graph) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin IK2Node_EventNodeInterface Interface.
	UE_API virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) override;
	//~ End IK2Node_EventNodeInterface Interface.


	UE_API FText GetModifierText() const;

	UE_API FName GetModifierName() const;

	UE_API FText GetKeyText() const;

	/** Get the 'pressed' input pin */
	UE_API UEdGraphPin* GetPressedPin() const;

	/** Get the 'released' input pin */
	UE_API UEdGraphPin* GetReleasedPin() const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

#undef UE_API
