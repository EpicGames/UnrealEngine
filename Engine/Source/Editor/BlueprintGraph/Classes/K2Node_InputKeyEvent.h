// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "Framework/Commands/InputChord.h"
#include "HAL/Platform.h"
#include "K2Node_Event.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputKeyEvent.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UClass;
class UDynamicBlueprintBinding;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_InputKeyEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	FInputChord InputChord;

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	uint32 bConsumeInput:1;

	UPROPERTY()
	uint32 bExecuteWhenPaused:1;

	UPROPERTY()
	uint32 bOverrideParentBinding:1;

	//~ Begin UK2Node Interface
	UE_API virtual UClass* GetDynamicBindingClass() const override;
	UE_API virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	//~ End UK2Node Interface
};

#undef UE_API
