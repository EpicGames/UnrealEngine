// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_EventNodeInterface.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputTouch.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class UEdGraph;
class UEdGraphPin;
class UEnum;
class UObject;
struct FEdGraphSchemaAction;

UCLASS(MinimalAPI)
class UK2Node_InputTouch : public UK2Node, public IK2Node_EventNodeInterface
{
	GENERATED_UCLASS_BODY()

	// Prevents actors with lower priority from handling this input
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bConsumeInput:1;

	// Should the binding execute even when the game is paused
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bExecuteWhenPaused:1;

	// Should any bindings to this event in parent classes be removed
	UPROPERTY(EditAnywhere, Category="Input")
	uint32 bOverrideParentBinding:1;

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UK2Node Interface.
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End UK2Node Interface

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin IK2Node_EventNodeInterface Interface.
	virtual TSharedPtr<FEdGraphSchemaAction> GetEventNodeAction(const FText& ActionCategory) override;
	//~ End IK2Node_EventNodeInterface Interface.

	BLUEPRINTGRAPH_API static UEnum* GetTouchIndexEnum();

	/** Get the 'pressed' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetPressedPin() const;

	/** Get the 'released' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetReleasedPin() const;

	/** Get the 'moved' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetMovedPin() const;

	BLUEPRINTGRAPH_API UEdGraphPin* GetLocationPin() const;
	BLUEPRINTGRAPH_API UEdGraphPin* GetFingerIndexPin() const;
	
};
