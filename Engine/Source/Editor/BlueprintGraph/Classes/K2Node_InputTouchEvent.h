// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/EnumAsByte.h"
#include "CoreMinimal.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/Platform.h"
#include "K2Node_Event.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputTouchEvent.generated.h"

class UClass;
class UDynamicBlueprintBinding;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_InputTouchEvent : public UK2Node_Event
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TEnumAsByte<EInputEvent> InputKeyEvent;

	UPROPERTY()
	uint8 bConsumeInput:1;

	UPROPERTY()
	uint8 bExecuteWhenPaused:1;

	UPROPERTY()
	uint8 bOverrideParentBinding:1;

	//~ Begin UK2Node Interface
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	//~ End UK2Node Interface
};
