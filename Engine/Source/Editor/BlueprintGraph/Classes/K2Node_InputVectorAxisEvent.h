// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "K2Node_InputAxisKeyEvent.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_InputVectorAxisEvent.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class UClass;
class UDynamicBlueprintBinding;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_InputVectorAxisEvent : public UK2Node_InputAxisKeyEvent
{
	GENERATED_UCLASS_BODY()

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UK2Node Interface
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual UClass* GetDynamicBindingClass() const override;
	virtual void RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UK2Node Interface
};
