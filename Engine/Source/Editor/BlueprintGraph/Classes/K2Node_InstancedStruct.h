// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "K2Node_CallFunction.h"
#include "K2Node_InstancedStruct.generated.h"

class UEdGraphPin;
class FBlueprintActionDatabaseRegistrar;

/**
 * Node customization for MakeInstancedStruct(), SetInstancedStructValue(), and GetInstancedStructValue().
 */
UCLASS()
class UK2Node_InstancedStruct : public UK2Node_CallFunction
{
	GENERATED_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	//~ End K2Node Interface

protected:
	//~ UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface
};

