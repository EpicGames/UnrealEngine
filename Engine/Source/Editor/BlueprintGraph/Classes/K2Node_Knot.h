// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Knot.generated.h"

class FBlueprintActionDatabaseRegistrar;
class INameValidatorInterface;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_Knot : public UK2Node
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override;
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual bool IsCompilerRelevant() const override { return false; }
	virtual UEdGraphPin* GetPassThroughPin(const UEdGraphPin* FromPin) const override;	
	virtual bool ShouldDrawNodeAsControlPointOnly(int32& OutInputPinIndex, int32& OutOutputPinIndex) const override { OutInputPinIndex = 0;  OutOutputPinIndex = 1; return true; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodeSafeToIgnore() const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PostReconstructNode() override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool IsNodePure() const override { return true; }
	// End of UK2Node interface

	UEdGraphPin* GetInputPin() const
	{
		return Pins[0];
	}

	UEdGraphPin* GetOutputPin() const
	{
		return Pins[1];
	}

	void PropagatePinType();

	BLUEPRINTGRAPH_API UEdGraphNode* GetExecTerminal() const;

private:
	void PropagatePinTypeFromDirection(bool bFromInput);

	/** Recursion guard boolean to prevent PropagatePinType from infinitely recursing if you manage to create a loop of knots */
	bool bRecursionGuard;
};
