// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Literal.generated.h"

class AActor;
class FBlueprintActionDatabaseRegistrar;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_Literal : public UK2Node
{
	GENERATED_UCLASS_BODY()

private:
	/** If this is an object reference literal, keep a reference here so that it can be updated as objects move around */
	UPROPERTY()
	TObjectPtr<class UObject> ObjectRef;

public:

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override { return true; }
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual AActor* GetReferencedLevelActor() const override;
	virtual bool DrawNodeAsVariable() const override { return true; }
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual void PostReconstructNode() override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UK2Node Interface

	/** Accessor for the value pin of the node */
	BLUEPRINTGRAPH_API UEdGraphPin* GetValuePin() const;

	/** Sets the LiteralValue for the pin, and changes the pin type, if necessary */
	BLUEPRINTGRAPH_API void SetObjectRef(UObject* NewValue);

	/** Gets the referenced object */
	UObject* GetObjectRef() const { return ObjectRef; }
};

