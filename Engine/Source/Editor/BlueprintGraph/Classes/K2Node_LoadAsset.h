// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_LoadAsset.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_LoadAsset : public UK2Node
{
	GENERATED_BODY()
public:
	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodePure() const override { return false; }
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FName GetCornerIcon() const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual FText GetMenuCategory() const override;
	virtual bool NodeCausesStructuralBlueprintChange() const { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	// End of UK2Node interface

	UE_API virtual const FName& GetInputPinName() const;
	UE_API virtual const FName& GetOutputPinName() const;

protected:
	UE_API virtual FName NativeFunctionName() const;

	UE_API virtual const FName& GetInputCategory() const;
	UE_API virtual const FName& GetOutputCategory() const;
};

UCLASS(MinimalAPI)
class UK2Node_LoadAssetClass : public UK2Node_LoadAsset
{
	GENERATED_BODY()
public:
	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

protected:
	virtual FName NativeFunctionName() const override;

	virtual const FName& GetInputCategory() const override;
	virtual const FName& GetOutputCategory() const override;

	virtual const FName& GetInputPinName() const override;
	virtual const FName& GetOutputPinName() const override;
};

UCLASS(MinimalAPI)
class UK2Node_LoadAssets : public UK2Node_LoadAsset
{
	GENERATED_BODY()
public:
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

protected:
	virtual FName NativeFunctionName() const override;

	virtual const FName& GetInputPinName() const override;
	virtual const FName& GetOutputPinName() const override;
};

#undef UE_API
