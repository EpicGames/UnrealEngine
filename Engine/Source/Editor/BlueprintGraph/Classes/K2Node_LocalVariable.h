// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_LocalVariable.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UObject;
struct FEdGraphPinType;

UCLASS(MinimalAPI, deprecated)
class UDEPRECATED_K2Node_LocalVariable : public UK2Node_TemporaryVariable
{
	GENERATED_UCLASS_BODY()

	/** If this is not an override, allow user to specify a name for the function created by this entry point */
	UPROPERTY()
	FName CustomVariableName;

	/** The local variable's assigned tooltip */
	UPROPERTY()
	FText VariableTooltip;

	//~ Begin UEdGraphNode Interface.
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual void PostPlacedNewNode() override;
	virtual void PostPasteNode() override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual void ReconstructNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override {}
	//~ End UK2Node Interface.

	/**
	 * Assigns the local variable a type
	 *
	 * @param InVariableType		The type to assign this local variable to
	 */
	BLUEPRINTGRAPH_API void ChangeVariableType(const FEdGraphPinType& InVariableType);
};
