// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeSignature.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_Tunnel.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "K2Node_MacroInstance.generated.h"

class FArchive;
class UBlueprint;
class UObject;
struct FKismetUserDeclaredFunctionMetadata;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_MacroInstance : public UK2Node_Tunnel
{
	GENERATED_UCLASS_BODY()

private:
	/** A macro is like a composite node, except that the associated graph lives
	  * in another blueprint, and can be instanced multiple times. */
	UPROPERTY()
	TObjectPtr<class UEdGraph> MacroGraph_DEPRECATED;

	UPROPERTY()
	FGraphReference MacroGraphReference;

public:
	/** Stored type info for what type the wildcard pins in this macro should become. */
	UPROPERTY()
	struct FEdGraphPinType ResolvedWildcardType;

	/** Whether we need to reconstruct the node after the pins have changed */
	bool bReconstructNode;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void PreloadRequiredAssets() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetKeywords() const override;
	virtual void PostPasteNode() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool CanUserDeleteNode() const override { return true; }
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual void NodeConnectionListChanged() override;
	virtual FString GetDocumentationLink() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual UObject* GetJumpTargetForDoubleClick() const override { return GetMacroGraph(); }
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool DrawNodeAsExit() const override { return false; }
	virtual bool DrawNodeAsEntry() const override { return false; }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual void PostReconstructNode() override;
	virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual FBlueprintNodeSignature GetSignature() const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	virtual FText GetCompactNodeTitle() const override;
	virtual bool ShouldDrawCompact() const override;
	virtual FName GetCornerIcon() const override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override { return false; }
	//~ End UK2Node_EditablePinBase Interface

	// Begin UK2Node_Tunnel interface
	virtual void PostFixupAllWildcardPins(bool bInAllWildcardPinsUnlinked) override;
	// End UK2Node_Tunnel interface

	void SetMacroGraph(UEdGraph* Graph) { MacroGraphReference.SetGraph(Graph); }
	UEdGraph* GetMacroGraph() const { return MacroGraphReference.GetGraph(); }
	UBlueprint* GetSourceBlueprint() const { return MacroGraphReference.GetBlueprint(); }
	// Public function for inferring wildcards after macro expansion. Intended
	// for use only by the blueprint compiler. Inference is typically performed
	// automatically when a pin connection is made.
	BLUEPRINTGRAPH_API void InferWildcards(const TArray<UEdGraphNode*>& InNodes) const;

	// Finds the associated metadata for the macro instance if there is any; this function is not particularly fast.
	BLUEPRINTGRAPH_API static FKismetUserDeclaredFunctionMetadata* GetAssociatedGraphMetadata(const UEdGraph* AssociatedMacroGraph);
	static void FindInContentBrowser(TWeakObjectPtr<UK2Node_MacroInstance> MacroInstance);

private:
	// Helpers and implementation of 'smart' wildcard inference - which doesn't
	// just grab the first user type and overwrite every wildcard pin with it.
	// Instead it intelligently propagates types from the macro inputs throughout
	// the graph, and then back to any input pins:
	void InferWildcards();
	TArray<UEdGraphPin*> GetAllWildcardPins() const;
	void SmartInferWildcardsImpl(const TArray<UEdGraphNode*>& InNodes) const;

	/** Constructing FText strings can be costly, so we cache the node's tooltip */
	FNodeTextCache CachedTooltip;
};

