// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphNode.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "K2Node_MakeContainer.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class FString;
class UEdGraphPin;
class UObject;
struct FKismetFunctionContext;

class FKCHandler_MakeContainer : public FNodeHandlingFunctor
{
public:
	FKCHandler_MakeContainer(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;

protected:
	EKismetCompiledStatementType CompiledStatementType;
};


UCLASS(MinimalAPI, abstract)
class UK2Node_MakeContainer : public UK2Node, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

	/** The number of input pins to generate for this node */
	UPROPERTY()
	int32 NumInputs;

public:
	UE_API void RemoveInputPin(UEdGraphPin* Pin);

	UE_API UEdGraphPin* GetOutputPin() const;

	/** returns a reference to the output array pin of this node, which is responsible for defining the type */
	virtual FName GetOutputPinName() const PURE_VIRTUAL(UK2Node_MakeContainer::GetOutputPinName, return NAME_None;);
	UE_API virtual FName GetPinName(int32 PinIndex) const;
	UE_API virtual void GetKeyAndValuePins(TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const;

public:
	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual void PostReconstructNode() override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodePure() const override { return true; }
	UE_API virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override PURE_VIRTUAL(UK2Node_MakeContainer::CreateNodeHandler, return nullptr;);
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins);
	UE_API virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	// End of UK2Node interface

	// IK2Node_AddPinInterface interface
	UE_API virtual void AddInputPin() override;
	// End of IK2Node_AddPinInterface interface

protected:
	friend class FKismetCompilerContext;

	/** If needed, will clear all pins to be wildcards */
	UE_API void ClearPinTypeToWildcard();

	UE_API bool CanResetToWildcard() const;

	/** Helper function for context menu add pin to ensure transaction is set up correctly. */
	UE_API void InteractiveAddInputPin();

	/** Propagates the pin type from the output (set) pin to the inputs, to make sure types are consistent */
	UE_API void PropagatePinType();

	UE_API void SyncPinNames();

	EPinContainerType ContainerType;
};

#undef UE_API
