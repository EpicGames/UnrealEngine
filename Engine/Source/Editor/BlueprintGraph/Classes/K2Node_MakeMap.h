// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_MakeContainer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MakeMap.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraphPin;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_MakeMap : public UK2Node_MakeContainer
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual FText GetMenuCategory() const override;
	// End of UK2Node interface

	// UK2Node_MakeContainer interface
	UE_API virtual FName GetOutputPinName() const override;
	UE_API virtual FName GetPinName(int32 PinIndex) const override;
	UE_API virtual void GetKeyAndValuePins(TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const override;
	// UK2Node_MakeContainer interface

	// IK2Node_AddPinInterface interface
	UE_API virtual void AddInputPin() override;
	// End of IK2Node_AddPinInterface interface
};

#undef UE_API
