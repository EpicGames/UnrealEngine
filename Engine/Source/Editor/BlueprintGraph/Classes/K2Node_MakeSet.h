// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "K2Node_MakeContainer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MakeSet.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraphPin;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI)
class UK2Node_MakeSet : public UK2Node_MakeContainer
{
	GENERATED_UCLASS_BODY()

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual FText GetMenuCategory() const override;
	// End of UK2Node interface

	// UK2Node_MakeContainer interface
	virtual FName GetOutputPinName() const override;
	// UK2Node_MakeContainer interface
};
