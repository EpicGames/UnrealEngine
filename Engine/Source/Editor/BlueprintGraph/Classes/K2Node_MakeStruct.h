// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_StructMemberSet.h"
#include "K2Node_StructOperation.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MakeStruct.generated.h"

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class FObjectPreSaveContext;
class FProperty;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;
class UObject;
class UScriptStruct;

// Pure kismet node that creates a struct with specified values for each member
UCLASS(MinimalAPI)
class UK2Node_MakeStruct : public UK2Node_StructMemberSet
{
	GENERATED_UCLASS_BODY()

	/** Helper property to handle upgrades from an old system of displaying pins for
	 *	the override values that properties referenced as a conditional of being set in a struct */
	UPROPERTY()
	bool bMadeAfterOverridePinRemoval;

	/**
	* Returns false if:
	*   1. The Struct has a 'native make' method
	* Returns true if:
	*   1. The Struct is tagged as BlueprintType
	*   and
	*   2. The Struct has any property that is tagged as CPF_BlueprintVisible and is not CPF_BlueprintReadOnly
	*/
	BLUEPRINTGRAPH_API static bool CanBeMade(const UScriptStruct* Struct, bool bForInternalUse = false);
	
	/** Can this struct be used as a split pin */
	BLUEPRINTGRAPH_API static bool CanBeSplit(const UScriptStruct* Struct, UBlueprint* InBP);

	// UObject interface
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void PreloadRequiredAssets() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void PostPlacedNewNode() override;
	//~ End  UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool NodeCausesStructuralBlueprintChange() const override { return false; }
	virtual bool IsNodePure() const override { return true; }
	virtual bool DrawNodeAsVariable() const override { return false; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex)  const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override;
	//~ End K2Node Interface

protected:
	struct FMakeStructPinManager : public FStructOperationOptionalPinManager
	{
		const uint8* const SampleStructMemory;
		UBlueprint* OwningBP;
	public:
		FMakeStructPinManager(const uint8* InSampleStructMemory, UBlueprint* InOwningBP);

		bool HasAdvancedPins() const { return bHasAdvancedPins; }
	protected:
		virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override;
		virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const override;
		virtual bool CanTreatPropertyAsOptional(FProperty* TestProperty) const override;

		/** set by GetRecordDefaults(), mutable as it is a const function */
		mutable bool bHasAdvancedPins;
	};

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};
