// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/Blueprint.h" // for FBPVariableDescription
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MakeVariable.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class FProperty;
class UEdGraphPin;
class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_MakeVariable : public UK2Node
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY()
	FBPVariableDescription VariableType;

public:
	void SetupVariable(const FBPVariableDescription& InVariableType, UEdGraphPin* TargetInputPin, FKismetCompilerContext& CompilerContext, UFunction* Scope, const FProperty* Property );

	// UK2Node interface
	virtual bool IsNodePure() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	// End of UK2Node interface

	UEdGraphPin* GetOutputPin() const;
	const FBPVariableDescription& GetVariableType() const { return VariableType; }
};
