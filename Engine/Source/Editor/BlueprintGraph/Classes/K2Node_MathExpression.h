// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_Composite.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MathExpression.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FBlueprintActionDatabaseRegistrar;
class INameValidatorInterface;
class UObject;
struct FEdGraphPinType;

/**
* This node type acts like a collapsed node, a single node that represents
* a larger sub-network of nodes (contained within a sub-graph). This node will
* take the math expression it was named with, and attempt to convert it into a
* series of math nodes. If it is unsuccessful, then it generates a series of
* actionable errors.
*/
UCLASS(MinimalAPI)
class UK2Node_MathExpression : public UK2Node_Composite
{
	GENERATED_UCLASS_BODY()

public:

	// The math expression to evaluate
	UPROPERTY(EditAnywhere, Category = Expression)
	FString Expression;

	UPROPERTY()
	bool bMadeAfterRotChange;

public:
	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	UE_API virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	UE_API virtual void OnRenameNode(const FString& NewName) override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PostPlacedNewNode() override;
	UE_API virtual void ReconstructNode() override;
	UE_API virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results )  override;
	virtual bool ShouldMergeChildGraphs() const override { return ShouldExpandInsteadCompile(); }
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual bool IsNodePure() const { return !ShouldExpandInsteadCompile(); }
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface
	virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override { return false; }
	//~ End UK2Node_EditablePinBase Interface

private:
	/* Returns true, when the node can/should not be optimized.*/
	UE_API bool ShouldExpandInsteadCompile() const;

	/**
	* Clears this node's sub-graph, and then takes the supplied string and
	* parses, and converts it into a series of new graph nodes.
	*
	* @param  NewExpression	The new string we wish to replace the current "Expression".
	*/
	UE_API void RebuildExpression(FString NewExpression);

	/**
	* Clears the cached expression string, deletes all generated node, clears
	* input pins, and resets the parser and graph generator (priming this node
	* for a new expression).
	*/
	UE_API void ClearExpression();

	/** Sanitizes an expression for display, removing outermost parentheses */
	UE_API FString SanitizeDisplayExpression(FString InExpression) const;

	/** Helper function to build the node's full title */
	UE_API FText GetFullTitle(FText InExpression) const;
private:
	/** Cached so we don't have to regenerate it when the graph is recompiled */
	TSharedPtr<class FCompilerResultsLog> CachedMessageLog;

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;

	/** Constructing the display string for a Math Expression is costly, so we cache it */
	FNodeTextCache CachedDisplayExpression;
};


#undef UE_API
