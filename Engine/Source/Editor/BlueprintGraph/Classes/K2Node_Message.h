// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Message.generated.h"

class UEdGraph;
class UEdGraphPin;
class UFunction;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_Message : public UK2Node_CallFunction
{
	GENERATED_UCLASS_BODY()


	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual FName GetCornerIcon() const override;
	virtual bool IsNodePure() const override { return false; }
	//~ End K2Node Interface

	//~ Begin K2Node_CallFunction Interface.
	virtual UEdGraphPin* CreateSelfPin(const UFunction* Function) override;

protected:
	virtual void FixupSelfMemberContext() override;

	virtual bool CanToggleNodePurity() const override { return false; }

	//~ End K2Node_CallFunction Interface.

	/**
	 * Helper function for expand node to expand the message to handle the case that a ULevelStreaming may be passed which needs to have its LevelScriptActor pulled out
	 *
	 * @param InCompilerContext			CompilerContext from ExpandNode
	 * @param InSourceGraph				Graph that the node are expanding from
	 * @param InStartingExecPin			Exec pin that connects to the message node, to correctly be mapped into the exec pins of the expanded nodes
	 * @param InMessageSelfPin			The message node's self pin, to correctly be mapped into the object pins of the expanded nodes
	 * @param InCastToInterfaceNode		Previously generated node to cast to the message's interface, to be wired into the expanded nodes
	 */
	void ExpandLevelStreamingHandlers(class FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph, class UEdGraphPin* InStartingExecPin, class UEdGraphPin* InMessageSelfPin, class UK2Node_DynamicCast* InCastToInterfaceNode);

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTitleTextTable CachedNodeTitles;
};
