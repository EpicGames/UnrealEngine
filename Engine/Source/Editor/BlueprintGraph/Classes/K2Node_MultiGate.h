// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_ExecutionSequence.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_MultiGate.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_MultiGate : public UK2Node_ExecutionSequence
{
	GENERATED_UCLASS_BODY()

	/** Reference to the integer that contains */
	UPROPERTY(transient)
	TObjectPtr<class UK2Node_TemporaryVariable> DataNode;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End UK2Node Interface

	//~ IK2Node_AddPinInterface interface
	virtual bool CanAddPin() const override;
	//~ End IK2Node_AddPinInterface interface

	/** Getting pin access */
	UEdGraphPin* GetResetPin() const;
	UEdGraphPin* GetIsRandomPin() const;
	UEdGraphPin* GetLoopPin() const;
	UEdGraphPin* GetStartIndexPin() const;
 	void GetOutPins(TArray<UEdGraphPin*>& OutPins) const;
	int32 GetNumOutPins() const;

	/** Gets the name and class of the MarkBit function from the KismetNodeHelperLibrary */
	void GetMarkBitFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the HasUnmarkedBit function from the KismetNodeHelperLibrary */
	void GetHasUnmarkedBitFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the MarkFirstUnmarkedBit function from the KismetNodeHelperLibrary */
	void GetUnmarkedBitFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the Greater_IntInt function from the KismetMathLibrary */
	void GetConditionalFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the EqualEqual_IntInt function from the KismetMathLibrary */
	void GetEqualityFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the NotEqual_BoolBool function from the KismetMathLibrary */
	void GetBoolNotEqualFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the PrintString function */
	void GetPrintStringFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the ClearAllBits function from the KismetNodeHelperLibrary */
	void GetClearAllBitsFunction(FName& FunctionName, UClass** FunctionClass);

private:
	// Returns the exec output pin name for a given 0-based index
 	virtual FName GetPinNameGivenIndex(int32 Index) const override;
};

