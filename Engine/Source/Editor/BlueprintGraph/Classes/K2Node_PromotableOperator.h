// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_AddPinInterface.h"
#include "K2Node_CallFunction.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_PromotableOperator.generated.h"

class FKismetCompilerContext;
class FString;
class UEdGraph;
class UEdGraphPin;
class UFunction;
class UGraphNodeContextMenuContext;
class UObject;
class UToolMenu;
struct FEdGraphPinType;

/** The promotable operator node allows for pin types to be promoted to others, i.e. float to double */
UCLASS(MinimalAPI)
class UK2Node_PromotableOperator : public UK2Node_CallFunction, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

public:

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual void GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const override;
	virtual FText GetTooltipText() const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual void NodeConnectionListChanged() override;
	virtual void PostPasteNode() override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* ChangedPin) override;
	virtual void PostReconstructNode() override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	// End of UK2Node interface

	// IK2Node_AddPinInterface interface
	virtual void AddInputPin() override;
	virtual bool CanAddPin() const override;
	virtual bool CanRemovePin(const UEdGraphPin* Pin) const override;
	virtual void RemoveInputPin(UEdGraphPin* Pin) override;
	// End of IK2Node_AddPinInterface interface

	/** Gets the additional pin that was created at this index */
	UEdGraphPin* GetAdditionalPin(int32 PinIndex) const;

	/** Attempts to find the error tolerance pin on this node. Can return nullptr. */
	UEdGraphPin* FindTolerancePin() const;

	// UK2Node_CallFunction interface
	virtual void SetFromFunction(const UFunction* Function);
	// End of UK2Node_CallFunction interface

	/** Recombines all split pins and sets the node to have default values (all wildcard pins) */
	BLUEPRINTGRAPH_API void ResetNodeToWildcard();
	
protected:
	// UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	// End of UK2Node_CallFunction interface

private:

	/**
	 * Add an additional pin to this node based on it's index. Will create a new wildcard pin
	 * with the appropriate name and notify that this pin has changed
	 */
	UEdGraphPin* AddInputPinImpl(int32 PinIndex);

	/** Returns true if this pin was added via the IK2Node_AddPinInterface interface */
	bool IsAdditionalPin(const UEdGraphPin& Pin) const;
		
	/** Returns true if the given pin is a tolerance pin for a comparison operator */
	bool IsTolerancePin(const UEdGraphPin& Pin) const;

	/** Helper function to recombine all split pins that this node may have */
	void RecombineAllSplitPins();

	/** Update the pins on this node with the function that is the best match given the current connections */
	void UpdateFromBestMatchingFunction();

	/** 
	* @return	True if this node has any connections attached to it, 
	*			or the default values have been modified by the user
	*/
	bool HasAnyConnectionsOrDefaults() const;

	/**
	* @return	True if this is a comparison operator node and there are 
	*			no connections/default values on input pins. The output pins
	*			is not considered in this check, because it will always be a bool
	*			so it does not determine the types.
	*/
	bool HasDeterminingComparisonTypes() const;

	/**
	* Attempts to create a cast node and connect it to a call function node
	* 
	* @return	True if the intermediate connection was made successfully	
	*/
	bool CreateIntermediateCast(UK2Node_CallFunction* SourceNode, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* InputPin, UEdGraphPin* OutputPin);

	/**
	* Spawn a new intermediate call function node with the given operator function 
	* and allocate its default pins. Place it next to the given Previous node 
	* 
	* @return	An intermediate call function node placed next to the "PreviousNode"
	*/
	UK2Node_CallFunction* CreateIntermediateNode(UK2Node_CallFunction* PreviousNode, const UFunction* const OpFunction, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph);

	/** 
	* Re-evaluates the in types on this node based on all the current connections
	* and the given pin that has changed. 
	* 
	* @param ChangedPin			The pin that has been change
	* @param bFromConversion	True if the pin modification is coming from a conversion.
	*/
	void EvaluatePinsFromChange(UEdGraphPin* ChangedPin, const bool bFromConversion = false);

	/** Helper to make sure we have the most up to date operation name. Returns true upon success */
	bool UpdateOpName();

	/** 
	* Update the pins on this node based on the given function. This modifies pins, meant 
	* for use by PinConnectionListChanged, not during node construction. 
	*/
	void UpdatePinsFromFunction(const UFunction* Function, UEdGraphPin* ChangedPin = nullptr, bool bIsFromConversion = false);

	/**
	* Returns all pins that have the EGPD_Input direction
	* @param bIncludeLinks	If true, than this will also include all the pins that are linked to the inputs.
	*						This is useful for gathering what the highest type may be
	*/
	TArray<UEdGraphPin*> GetInputPins(bool bIncludeLinks = false) const;

	/**
	* Get all pins on this node that can be considered when determining what function
	* is the best match
	*
	* @param OutArray	Array to populate with the considered pins
	*/
	void GetPinsToConsider(TArray<UEdGraphPin*>& OutArray) const;
	
	/**
	* Build the context menu to convert the given pin to a different type. 
	* If the pin is a wildcard, offer any possible types. Otherwise, offer 
	* to reset the node to wildcard.
	* 
	* @param Menu			The parent context menu
	* @param ContextPin		The pin to convert
	*/
	void CreateConversionMenu(struct FToolMenuSection& ConversionSection, UEdGraphPin* ContextPin) const;

	/** The name that this operation uses ("Add", "Multiply", etc) */
	UPROPERTY()
	FName OperationName;

	/** The current number of additional pins on this node */
	UPROPERTY()
	int32 NumAdditionalInputs;

	/** Guard to prevent possible recursive calls from ResetPinToAutogeneratedDefaultValue when breaking all links to this node */
	bool bDefaultValueReentranceGuard;

public:

	/** Returns the first pin with the EGPD_Output direction */
	BLUEPRINTGRAPH_API UEdGraphPin* GetOutputPin() const;

	const FName GetOperationName() const { return OperationName; }

	/**
	* Convert the given pin to the new pin type. If the new type is wildcard then break 
	* all connections and reset the node. Invalidates tooltips upon completion. 
	* This function is to be used by the context menu and includes a transaction. 
	* 
	* @param PinToChange		The Pin to convert
	* @param NewPinType			The pin's new type
	*/
	BLUEPRINTGRAPH_API void ConvertPinType(UEdGraphPin* PinToChange, const FEdGraphPinType NewPinType);

	/**
	* Returns true if the given pin can be converted via the context menu to another type
	*
	* @param Pin		The pin to consider
	*
	* @return bool		True if a conversion is possible
	*/
	BLUEPRINTGRAPH_API bool CanConvertPinType(const UEdGraphPin* Pin) const;

	/**
	 * Returns true if we can convert this node to another comparison operator type.
	 * This will only be true if the current operator is a comparison operator such as ==, !=, <, etc.
	 */
	static bool CanConvertComparisonOperatorNodeType(const UEdGraphNode* Node);

	/**
	 * Converts the given node to use the new operator name
	 */
	static void ConvertComparisonOperatorNode(UEdGraphNode* Node, const FName NewOpName);
};
