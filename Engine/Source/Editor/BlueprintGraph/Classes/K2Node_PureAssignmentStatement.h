// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_PureAssignmentStatement.generated.h"

class FName;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_PureAssignmentStatement : public UK2Node
{
	GENERATED_UCLASS_BODY()

	// Name of the Variable pin for this node
	static FName VariablePinName;
	// Name of the Value pin for this node
	static FName ValuePinName;
	// Name of the output pin for this node
	static FName OutputPinName;

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface

	/** Get the Then output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetOutputPin() const;
	/** Get the Variable input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetVariablePin() const;
	/** Get the Value input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetValuePin() const;
};

