// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_RemoveDelegate.generated.h"

UCLASS(MinimalAPI)
class UK2Node_RemoveDelegate : public UK2Node_BaseMCDelegate
{
	GENERATED_UCLASS_BODY()

public:
	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin of UK2Node Interface
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	//~ End UK2Node Interface
};
