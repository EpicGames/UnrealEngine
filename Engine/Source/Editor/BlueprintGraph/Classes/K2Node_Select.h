// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_AddPinInterface.h"
#include "KismetCompilerMisc.h"
#include "NodeDependingOnEnumInterface.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Select.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FName;
class FString;
class UEdGraph;
class UObject;
struct FLinearColor;

UCLASS(MinimalAPI, meta=(Keywords = "Ternary Select"))
class UK2Node_Select : public UK2Node, public INodeDependingOnEnumInterface, public IK2Node_AddPinInterface
{
	GENERATED_UCLASS_BODY()

private:
	/** The number of selectable options this node currently has */
	UPROPERTY()
	int32 NumOptionPins;

	/** The pin type of the index pin */
	UPROPERTY()
	FEdGraphPinType IndexPinType;

	/** Name of the enum being switched on */
	UPROPERTY()
	TObjectPtr<UEnum> Enum;

	/** List of the current entries in the enum (Pin Names) */
	UPROPERTY()
	TArray<FName> EnumEntries;

	/** List of the current entries in the enum (Pin Friendly Names) */
	UPROPERTY(Transient)
	TArray<FText> EnumEntryFriendlyNames;

	/** Whether we need to reconstruct the node after the pins have changed */
	UPROPERTY(Transient)
	uint8 bReconstructNode:1;

	uint8 bReconstructForPinTypeChange:1;

	/**
	* Respond to a change to the PinType for a selected pin
	* @param Pin	The pin that is changing type
	*/
	void OnPinTypeChanged(UEdGraphPin* Pin);

public:

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void NodeConnectionListChanged() override;
	virtual void PinTypeChanged(UEdGraphPin* Pin) override;
	virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	virtual void PostPasteNode() override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual void PostReconstructNode() override;
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End UK2Node Interface

	// INodeDependingOnEnumInterface
	virtual class UEnum* GetEnum() const override { return Enum; }
	virtual void ReloadEnum(class UEnum* InEnum) override;
	virtual bool ShouldBeReconstructedAfterEnumChanged() const override { return true; }
	// End of INodeDependingOnEnumInterface

	//~ IK2Node_AddPinInterface
	virtual bool CanAddPin() const override;
	virtual void AddInputPin() override;
	//~ End IK2Node_AddPinInterface

	/** Get the return value pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetReturnValuePin() const;
	/** Get the condition pin */
	BLUEPRINTGRAPH_API virtual UEdGraphPin* GetIndexPin() const;
	/** Returns a list of pins that represent the selectable options */
	BLUEPRINTGRAPH_API virtual void GetOptionPins(TArray<UEdGraphPin*>& OptionPins) const;

	/** Gets the name and class of the EqualEqual_IntInt function */
	BLUEPRINTGRAPH_API void GetConditionalFunction(FName& FunctionName, UClass** FunctionClass);
	/** Gets the name and class of the PrintString function */
	BLUEPRINTGRAPH_API static void GetPrintStringFunction(FName& FunctionName, UClass** FunctionClass);

	/** Removes the last option pin from the node */
	BLUEPRINTGRAPH_API void RemoveOptionPinToNode();
	/** Return whether an option pin can be removed to the node */
	BLUEPRINTGRAPH_API virtual bool CanRemoveOptionPinToNode() const;

	/**
	 * Notification from the editor that the user wants to change the PinType on a selected pin
	 *
	 * @param Pin	The pin the user wants to change the type for
	 */
	BLUEPRINTGRAPH_API void ChangePinType(UEdGraphPin* Pin);
	/**
	 * Whether the user can change the pintype on a selected pin
	 *
	 * @param Pin	The pin in question
	 */
	BLUEPRINTGRAPH_API bool CanChangePinType(UEdGraphPin* Pin) const;

	// Bind the options to a named enum 
	virtual void SetEnum(UEnum* InEnum, bool bForceRegenerate = false);

private:
	UEdGraphPin* GetIndexPinUnchecked() const;
};

