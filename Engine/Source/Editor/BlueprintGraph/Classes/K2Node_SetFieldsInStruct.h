// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_MakeStruct.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SetFieldsInStruct.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FProperty;
class FString;
class UBlueprint;
class UEdGraphPin;
class UObject;
struct FLinearColor;
struct FOptionalPinFromProperty;
template <typename FuncType> class TFunctionRef;

// Pure kismet node that creates a struct with specified values for each member
UCLASS(MinimalAPI)
class UK2Node_SetFieldsInStruct : public UK2Node_MakeStruct
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual bool CanSplitPin(const UEdGraphPin* Pin) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	//~ End  UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return false; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges) override {};
	//~ End K2Node Interface

	BLUEPRINTGRAPH_API static bool ShowCustomPinActions(const UEdGraphPin* InGraphPin, bool bIgnorePinsNum);

	enum EPinsToRemove
	{
		GivenPin,
		AllOtherPins
	};

	BLUEPRINTGRAPH_API void RemoveFieldPins(UEdGraphPin* InGraphPin, EPinsToRemove Selection);
	BLUEPRINTGRAPH_API bool AllPinsAreShown() const;
	BLUEPRINTGRAPH_API void RestoreAllPins();

protected:
	void BackTracePinPath(UEdGraphPin* OutputPin, TFunctionRef<void(UEdGraphPin*)> Predicate) const;

	struct FSetFieldsInStructPinManager : public FMakeStructPinManager
	{
	public:
		FSetFieldsInStructPinManager(const uint8* InSampleStructMemory, UBlueprint* BP) : FMakeStructPinManager(InSampleStructMemory, BP)
		{}

		virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override;
	};

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;

	mutable bool bRecursionGuard;
};
