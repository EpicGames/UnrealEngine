// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

#include "K2Node_SetForEach.generated.h"

/** Custom blueprint node for iterating Sets in blueprints */
UCLASS(MinimalAPI, CollapseCategories)
class UK2Node_SetForEach : public UK2Node
{
	GENERATED_BODY()

public:

	UK2Node_SetForEach();

	/** Pin Accessors */
	[[nodiscard]] BLUEPRINTGRAPH_API UEdGraphPin* GetSetPin() const;
	[[nodiscard]] BLUEPRINTGRAPH_API UEdGraphPin* GetBreakPin() const;
	[[nodiscard]] BLUEPRINTGRAPH_API UEdGraphPin* GetForEachPin() const;
	[[nodiscard]] BLUEPRINTGRAPH_API UEdGraphPin* GetValuePin() const;
	[[nodiscard]] BLUEPRINTGRAPH_API UEdGraphPin* GetCompletedPin() const;

	/** UK2Node interface */
	[[nodiscard]] virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	[[nodiscard]] virtual FText GetMenuCategory() const override;
	virtual void PostReconstructNode() override;
	/** End of UK2Node interface */

	/** UEdGraphNode interface */
	virtual void AllocateDefaultPins() override;
	virtual void ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	[[nodiscard]] virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	[[nodiscard]] virtual FText GetTooltipText() const override;
	[[nodiscard]] virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	[[nodiscard]] virtual bool ShouldShowNodeProperties() const override { return true; }
	/** End of UEdGraphNode interface */

	/** UObject interface */
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	/** End of UObject interface */

private:

	/** Pin Names */
	static const FName SetPinName;
	static const FName BreakPinName;
	static const FName ValuePinName;
	static const FName CompletedPinName;

	/** Determine if there is any configuration options that shouldn't be allowed */
	[[nodiscard]] bool CheckForErrors(const FKismetCompilerContext& CompilerContext);

	/** Updates the wildcard pins based on current links */
	void RefreshWildcardPins();

	/** A user editable hook for the display name of the value pin */
	UPROPERTY(EditDefaultsOnly, Category = "Set For Each")
	FString ValueName;
};
