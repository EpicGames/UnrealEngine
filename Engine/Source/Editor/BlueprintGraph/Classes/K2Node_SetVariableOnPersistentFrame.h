// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SetVariableOnPersistentFrame.generated.h"

class UObject;

/*
 *	FOR INTERNAL USAGE ONLY!
 */

UCLASS(MinimalAPI)
class UK2Node_SetVariableOnPersistentFrame : public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return false; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	//~ End K2Node Interface
};

