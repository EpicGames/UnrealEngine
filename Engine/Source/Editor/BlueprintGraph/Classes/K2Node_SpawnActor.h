// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SpawnActor.generated.h"

class FString;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_SpawnActor : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void PinDefaultValueChanged(UEdGraphPin* Pin) override;
	virtual FText GetTooltipText() const override;
	virtual bool IsDeprecated() const override;
	virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	//~ End UK2Node Interface


	/** Create new pins to show properties on archetype */
	void CreatePinsForClass(UClass* InClass);
	/** See if this is a spawn variable pin, or a 'default' pin */
	BLUEPRINTGRAPH_API bool IsSpawnVarPin(UEdGraphPin* Pin) const;

	/** Get the blueprint input pin */	
	BLUEPRINTGRAPH_API UEdGraphPin* GetBlueprintPin(const TArray<UEdGraphPin*>* InPinsToSearch=NULL) const;
	/** Get the world context input pin, can return NULL */	
	BLUEPRINTGRAPH_API UEdGraphPin* GetWorldContextPin() const;
	/** Get the spawn transform input pin */	
	BLUEPRINTGRAPH_API UEdGraphPin* GetSpawnTransformPin() const;
	/** Get the no collision fail input pin */	
	BLUEPRINTGRAPH_API UEdGraphPin* GetNoCollisionFailPin() const;
	/** Get the result output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetResultPin() const;

private:
	/** Get the class that we are going to spawn */
	BLUEPRINTGRAPH_API UClass* GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch=NULL) const;

	/** Tooltip text for this node. */
	FText NodeTooltip;

	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};
