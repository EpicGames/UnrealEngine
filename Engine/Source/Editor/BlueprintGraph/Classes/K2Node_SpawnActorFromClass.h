// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_ConstructObjectFromClass.h"
#include "K2Node_GenericCreateObject.h"
#include "KismetCompilerMisc.h"
#include "Textures/SlateIcon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SpawnActorFromClass.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FString;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FLinearColor;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_SpawnActorFromClass : public UK2Node_ConstructObjectFromClass
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	UE_API virtual void PostPlacedNewNode() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin UK2Node Interface
	virtual bool IsNodeSafeToIgnore() const override { return true; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_ConstructObjectFromClass Interface
	UE_API virtual UClass* GetClassPinBaseClass() const;
	UE_API virtual bool IsSpawnVarPin(UEdGraphPin* Pin) const override;
	//~ End UK2Node_ConstructObjectFromClass Interface

	
private:
	UE_API void FixupScaleMethodPin();
	
	/** Get the spawn transform input pin */	
	UE_API UEdGraphPin* GetSpawnTransformPin() const;
	/** Get the collision handling method input pin */
	UE_API UEdGraphPin* GetCollisionHandlingOverridePin() const;
	/** Get the collision handling method input pin */
	UE_API UEdGraphPin* GetScaleMethodPin() const;
	UE_API UEdGraphPin* TryGetScaleMethodPin() const;
	/** Get the actor owner pin */
	UE_API UEdGraphPin* GetOwnerPin() const;

	UE_API void MaybeUpdateCollisionPin(TArray<UEdGraphPin*>& OldPins);
};

#undef UE_API
