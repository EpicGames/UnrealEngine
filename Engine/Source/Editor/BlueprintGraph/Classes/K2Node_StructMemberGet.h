// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_StructOperation.h"
#include "KismetCompilerMisc.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_StructMemberGet.generated.h"

class FProperty;
class UObject;
struct FOptionalPinFromProperty;

// Pure kismet node that gets one or more member variables of a struct
UCLASS(MinimalAPI)
class UK2Node_StructMemberGet : public UK2Node_StructOperation
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=PinOptions, EditFixedSize)
	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodePure() const override { return true; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	// End of UK2Node interface

	// AllocateDefaultPins with just one member set
	BLUEPRINTGRAPH_API void AllocatePinsForSingleMemberGet(FName MemberName);

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;

	TArray<FName> OldShownPins;
};

