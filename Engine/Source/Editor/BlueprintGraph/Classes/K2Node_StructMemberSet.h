// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_StructOperation.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_StructMemberSet.generated.h"

class FName;
class FProperty;
class FBoolProperty;
class UEdGraphPin;
class UObject;

// Imperative kismet node that sets one or more member variables of a struct
UCLASS(MinimalAPI)
class UK2Node_StructMemberSet : public UK2Node_StructOperation
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=PinOptions, EditFixedSize)
	TArray<FOptionalPinFromProperty> ShowPinForProperties;

	// UObject interface
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsNodePure() const override { return false; }
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex)  const override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	// End of UK2Node interface

	// Helper for AllocateDefaultPins
	BLUEPRINTGRAPH_API void AllocateExecPins();

	// Returns the override condition that's bound to the given property, if one exists. An override is a edit condition flag that is not exposed as an optional input pin.
	BLUEPRINTGRAPH_API FBoolProperty* GetOverrideConditionForProperty(const FProperty* InProperty) const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;

	TArray<FName> OldShownPins;
};

