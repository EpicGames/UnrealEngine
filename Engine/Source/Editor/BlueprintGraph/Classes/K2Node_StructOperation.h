// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "K2Node.h"
#include "K2Node_Variable.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "K2Node_StructOperation.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI, abstract)
class UK2Node_StructOperation : public UK2Node_Variable
{
	GENERATED_UCLASS_BODY()

	/** Class that this variable is defined in.  */
	UPROPERTY()
	TObjectPtr<UScriptStruct> StructType;

	//~ Begin UEdGraphNode Interface
	virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput = nullptr) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	//virtual bool DrawNodeAsVariable() const override { return true; }
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	//~ End UK2Node Interface

protected:
	// Updater for subclasses that allow hiding pins
	struct FStructOperationOptionalPinManager : public FOptionalPinManager
	{
		//~ Begin FOptionalPinsUpdater Interface
		virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
		{
			Record.bCanToggleVisibility = true;
			Record.bShowPin = true;
			if (TestProperty)
			{
				Record.bShowPin = !TestProperty->HasMetaData(TEXT("PinHiddenByDefault"));
				if (Record.bShowPin)
				{
					if (UStruct* OwnerStruct = TestProperty->GetOwnerStruct())
					{
						Record.bShowPin = !OwnerStruct->HasMetaData(TEXT("HiddenByDefault"));
					}
				}
			}
		}

		virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const override;
		// End of FOptionalPinsUpdater interfac
	};

	bool DoRenamedPinsMatch(const UEdGraphPin* NewPin, const UEdGraphPin* OldPin, bool bStructInVariablesOut) const;

	/** Utility function to set up menu actions to set the struct type and promote category */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FMakeStructSpawnerAllowedDelegate, const UScriptStruct*, bool);
	void SetupMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FMakeStructSpawnerAllowedDelegate& AllowedDelegate, EEdGraphPinDirection PinDirectionToPromote) const;
};

