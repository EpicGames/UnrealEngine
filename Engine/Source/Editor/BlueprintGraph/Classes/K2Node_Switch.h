// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphPin.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "K2Node_Switch.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UObject;

UCLASS(MinimalAPI, abstract)
class UK2Node_Switch : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** If true switch has a default pin */
	UPROPERTY(EditAnywhere, Category=PinOptions)
	uint32 bHasDefaultPin:1;

	/* The function underpining the switch, if required */
	UPROPERTY()
	FName FunctionName;

	/** The class that the function is from. */
	UPROPERTY()
	TSubclassOf<class UObject> FunctionClass;

	// UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual FText GetMenuCategory() const override;
	virtual bool CanEverRemoveExecutionPin() const override { return true; }
	// End of UK2Node interface

	// UK2Node_Switch interface

	UE_API virtual FString GetExportTextForPin(const UEdGraphPin* Pin) const;

	/** Gets a unique pin name, the next in the sequence */
	virtual FName GetUniquePinName() { return NAME_None; }

	/** Gets the pin type from the schema for the subclass */
	virtual FEdGraphPinType GetPinType() const PURE_VIRTUAL(UK2Node_Switch::GetPinType, return FEdGraphPinType();)

	UE_API virtual FEdGraphPinType GetInnerCaseType() const;

	/**
	 * Adds a new execution pin to a switch node
	 */
	UE_API virtual void AddPinToSwitchNode();

	/**
	 * Removes the specified execution pin from an switch node
	 *
	 * @param	TargetPin	The pin to remove from the node
	 */
	UE_API virtual void RemovePinFromSwitchNode(UEdGraphPin* TargetPin);

	/** Whether an execution pin can be removed from the node or not */
	UE_API virtual bool CanRemoveExecutionPin(UEdGraphPin* TargetPin) const;

	/** Determines whether the add button should be enabled */
	virtual bool SupportsAddPinButton() const { return true; }

	/** Getting pin access */
	UE_API UEdGraphPin* GetSelectionPin() const;
	UE_API UEdGraphPin* GetDefaultPin() const;
	UE_API UEdGraphPin* GetFunctionPin() const;

	static UE_API FName GetSelectionPinName();

	UE_API virtual FName GetPinNameGivenIndex(int32 Index) const;

protected:
	virtual void CreateSelectionPin() {}
	virtual void CreateCasePins() {}
	UE_API virtual void CreateFunctionPin();
	virtual void RemovePin(UEdGraphPin* TargetPin) {}

private:
	// Editor-only field that signals a default pin setting change
	bool bHasDefaultPinValueChanged;
};

#undef UE_API
