// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_Switch.h"
#include "NodeDependingOnEnumInterface.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SwitchEnum.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FName;
class FString;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_SwitchEnum : public UK2Node_Switch, public INodeDependingOnEnumInterface
{
	GENERATED_UCLASS_BODY()

	/** Name of the enum being switched on */
	UPROPERTY()
	TObjectPtr<UEnum> Enum;

	/** List of the current entries in the enum */
	UPROPERTY()
	TArray<FName> EnumEntries;

	/** List of the current entries in the enum */
	UPROPERTY(Transient)
	TArray<FText> EnumFriendlyNames;

	// INodeDependingOnEnumInterface
	virtual class UEnum* GetEnum() const override { return Enum; }
	virtual void ReloadEnum(class UEnum* InEnum) override;
	virtual bool ShouldBeReconstructedAfterEnumChanged() const override {return true;}
	// End of INodeDependingOnEnumInterface

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual bool IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool CanEverRemoveExecutionPin() const override { return false; }
	virtual bool CanUserEditPinAdvancedViewFlag() const override { return true; }
	virtual void PreloadRequiredAssets() override;
	// End of UK2Node interface

	// UK2Node_Switch Interface
	virtual FEdGraphPinType GetPinType() const override;
	virtual void AddPinToSwitchNode() override;
	virtual void RemovePinFromSwitchNode(UEdGraphPin* TargetPin) override;
	virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const override;
	virtual bool SupportsAddPinButton() const { return false; }
	// End of UK2Node_Switch Interface
	
	/** Bind the switch to a named enum */
	BLUEPRINTGRAPH_API void SetEnum(UEnum* InEnum);

protected:
	/** Helper method to set-up pins */
	virtual void CreateCasePins() override;
	/** Helper method to set-up correct selection pin */
	virtual void CreateSelectionPin() override;
	
	/** Don't support removing pins from an enum */
	virtual void RemovePin(UEdGraphPin* TargetPin) override {}

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};
