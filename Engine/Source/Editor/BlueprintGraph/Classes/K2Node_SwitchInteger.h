// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_Switch.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SwitchInteger.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_SwitchInteger : public UK2Node_Switch
{
	GENERATED_UCLASS_BODY()

	/** Set the starting index for the node */
	UPROPERTY(EditAnywhere, Category=PinOptions, meta=(NoSpinbox=true))
	int32 StartIndex;

	// UObject interface
	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UK2Node_Switch Interface
	virtual FName GetUniquePinName() override;
	virtual FName GetPinNameGivenIndex(int32 Index) const override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual FEdGraphPinType GetPinType() const override;
	virtual bool CanRemoveExecutionPin(UEdGraphPin* TargetPin) const override;
	// End of UK2Node_Switch Interface

protected:
	virtual void CreateCasePins() override;
	virtual void CreateSelectionPin() override;
	virtual void RemovePin(UEdGraphPin* TargetPin) override;
};
