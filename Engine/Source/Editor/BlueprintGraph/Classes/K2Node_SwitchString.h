// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node_Switch.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_SwitchString.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_SwitchString : public UK2Node_Switch
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=PinOptions)
	TArray<FName> PinNames;

	UPROPERTY(EditAnywhere, Category=PinOptions)
	uint32 bIsCaseSensitive : 1;

	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	// End of UK2Node interface

	// UK2Node_Switch Interface
	BLUEPRINTGRAPH_API virtual void AddPinToSwitchNode() override;
	virtual FName GetUniquePinName() override;
	virtual FEdGraphPinType GetPinType() const override;
	// End of UK2Node_Switch Interface

	virtual FName GetPinNameGivenIndex(int32 Index) const override;

protected:
	virtual void CreateSelectionPin() override;
	virtual void CreateCasePins() override;
	virtual void RemovePin(UEdGraphPin* TargetPin) override;

	/** Sets the nodes function to call based on case sensitivity */
	void SetupCaseSensitivityFunction();
};
