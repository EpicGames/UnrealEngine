// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeSignature.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_TemporaryVariable.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_TemporaryVariable : public UK2Node
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	struct FEdGraphPinType VariableType;

	/** Whether or not this variable should be flagged with CPF_SaveGame, and inherit its name from the GUID of the macro that that gave rise to it */
	UPROPERTY()
	bool bIsPersistent;

	// get variable pin
	BLUEPRINTGRAPH_API UEdGraphPin* GetVariablePin();

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FString GetDescriptiveCompiledName() const override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* TargetGraph) const override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool IsNodePure() const override;
	virtual void PostReconstructNode() override;
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual FBlueprintNodeSignature GetSignature() const override;
	//~ End UK2Node Interface.

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

