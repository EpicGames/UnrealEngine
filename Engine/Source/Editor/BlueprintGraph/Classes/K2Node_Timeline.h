// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Timeline.generated.h"

class FBlueprintActionDatabaseRegistrar;
class FKismetCompilerContext;
class INameValidatorInterface;
class UEdGraph;
class UEdGraphPin;
class UObject;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UCLASS(MinimalAPI)
class UK2Node_Timeline : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** The name of the timeline. Used to name ONLY the member variable (Component). To obtain the name of timeline template use UTimelineTemplate::TimelineVariableNameToTemplateName */
	UPROPERTY()
	FName TimelineName;

	/** If the timeline is set to autoplay */
	UPROPERTY(Transient)
	uint32 bAutoPlay:1;

	/** Unique ID for the template we use, required to indentify the timeline after a paste */
	UPROPERTY()
	FGuid TimelineGuid;

	/** If the timeline is set to loop */
	UPROPERTY(Transient)
	uint32 bLoop:1;

	/** If the timeline is set to replicate */
	UPROPERTY(Transient)
	uint32 bReplicated:1;

	/** If the timeline should ignore global time dilation */
	UPROPERTY(Transient)
	uint32 bIgnoreTimeDilation : 1;

	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	virtual void PreloadRequiredAssets() override;
	virtual void DestroyNode() override;
	virtual void PostPasteNode() override;
	virtual void PrepareForCopying() override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	virtual void FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results )  override;
	virtual void OnRenameNode(const FString& NewName) override;
	virtual TSharedPtr<class INameValidatorInterface> MakeNameValidator() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDocumentationExcerptName() const override;
	virtual FName GetCornerIcon() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual UObject* GetJumpTargetForDoubleClick() const override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface.
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetToolTipHeading() const override;
	//~ End UK2Node Interface.

	/** Get the 'play' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetPlayPin() const;

	/** Get the 'play from start' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetPlayFromStartPin() const;

	/** Get the 'stop' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetStopPin() const;
	
	/** Get the 'update' output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetUpdatePin() const;

	/** Get the 'reverse' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetReversePin() const;

	/** Get the 'reverse from end' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetReverseFromEndPin() const;

	/** Get the 'finished' output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetFinishedPin() const;

	/** Get the 'newtime' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetNewTimePin() const;

	/** Get the 'setnewtime' input pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetSetNewTimePin() const;

	/** Get the 'Direction' output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetDirectionPin() const;

	/** Get the 'Direction' output pin */
	BLUEPRINTGRAPH_API UEdGraphPin* GetTrackPin(const FName TrackName) const;

	/** Try to rename the timeline
	 * @param NewName	The newname for the timeline
	 * @return bool	true if node was successfully renamed, false otherwise
	 */
	BLUEPRINTGRAPH_API bool RenameTimeline(const FString& NewName);

private:
	void ExpandForPin(UEdGraphPin* TimelinePin, const FName PropertyName, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph);
};

