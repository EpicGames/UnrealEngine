// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_EditablePinBase.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_Tunnel.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FCompilerResultsLog;
class UBlueprint;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FEdGraphPinType;

UCLASS(MinimalAPI)
class UK2Node_Tunnel : public UK2Node_EditablePinBase
{
	GENERATED_UCLASS_BODY()

	// A tunnel node either has output pins that came from another tunnel's input pins, or vice versa
	// Note: OutputSourceNode might be equal to InputSinkNode
	
	// The output pins of this tunnel node came from the input pins of OutputSourceNode
	UPROPERTY()
	TObjectPtr<UK2Node_Tunnel> OutputSourceNode;

	// The input pins of this tunnel go to the output pins of InputSinkNode
	UPROPERTY()
	TObjectPtr<UK2Node_Tunnel> InputSinkNode;

	// Whether this node is allowed to have inputs
	UPROPERTY()
	uint32 bCanHaveInputs:1;

	// Whether this node is allowed to have outputs
	UPROPERTY()
	uint32 bCanHaveOutputs:1;

	// The metadata for the function/subgraph associated with this tunnel node; it's only editable and used
	// on the tunnel entry node inside the subgraph or macro.  This structure is ignored on any other tunnel nodes.
	UPROPERTY()
	struct FKismetUserDeclaredFunctionMetadata MetaData;

	//~ Begin UEdGraphNode Interface.
	UE_API virtual void DestroyNode() override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual bool CanUserDeleteNode() const override;
	UE_API virtual bool CanDuplicateNode() const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual FName CreateUniquePinName(FName SourcePinName) const override;
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	UE_API virtual bool IsCompatibleWithGraph(const UEdGraph* InGraph) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface.
	UE_API virtual bool IsNodeSafeToIgnore() const override;
	UE_API virtual bool DrawNodeAsEntry() const override;
	UE_API virtual bool DrawNodeAsExit() const override;
	virtual bool NodeCausesStructuralBlueprintChange() const override { return true; }
	UE_API virtual void ClearCachedBlueprintData(UBlueprint* Blueprint) override;
	UE_API virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	UE_API virtual void FixupPinStringDataReferences(FArchive* SavingArchive) override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_EditablePinBase Interface.
	UE_API virtual UEdGraphPin* CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo) override;
	UE_API virtual bool CanModifyExecutionWires() override;
	UE_API virtual ERenamePinResult RenameUserDefinedPinImpl(const FName OldName, const FName NewName, bool bTest = false) override;
	virtual bool CanUseRefParams() const override { return true; }
	UE_API virtual bool CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage) override;
	UE_API virtual bool ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue) override;
	//~ End UK2Node_EditablePinBase Interface

protected:
	/**
	* Handles any work needed to be done after fixing up all wildcard pins during reconstruction
	*
	* @param bInAllWildcardPinsUnlinked	TRUE if all wildcard pins were unlinked
	*/
	virtual void PostFixupAllWildcardPins(bool bInAllWildcardPinsUnlinked) {}
	// Feature flag for 'smart' wildcard inference which allows for multiple 
	// types to be inferred.
	static UE_API bool ShouldDoSmartWildcardInference();

	// Utility function that subclasses must call after allocating their default pins
	UE_API void CacheWildcardPins();

	// Cache of the pins that were created as wildcards
	TArray<UEdGraphPin*> WildcardPins;

public:
	// The input pins of this tunnel go to the output pins of InputSinkNode (can be NULL).
	UE_API virtual UK2Node_Tunnel* GetInputSink() const;

	// The output pins of this tunnel node came from the input pins of OutputSourceNode (can be NULL).
	UE_API virtual UK2Node_Tunnel* GetOutputSource() const;
};



#undef UE_API
