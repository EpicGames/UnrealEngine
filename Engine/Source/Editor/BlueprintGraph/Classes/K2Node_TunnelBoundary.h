// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_TunnelBoundary.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FKismetCompilerContext;
class UK2Node_Tunnel;
class UObject;

UENUM()
enum class ETunnelBoundaryType : uint8
{
	Unknown = 0,
	EntrySite,
	InputSite,
	OutputSite
};

UCLASS(MinimalAPI)
class UK2Node_TunnelBoundary : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Base Name */
	UPROPERTY(Transient)
	FName BaseName;

	/** Node Type */
	UPROPERTY(Transient)
	ETunnelBoundaryType TunnelBoundaryType;

public:

	//~ Begin UEdGraphNode Interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(FKismetCompilerContext& CompilerContext) const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	//~ End UK2Node Interface

	/** Returns the type of tunnel boundary this node is */
	ETunnelBoundaryType GetTunnelBoundaryType() const { return TunnelBoundaryType; }

	/** Set node attributes based on the tunnel source node */
	UE_API void SetNodeAttributes(const UK2Node_Tunnel* SourceNode);
};

#undef UE_API
