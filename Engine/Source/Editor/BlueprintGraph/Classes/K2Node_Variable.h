// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Math/Color.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "K2Node_Variable.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FProperty;
class UActorComponent;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphPin;
class UObject;
class UStruct;
struct FBPVariableDescription;
struct FEdGraphPinType;
template <typename KeyType, typename ValueType> struct TKeyValuePair;

UENUM()
namespace ESelfContextInfo
{
	enum Type : int
	{
		Unspecified,
		NotSelfContext,
	};
}

UCLASS(MinimalAPI, abstract)
class UK2Node_Variable : public UK2Node
{
	GENERATED_UCLASS_BODY()

	/** Reference to variable we want to set/get */
	UPROPERTY(meta=(BlueprintSearchable="true", BlueprintSearchableHiddenExplicit="true", BlueprintSearchableFormatVersion="FIB_VER_VARIABLE_REFERENCE"))
	FMemberReference	VariableReference;

	UPROPERTY()
	TEnumAsByte<ESelfContextInfo::Type> SelfContextInfo;

protected:
	/** Class that this variable is defined in. Should be NULL if bSelfContext is true.  */
	UPROPERTY()
	TSubclassOf<class UObject>  VariableSourceClass_DEPRECATED;

	/** Name of variable */
	UPROPERTY()
	FName VariableName_DEPRECATED;

	/** Whether or not this should be a "self" context */
	UPROPERTY()
	uint32 bSelfContext_DEPRECATED:1;

	/**
	 * Remap a reference from one variable to another, if this variable is of class type 'MatchInVariableClass', and if linked to anything that is a child of 'RemapIfLinkedToClass'.
	 * Only intended for versioned fixup where redirects can't be applied.
	 */
	UE_API bool RemapRestrictedLinkReference(FName OldVariableName, FName NewVariableName, const UClass* MatchInVariableClass, const UClass* RemapIfLinkedToClass, bool bLogWarning);

public:
	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	UE_API virtual FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FString GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const override;
	UE_API virtual void ReconstructNode() override;
	UE_API virtual FString GetDocumentationLink() const override;
	UE_API virtual FString GetDocumentationExcerptName() const override;
	UE_API virtual FName GetCornerIcon() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void AutowireNewNode(UEdGraphPin* FromPin) override;
	UE_API virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	UE_API virtual void PostPasteNode() override;
	UE_API virtual bool HasDeprecatedReference() const override;
	UE_API virtual FEdGraphNodeDeprecationResponse GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual bool CanJumpToDefinition() const override;
	UE_API virtual void JumpToDefinition() const override;
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	UE_API virtual FString GetPinMetaData(FName InPinName, FName InKey) override;
	UE_API virtual bool HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool DrawNodeAsVariable() const override { return true; }
	UE_API virtual ERedirectType DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex)  const override;
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	UE_API virtual FText GetToolTipHeading() const override;
	UE_API virtual void GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const override;
	UE_API virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	UE_API virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) override;
	UE_API virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const override;
	//~ End K2Node Interface

	/** Set up this variable node from the supplied FProperty */
	UE_API void SetFromProperty(const FProperty* Property, bool bSelfContext, UClass* OwnerClass);

	/** Util to get variable name as a string */
	FString GetVarNameString() const
	{
		return GetVarName().ToString();
	}

	FText GetVarNameText() const
	{
		return FText::FromName( GetVarName() );
	}


	/** Util to get variable name */
	FName GetVarName() const
	{
		return VariableReference.GetMemberName();
	}

	/**
	 * Creates a reader or writer pin for a variable.
	 *
	 * @param	Direction	  	The direction of the variable access.
	 * @param	InPinName		Optional pin name, will default to the variable name if not included.
	 *
	 * @return	true if it succeeds, false if it fails.
	 */
	UE_API bool CreatePinForVariable(EEdGraphPinDirection Direction, FName InPinName = NAME_None);

	/** Creates 'self' pin */
	UE_API void CreatePinForSelf();

	/**
	 * Creates a reader or writer pin for a variable from an old pin.
	 *
	 * @param	Direction	  	The direction of the variable access.
	 * @param	OldPins			Old pins.
	 * @param	InPinName		Optional pin name, will default to the variable name if not included.
	 *
	 * @return	true if it succeeds, false if it fails.
	 */
	UE_API bool RecreatePinForVariable(EEdGraphPinDirection Direction, TArray<UEdGraphPin*>& OldPins, FName InPinName = NAME_None);

	/** Get the class to look for this variable in */
	UE_API UClass* GetVariableSourceClass() const;

	/** Get the FProperty for this variable node */
	UE_API FProperty* GetPropertyForVariable() const;
	UE_API FProperty* GetPropertyForVariableFromSkeleton() const;

	/** Returns true if the variable names match, this looks for redirectors */
	static UE_API bool DoesRenamedVariableMatch(FName OldVariableName, FName NewVariableName, UStruct* StructType);

private:
	/** 
	 * Gets the property for the variable on the owning class or on the owning class's sparse class data structure.
	 */
	UE_API FProperty* GetPropertyForVariable_Internal(UClass* OwningClass) const;

public:

	/** Accessor for the value output pin of the node */
	UE_API UEdGraphPin* GetValuePin() const;

	/** Validates there are no errors in the node */
	UE_API void CheckForErrors(const class UEdGraphSchema_K2* Schema, class FCompilerResultsLog& MessageLog);

	/**
	 * Utility method intended to serve as a choke point for various slate 
	 * widgets to grab an icon from (for a specified variable).
	 * 
	 * @param  VarScope		The scope that owns the variable in question.
	 * @param  VarName		The name of the variable you're querying for.
	 * @param  IconColorOut	A color out, further discerning the variable's type.
	 * @return A icon representing the specified variable's type.
	 */
	static UE_API FSlateIcon GetVariableIconAndColor(const UStruct* VarScope, FName VarName, FLinearColor& IconColorOut);

	/**
	 * Utility method intended to serve as a choke point for various slate 
	 * widgets to grab an icon from (for a specified variable pin type).
	 * 
	 * @param  InPinType	The pin type of the variable in question.
	 * @param  IconColorOut	A color out, further discerning the variable's type.
	 * @return A icon representing the specified variable's type.
	 */
	static UE_API FSlateIcon GetVarIconFromPinType(const FEdGraphPinType& InPinType, FLinearColor& IconColorOut);

protected:
	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API FBPVariableDescription const* GetBlueprintVarDescription() const;

	/**
	 * Utility function to retrieve actor component for variable property.
	 *
	 * @return Found UActorComponent for the FProperty, else nullptr
	 */
	UE_API const UActorComponent* GetActorComponent(const FProperty* VariableProperty) const;

	/** Adds the variable reference to the suppressed deprecation warnings list */
	UE_API void SuppressDeprecationWarning() const;

	/** Returns whether a Function Graph contains a parameter with the given name */
	static UE_API bool FunctionParameterExists(const UEdGraph* InFunctionGraph, const FName InParameterName);
};

#undef UE_API
