// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Text.h"
#include "K2Node_Variable.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "K2Node_VariableGet.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class FArchive;
class FProperty;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FBPVariableDescription;
struct FEdGraphPinType;

UENUM()
enum class EGetNodeVariation
{
	Pure,
	ValidatedObject,
	Branch,
};

UCLASS(MinimalAPI)
class UK2Node_VariableGet : public UK2Node_Variable
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UObject Interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	//~ Begin UEdGraphNode Interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const override;
	virtual bool IncludeParentNodeContextMenu() const override { return true; }
	UE_API virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool IsNodePure() const override { return CurrentVariation == EGetNodeVariation::Pure; }
	UE_API virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	UE_API virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface

	static UE_API FText GetPropertyTooltip(const FProperty* VariableProperty);
	static UE_API FText GetBlueprintVarTooltip(const FBPVariableDescription& VarDesc);

	/**
	 * Will change the node's purity, and reallocate pins accordingly (adding/
	 * removing exec pins).
	 *
	 * @param  bNewPurity  The new value for bNewPurity.
	 */
	void SetPurity(bool bNewPurity);

private:

	/** Adds pins required for the node to function in a impure manner */
	UE_API void CreateImpurePins(TArray<UEdGraphPin*>* InOldPinsPtr);

	/** Flips the node's purity (adding/removing exec pins as needed). */
	UE_API void TogglePurity(EGetNodeVariation BoundVariation);

	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;

	UPROPERTY()
	bool bIsPureGet_DEPRECATED = true;

	UPROPERTY()
	EGetNodeVariation CurrentVariation;
};

#undef UE_API
