// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Text.h"
#include "K2Node_Variable.h"
#include "KismetCompilerMisc.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "K2Node_VariableSet.generated.h"

class FProperty;
class UEdGraph;
class UEdGraphPin;
class UObject;
struct FBPVariableDescription;

UCLASS(MinimalAPI)
class UK2Node_VariableSet : public UK2Node_Variable
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual bool ShouldOverridePinNames() const override { return true; }
	virtual FText GetPinNameOverride(const UEdGraphPin& Pin) const override;
	virtual void ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UEdGraphNode Interface

	//~ Begin K2Node Interface
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	//~ End K2Node Interface

	/** Retrieves the output pin name for the node */
	FName GetVariableOutputPinName() const;

	BLUEPRINTGRAPH_API bool HasLocalRepNotify() const;
	BLUEPRINTGRAPH_API FName GetRepNotifyName() const;
	BLUEPRINTGRAPH_API bool ShouldFlushDormancyOnSet() const;
	BLUEPRINTGRAPH_API bool IsNetProperty() const;
	BLUEPRINTGRAPH_API bool IsFieldNotifyProperty() const;
	BLUEPRINTGRAPH_API bool HasFieldNotificationBroadcast() const;

	static FText GetPropertyTooltip(FProperty const* VariableProperty);
	static FText GetBlueprintVarTooltip(FBPVariableDescription const& VarDesc);

private:
	/** Sets up the tooltip for the output pin */
	void CreateOutputPinTooltip();

private:
	/** Constructing FText strings can be costly, so we cache the node's title/tooltip */
	FNodeTextCache CachedTooltip;
	FNodeTextCache CachedNodeTitle;
};

