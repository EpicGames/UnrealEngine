// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "KismetCompilerMisc.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_VariableSetRef.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraphPin;
class UObject;

UCLASS(MinimalAPI)
class UK2Node_VariableSetRef : public UK2Node
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEdGraphNode Interface
	virtual void AllocateDefaultPins() override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual void NotifyPinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual class FNodeHandlingFunctor* CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual int32 GetNodeRefreshPriority() const override { return EBaseNodeRefreshPriority::Low_UsesDependentWildcard; }
	virtual bool IsActionFilteredOut(class FBlueprintActionFilter const& Filter) override;
	//~ End UK2Node Interface
		
	/** 
	 * Changes the type of variable set by this node, based on the specified pin
	 *
	 * @param Pin				The pin to gather type information from.
	 */
	void CoerceTypeFromPin(const UEdGraphPin* Pin);

	/** Returns the pin that specifies which by-ref variable to set */
	BLUEPRINTGRAPH_API UEdGraphPin* GetTargetPin() const;

	/** Returns the pin that specifies the value to set */
	BLUEPRINTGRAPH_API UEdGraphPin* GetValuePin() const;

private:
	/** Constructing FText strings can be costly, so we cache the node's title */
	FNodeTextCache CachedNodeTitle;
};

