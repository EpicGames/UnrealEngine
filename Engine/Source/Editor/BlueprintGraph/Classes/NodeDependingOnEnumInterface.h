// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "NodeDependingOnEnumInterface.generated.h"

class UObject;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UNodeDependingOnEnumInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class INodeDependingOnEnumInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual class UEnum* GetEnum() const PURE_VIRTUAL(INodeDependingOnEnumInterface::GetEnum,return NULL;);

	virtual void ReloadEnum(class UEnum*) PURE_VIRTUAL(INodeDependingOnEnumInterface::ReloadEnum, return;);

	virtual bool ShouldBeReconstructedAfterEnumChanged() const {return false;}
};
