// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyEventNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "Styling/AppStyle.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyEventNodeSpawner)

#define LOCTEXT_NAMESPACE "AnimNotifyEventNodeSpawner"

UAnimNotifyEventNodeSpawner* UAnimNotifyEventNodeSpawner::Create(const FSoftObjectPath& InSkeletonObjectPath, FName InNotifyName)
{
	check(InNotifyName != NAME_None);

	FString Label = InNotifyName.ToString();
	FString CustomEventName = FString::Printf(TEXT("AnimNotify_%s"), *Label);

	UAnimNotifyEventNodeSpawner* NodeSpawner = NewObject<UAnimNotifyEventNodeSpawner>(GetTransientPackage());
	NodeSpawner->SkeletonObjectPath = InSkeletonObjectPath;
	NodeSpawner->NodeClass = UK2Node_Event::StaticClass();
	NodeSpawner->CustomEventName = *CustomEventName;

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	MenuSignature.MenuName = FText::Format(LOCTEXT("EventWithSignatureName", "Event {0}"), FText::FromName(NodeSpawner->CustomEventName));
	MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");

	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/)
	{
		UK2Node_Event* ActorRefNode = CastChecked<UK2Node_Event>(NewNode);
		ActorRefNode->EventReference.SetExternalMember(ActorRefNode->CustomFunctionName, UAnimInstance::StaticClass());
	};

	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateLambda(PostSpawnSetupLambda);
	NodeSpawner->DefaultMenuSignature.Category = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::AnimNotify);

	return NodeSpawner;
}

#undef LOCTEXT_NAMESPACE
