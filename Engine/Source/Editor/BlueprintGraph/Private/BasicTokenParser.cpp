// Copyright Epic Games, Inc. All Rights Reserved.

#include "BasicTokenParser.h"

#include "HAL/UnrealMemory.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/TextChar.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Char.h"
#include "Trace/Detail/Channel.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "BasicTokenParser"
DEFINE_LOG_CATEGORY_STATIC(LogTokenParser, Log, All);

/*******************************************************************************
 * FBasicToken
*******************************************************************************/

//------------------------------------------------------------------------------
FBasicToken::FBasicToken()
{
	InitToken();
}

//------------------------------------------------------------------------------
void FBasicToken::InitToken(EPropertyType InConstType)
{
	ConstantType = InConstType;
	TokenType = FBasicToken::TOKEN_None;
	TokenName	 = NAME_None;
	StartPos	 = 0;
	StartLine	 = 0;
	*Identifier	 = 0;
	FMemory::Memzero(String, sizeof(Identifier));
}

//------------------------------------------------------------------------------
void FBasicToken::Clone(const FBasicToken& Other)
{
	TokenType = Other.TokenType;
	TokenName = Other.TokenName;
	StartPos  = Other.StartPos;
	StartLine = Other.StartLine;

	FCString::Strncpy(Identifier, Other.Identifier, NAME_SIZE);
	FMemory::Memcpy(String, Other.String, sizeof(String));
}

//------------------------------------------------------------------------------
bool FBasicToken::Matches(const TCHAR* Str, ESearchCase::Type SearchCase) const
{
	return (TokenType==TOKEN_Identifier || TokenType==TOKEN_Symbol) && ((SearchCase == ESearchCase::CaseSensitive) ? !FCString::Strcmp(Identifier, Str) : !FCString::Stricmp(Identifier, Str));
}

//------------------------------------------------------------------------------
bool FBasicToken::Matches(const FName& Name) const
{
	return TokenType==TOKEN_Identifier && TokenName==Name;
}

//------------------------------------------------------------------------------
bool FBasicToken::StartsWith(const TCHAR* Str, bool bCaseSensitive) const
{
	const int32 StrLength = FCString::Strlen(Str);
	return (TokenType==TOKEN_Identifier || TokenType==TOKEN_Symbol) && (bCaseSensitive ? (!FCString::Strncmp(Identifier, Str, StrLength)) : (!FCString::Strnicmp(Identifier, Str, StrLength)));
}

//------------------------------------------------------------------------------
bool FBasicToken::IsBool() const
{
	return ConstantType == CPT_Bool || ConstantType == CPT_Bool8 || ConstantType == CPT_Bool16 || ConstantType == CPT_Bool32 || ConstantType == CPT_Bool64;
}

//------------------------------------------------------------------------------
void FBasicToken::SetConstInt(int32 InInt)
{
	ConstantType = CPT_Int;
	Int			 = InInt;
	TokenType	 = TOKEN_Const;
}

//------------------------------------------------------------------------------
void FBasicToken::SetConstBool(bool InBool)
{
	ConstantType = CPT_Bool;
	NativeBool   = InBool;
	TokenType    = TOKEN_Const;
}

//------------------------------------------------------------------------------
void FBasicToken::SetConstDouble(double InDouble)
{
	ConstantType = CPT_Double;
	Double       = InDouble;
	TokenType    = TOKEN_Const;
}

//------------------------------------------------------------------------------
void FBasicToken::SetConstName(FName InName)
{
	ConstantType = CPT_Name;
	*(FScriptName*)NameBytes = NameToScriptName(InName);
	TokenType = TOKEN_Const;
}

//------------------------------------------------------------------------------
void FBasicToken::SetConstString(TCHAR* InString, int32 MaxLength)
{
	check(MaxLength>0);
	ConstantType = CPT_String;
	if( InString != String )
	{
		FCString::Strncpy(String, InString, MaxLength);
	}
	TokenType = TOKEN_Const;
}

//------------------------------------------------------------------------------
void FBasicToken::SetGuid(TCHAR* InString, int32 MaxLength)
{
	SetConstString(InString, MaxLength);
	TokenType = TOKEN_Guid;
}
//------------------------------------------------------------------------------
FString FBasicToken::GetConstantValue() const
{
	if (TokenType == TOKEN_Const)
	{
		switch (ConstantType)
		{
		case CPT_Byte:
			return FString::Printf(TEXT("%u"), Byte);
		case CPT_Int:
			return FString::Printf(TEXT("%i"), Int);
		case CPT_Bool:
			// Don't use FCoreTexts::True/FCoreTexts::False here because they can be localized
			return FString::Printf(TEXT("%s"), NativeBool ? *(FName::GetEntry(NAME_TRUE)->GetPlainNameString()) : *(FName::GetEntry(NAME_FALSE)->GetPlainNameString()));
		case CPT_Double:
			return FString::Printf(TEXT("%f"), Double);
		case CPT_Name:
			return FString::Printf(TEXT("%s"), *ScriptNameToName(*(FScriptName*)NameBytes).ToString());
		case CPT_String:
			return String;

		// unsupported (parsing never produces a constant token of these types:
		// ..., CPT_Int8, CPT_Int16, CPT_Int64, ..., CPT_Bool8, etc)
		default:
			return TEXT("InvalidTypeForAToken");
		}
	}
	else
	{
		return TEXT("NotConstant");
	}
}

//------------------------------------------------------------------------------
bool FBasicToken::GetConstInt(int32& ValOut) const
{
	if(TokenType==TOKEN_Const && ConstantType==CPT_Int)
	{
		ValOut = Int;
		return true;
	}
	else if(TokenType==TOKEN_Const && ConstantType==CPT_Byte)
	{
		ValOut = Byte;
		return true;
	}
	else if(TokenType==TOKEN_Const && ConstantType==CPT_Double && Double==FMath::TruncToInt(Double))
	{
		ValOut = static_cast<int32>(Double);
		return true;
	}
	else
	{
		return false;
	}
}

/*******************************************************************************
 * FBasicTokenParser::FErrorState
*******************************************************************************/

//------------------------------------------------------------------------------
void FBasicTokenParser::FErrorState::Throw(bool bLogFatal) const
{
	if (State != NoError)
	{
		FString ErrorCodeStr;
		switch (State)
		{
		case ParseError:
			ErrorCodeStr = TEXT("ParseError");
			break;
		case RequireError:
			ErrorCodeStr = TEXT("RequireError");
			break;

		default:
			ErrorCodeStr.AppendInt(State.GetValue());
		}

		FString ErrorString = FString::Printf(TEXT("FBasicTokenParser Error (%s): %s"), *ErrorCodeStr, *Description.ToString());
		// don't always log fatal (these could be presented as user facing errors), 
		// but this is a good point to flip this bool on, to help catch the first 
		// error in a possible chain of snowballing errors
		if (bLogFatal)
		{
			UE_LOGF(LogTokenParser, Fatal, "%ls", *ErrorString);
		}
		else
		{
			UE_LOGF(LogTokenParser, Error, "FErrorState::Throw: %ls", *ErrorString);
		}
	}
}

/*******************************************************************************
 * FBasicTokenParser
*******************************************************************************/
namespace BasicTokenParserImpl
{
	//------------------------------------------------------------------------------
	static bool IsNumericChar(TCHAR c)
	{
		return (c >= '0' && c <= '9');
	}

	//------------------------------------------------------------------------------
	static bool IsWhitespace(TCHAR c)
	{
		return FTextChar::IsWhitespace(c);
	}

	//------------------------------------------------------------------------------
	static bool IsEOL(TCHAR c)
	{
		return c == TEXT('\n') || c == TEXT('\r') || c == 0;
	}

	//------------------------------------------------------------------------------
	static bool IsSymbol(TCHAR c)
	{
		return 
			// should have been handled in different cases, but as a catchall:
			(c == '{') ||
			(c == '}') ||
			(c == '"') ||
			// enumerated operators from logic in K2Node_MathExpression.cpp
			(c == '|') ||
			(c == '&') ||
			(c == '~') ||
			(c == '^') ||
			(c == '!') ||
			(c == '<') ||
			(c == '>') ||
			(c == '=') ||
			(c == '+') ||
			(c == '-') ||
			(c == '*') ||
			(c == '/') ||
			(c == '%') ||
			(c == ':') ||
			(c == '(') ||
			(c == ')') ||
			(c == ',') ||
			// in terms of the current MathExpression node, these could all 
			// technically be used as identifier names, but seeing as 1) this parser 
			// is meant to be generic, and 2) we could leverage these symbols as 
			// operators in the future, we want to make sure they're reserved
			(c == '`') ||
			(c == '[') ||
			(c == ']') ||
			(c == '\\') ||
			(c == ';') ||
			(c == '\'') ||
			(c == '@') ||
			(c == '#') ||
			(c == '$') ||
			(c == '.') ||
			(c == '?');
	}

	//------------------------------------------------------------------------------
	static bool IsIdentifierDelim(TCHAR c)
	{
		// attempt to be the opposite of: 
		//		IsLetter(c) || IsNumericChar(c) || (c == '_')
		// for optimization purposes we don't have an IsLetter(), since 
		// localization would make that a slow operation
		return IsSymbol(c) || IsWhitespace(c) || IsEOL(c);
	}
}

//------------------------------------------------------------------------------
void FBasicTokenParser::ResetParser(const TCHAR* SourceBuffer, int32 StartingLineNumber)
{
	Input     = SourceBuffer;
	InputLen  = FCString::Strlen(Input);
	InputPos  = 0;
	PrevPos   = 0;
	PrevLine  = 1;
	InputLine = StartingLineNumber;

	ClearCachedComment();
	ClearErrorState();
}

//------------------------------------------------------------------------------
void FBasicTokenParser::ClearCachedComment()
{
	// Can't call Reset as FString uses protected inheritance
	PrevComment.Empty( PrevComment.Len() );
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetToken(FBasicToken& Token, bool bNoConsts/* = false*/)
{
	using namespace BasicTokenParserImpl;

	// if the parser is in a bad state, then don't continue parsing (who 
	// knows what will happen!?)
	if (!IsValid())
	{
		return false;
	}

	Token.TokenName	= NAME_None;
	TCHAR c = GetLeadingChar();
	TCHAR p = PeekChar();
	if( c == 0 )
	{
		UngetChar();
		return 0;
	}
	Token.StartPos		= PrevPos;
	Token.StartLine		= PrevLine;
	if( c=='{' )
	{
		// Alphanumeric token.
		int32 Length=0;
		Token.Identifier[Length++] = c;
		do
		{
			if( Length >= NAME_SIZE )
			{
				Length = ((int32)NAME_SIZE) - 1;
				Token.Identifier[Length]=0; // need this for the error description

				FText ErrorDesc = FText::Format(LOCTEXT("IdTooLong", "Identifer ({0}...) exceeds maximum length of {1}"), FText::FromString(Token.Identifier), FText::AsNumber((int32)NAME_SIZE));
				SetError(FErrorState::ParseError, ErrorDesc);

				break;
			}

			c = GetChar();
			if (c == 0)
			{
				SetError(FErrorState::ParseError, LOCTEXT("MissingBracket", "Missing closing bracket: }"));
				break;
			}

			Token.Identifier[Length++] = c;
		} while( c !='}' );

		Token.Identifier[Length]=0;
		Token.SetGuid(Token.Identifier);
		return IsValid();
	}
	// if const values are allowed, determine whether the non-identifier token represents a const
	else if ( !bNoConsts && (IsNumericChar(c) || ((c=='+' || c=='-') && IsNumericChar(p))) )
	{
		// Integer or double-precision floating point constant.
		bool  bIsDouble = 0;
		int32 Length   = 0;
		bool  bIsHex   = 0;
		do
		{
			if( c==TEXT('.') )
			{
				bIsDouble = true;
			}
			if( c==TEXT('X') || c == TEXT('x') )
			{
				bIsHex = true;
			}

			Token.Identifier[Length++] = c;
			if( Length >= NAME_SIZE )
			{
				Length = ((int32)NAME_SIZE) - 1;
				Token.Identifier[Length]=0; // need this for the error description

				FText ErrorDesc = FText::Format(LOCTEXT("IdTooLong", "Identifer ({0}...) exceeds maximum length of {1}"), FText::FromString(Token.Identifier), FText::AsNumber((int32)NAME_SIZE));
				SetError(FErrorState::ParseError, ErrorDesc);

				break;
			}
			c = FChar::ToUpper(GetChar());
		} while (IsNumericChar(c) || (!bIsDouble && c == TEXT('.')) || (!bIsHex && c == TEXT('X')) || (bIsHex && c >= TEXT('A') && c <= TEXT('F')));

		Token.Identifier[Length]=0;
		if (!bIsDouble || c != 'F')
		{
			UngetChar();
		}

		if (bIsDouble)
		{
			Token.SetConstDouble( FCString::Atod(Token.Identifier) );
		}
		else if (bIsHex)
		{
			TCHAR* End = Token.Identifier + FCString::Strlen(Token.Identifier);
			Token.SetConstInt( FCString::Strtoi(Token.Identifier,&End,0) );
		}
		else
		{
			Token.SetConstInt( FCString::Atoi(Token.Identifier) );
		}
		return IsValid();
	}
	else if( c=='"' )
	{
		// String constant.
		TCHAR Temp[MAX_STRING_CONST_SIZE];
		int32 Length=0;
		c = GetChar(1);
		while( (c!='"') && !IsEOL(c) )
		{
			if( c=='\\' )
			{
				c = GetChar(1);
				if( IsEOL(c) )
				{
					break;
				}
				else if(c == 'n')
				{
					// Newline escape sequence.
					c = '\n';
				}
			}
			Temp[Length++] = c;
			if( Length >= MAX_STRING_CONST_SIZE )
			{
				Length = ((int32)MAX_STRING_CONST_SIZE) - 1;
				Temp[Length]=0; // need this for the error description

				FText ErrorDesc = FText::Format(LOCTEXT("StringConstTooLong", "String constant ({0}...) exceeds maximum of {1} characters"), FText::FromString(Temp), FText::AsNumber((int32)MAX_STRING_CONST_SIZE));
				SetError(FErrorState::ParseError, ErrorDesc);

				c = TEXT('\"');				
				break;
			}
			c = GetChar(1);
		}
		Temp[Length]=0;

		if( c != '"' )
		{
			FText ErrorDesc = FText::Format(LOCTEXT("NoClosingQuote", "Unterminated quoted string ({0})"), FText::FromString(Temp));
			SetError(FErrorState::ParseError, ErrorDesc);

			UngetChar();
		}

		Token.SetConstString(Temp);
		return IsValid();
	}
	// this condition is meant to be a catchall that encompasses: 
	//		!IsLetter(c) && (c != '_')
	// unfortunately we had to remove IsLetter(), as it was slow (to account for 
	// different languages)
	//
	// IsNumericChar() is here to catch when bNoConsts is true (we don't allow 
	// identifiers to start with a number)
	else if (IsSymbol(c) || IsNumericChar(c) || IsWhitespace(c) || IsEOL(c))
	{
		// Symbol.
		int32 Length=0;
		Token.Identifier[Length++] = c;

		// Handle special 2-character symbols.
#define PAIR(cc,dd) ((c==cc)&&(d==dd)) /* Comparison macro for convenience */
		TCHAR d = GetChar();
		if
		(	PAIR('<','<')
		||	PAIR('>','>')
		||	PAIR('!','=')
		||	PAIR('<','=')
		||	PAIR('>','=')
		||	PAIR('+','+')
		||	PAIR('-','-')
		||	PAIR('+','=')
		||	PAIR('-','=')
		||	PAIR('*','=')
		||	PAIR('/','=')
		||	PAIR('&','&')
		||	PAIR('|','|')
		||	PAIR('^','^')
		||	PAIR('=','=')
		||	PAIR('*','*')
		||	PAIR('~','=')
		||	PAIR(':',':')
		)
		{
			Token.Identifier[Length++] = d;
			if( c=='>' && d=='>' )
			{
				if( GetChar()=='>' )
					Token.Identifier[Length++] = '>';
				else
					UngetChar();
			}
		}
		else UngetChar();
#undef PAIR

		Token.Identifier[Length] = 0;
		Token.TokenType = FBasicToken::TOKEN_Symbol;

		// Lookup the token's global name.
		Token.TokenName = FName(Token.Identifier, FNAME_Find);

		return true;
	}
	else
	{
		// Alphanumeric token.
		int32 Length = 0;
		do
		{
			Token.Identifier[Length++] = c;
			if (Length >= NAME_SIZE)
			{
				Length = ((int32)NAME_SIZE) - 1;
				Token.Identifier[Length] = 0; // need this for the error description

				FText ErrorDesc = FText::Format(LOCTEXT("IdTooLong", "Identifer ({0}...) exceeds maximum length of {1}"), FText::FromString(Token.Identifier), FText::AsNumber((int32)NAME_SIZE));
				SetError(FErrorState::ParseError, ErrorDesc);

				break;
			}
			c = GetChar();
		} while (!IsIdentifierDelim(c));
		UngetChar();
		Token.Identifier[Length] = 0;

		// Assume this is an identifier unless we find otherwise.
		Token.TokenType = FBasicToken::TOKEN_Identifier;

		// Lookup the token's global name.
		Token.TokenName = FName(Token.Identifier, FNAME_Find);

		// If const values are allowed, determine whether the identifier represents a constant
		if (!bNoConsts)
		{
			// See if the identifier is part of a vector, rotation or other struct constant.
			// boolean true/false
			if (Token.Matches(TEXT("true")))
			{
				Token.SetConstBool(true);
				return true;
			}
			else if (Token.Matches(TEXT("false")))
			{
				Token.SetConstBool(false);
				return true;
			}
		}

		return IsValid();
	}
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetRawToken(FBasicToken& Token, TCHAR StopChar/* = TCHAR('\n')*/)
{
	// if the parser is in a bad state, then don't continue parsing (who 
	// knows what will happen!?)
	if (!IsValid())
	{
		return false;
	}

	// Get token after whitespace.
	TCHAR Temp[MAX_STRING_CONST_SIZE];
	int32 Length=0;
	TCHAR c = GetLeadingChar();
	while( !BasicTokenParserImpl::IsEOL(c) && c != StopChar )
	{
		if( (c=='/' && PeekChar()=='/') || (c=='/' && PeekChar()=='*') )
		{
			break;
		}
		Temp[Length++] = c;
		if( Length >= MAX_STRING_CONST_SIZE )
		{
			Temp[Length] = 0;

			FText ErrorDesc = FText::Format(LOCTEXT("IdTooLong", "Identifer ({0}...) exceeds maximum length of {1}"), FText::FromString(Temp), FText::AsNumber((int32)MAX_STRING_CONST_SIZE));
			SetError(FErrorState::ParseError, ErrorDesc);
		}
		c = GetChar(true);
	}
	UngetChar();

	// Get rid of trailing whitespace.
	while( Length>0 && (Temp[Length-1]==' ' || Temp[Length-1]==9 ) )
	{
		Length--;
	}
	Temp[Length]=0;

	Token.SetConstString(Temp);

	return Length>0;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetRawTokenRespectingQuotes(FBasicToken& Token, TCHAR StopChar/* = TCHAR('\n')*/)
{
	// if the parser is in a bad state, then don't continue parsing (who 
	// knows what will happen!?)
	if (!IsValid())
	{
		return false;
	}

	// Get token after whitespace.
	TCHAR Temp[MAX_STRING_CONST_SIZE];
	int32 Length=0;
	TCHAR c = GetLeadingChar();

	bool bInQuote = false;

	while( !BasicTokenParserImpl::IsEOL(c) && ((c != StopChar) || bInQuote) )
	{
		if( !bInQuote && ( (c=='/' && PeekChar()=='/') || (c=='/' && PeekChar()=='*') ) )
		{
			break;
		}

		if (c == '"')
		{
			bInQuote = !bInQuote;
		}

		Temp[Length++] = c;
		if( Length >= MAX_STRING_CONST_SIZE )
		{
			Length = ((int32)MAX_STRING_CONST_SIZE) - 1;
			Temp[Length]=0; // needs to happen for the error description below

			FText ErrorDesc = FText::Format(LOCTEXT("IdTooLong", "Identifer ({0}...) exceeds maximum length of {1}"), FText::FromString(Temp), FText::AsNumber((int32)MAX_STRING_CONST_SIZE));
			SetError(FErrorState::ParseError, ErrorDesc);

			c = GetChar(true);
			break;
		}
		c = GetChar(true);
	}
	UngetChar();

	// Get rid of trailing whitespace.
	while( Length>0 && (Temp[Length-1]==' ' || Temp[Length-1]==9 ) )
	{
		Length--;
	}
	Temp[Length]=0;

	if (bInQuote)
	{
		FText ErrorDesc = FText::Format(LOCTEXT("NoClosingQuote", "Unterminated quoted string ({0})"), FText::FromString(Temp));
		SetError(FErrorState::ParseError, ErrorDesc);
	}
	Token.SetConstString(Temp);

	return Length>0 && IsValid();
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetIdentifier(FBasicToken& Token, bool bNoConsts)
{
	if (!GetToken(Token, bNoConsts))
	{
		return false;
	}

	if (Token.TokenType == FBasicToken::TOKEN_Identifier)
	{
		return true;
	}

	UngetToken(Token);
	return false;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetSymbol(FBasicToken& Token)
{
	if (!GetToken(Token))
	{
		return false;
	}

	if (Token.TokenType == FBasicToken::TOKEN_Symbol)
	{
		return true;
	}

	UngetToken(Token);
	return false;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::GetConstInt(int32& Result, const TCHAR* ErrorContext)
{
	FBasicToken Token;
	if (GetToken(Token))
	{
		if (Token.GetConstInt(Result))
		{
			return true;
		}
		else
		{
			UngetToken(Token);
		}
	}

	if (ErrorContext != NULL)
	{
		FText ErrorDesc = FText::Format(LOCTEXT("ContextualNoInt", "{0}: Missing expected integer constant"), FText::FromString(ErrorContext));
		SetError(FErrorState::ParseError, ErrorDesc);
	}
	return false;
}

//------------------------------------------------------------------------------
void FBasicTokenParser::UngetToken(FBasicToken& Token)
{
	InputPos = Token.StartPos;
	InputLine = Token.StartLine;
}

//------------------------------------------------------------------------------
TCHAR FBasicTokenParser::PeekChar()
{
	return (InputPos < InputLen) ? Input[InputPos] : 0;
}

//------------------------------------------------------------------------------
TCHAR FBasicTokenParser::GetChar(bool bLiteral/* = false*/)
{
	// if the parser is in a bad state, then don't continue parsing (who 
	// knows what will happen!?)... return a char signaling the end-of-stream
	if (!IsValid())
	{
		return 0;
	}

	int32 CommentCount = 0;

	PrevPos = InputPos;
	PrevLine = InputLine;

Loop:
	const TCHAR c = Input[InputPos++];
	if ( CommentCount > 0 )
	{
		// Record the character as a comment.
		PrevComment += c;
	}

	if (c == TEXT('\n'))
	{
		InputLine++;
	}
	else if (!bLiteral)
	{
		const TCHAR NextChar = PeekChar();
		if ( c==TEXT('/') && NextChar==TEXT('*') )
		{
			if ( CommentCount == 0 )
			{
				ClearCachedComment();
				// Record the slash and star.
				PrevComment += c;
				PrevComment += NextChar;
			}
			CommentCount++;
			InputPos++;
			goto Loop;
		}
		else if( c==TEXT('*') && NextChar==TEXT('/') )
		{
			if (--CommentCount < 0)
			{
				ClearCachedComment();
				SetError(FErrorState::ParseError, LOCTEXT("UnexpectedCommentClose", "Unexpected '*/' outside of comment"));
			}
			// Star already recorded; record the slash.
			PrevComment += Input[InputPos];

			InputPos++;
			goto Loop;
		}
	}

	if (CommentCount > 0)
	{
		if (c == 0)
		{
			ClearCachedComment();
			SetError(FErrorState::ParseError, LOCTEXT("NoCommentClose", "No end to a comment by the end of the expression"));
		}
		else
		{
			goto Loop;
		}
	}
	return c;
}

//------------------------------------------------------------------------------
TCHAR FBasicTokenParser::GetLeadingChar()
{
	// if the parser is in a bad state, then don't continue parsing (who 
	// knows what will happen!?)... return a char signaling the end-of-stream
	if (!IsValid())
	{
		return 0;
	}

	TCHAR TrailingCommentNewline = 0;
	for (;;)
	{
		bool MultipleNewlines = false;

		TCHAR c;

		// Skip blanks.
		do
		{
			c = GetChar();

			// Check if we've encountered another newline since the last one
			if (c == TrailingCommentNewline)
			{
				MultipleNewlines = true;
			}
		} while (FTextChar::IsWhitespace(c));

		if (c != TEXT('/') || PeekChar() != TEXT('/'))
		{
			return c;
		}

		// Clear the comment if we've encountered newlines since the last comment
		if (MultipleNewlines)
		{
			ClearCachedComment();
		}

		// Record the first slash.  The first iteration of the loop will get the second slash.
		PrevComment += c;

		do
		{
			c = GetChar(true);
			if (c == 0)
				return c;
			PrevComment += c;
		} while (!BasicTokenParserImpl::IsEOL(c));

		TrailingCommentNewline = c;

		for (;;)
		{
			c = GetChar();
			if (c == 0)
				return c;
			if (c == TrailingCommentNewline || !BasicTokenParserImpl::IsEOL(c))
			{
				UngetChar();
				break;
			}

			PrevComment += c;
		}
	}
}

//------------------------------------------------------------------------------
void FBasicTokenParser::UngetChar()
{
	InputPos = PrevPos;
	InputLine = PrevLine;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::MatchIdentifier(FName Match)
{
	FBasicToken Token;
	if (!GetToken(Token))
	{
		return false;
	}

	if ((Token.TokenType == FBasicToken::TOKEN_Identifier) && (Token.TokenName == Match))
	{
		return true;
	}

	UngetToken(Token);
	return false;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::MatchIdentifier(const TCHAR* Match)
{
	FBasicToken Token;
	if (GetToken(Token))
	{
		if (Token.TokenType == FBasicToken::TOKEN_Identifier && FCString::Stricmp(Token.Identifier, Match) == 0)
		{
			return true;
		}
		else
		{
			UngetToken(Token);
		}
	}

	return false;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::PeekIdentifier(FName Match)
{
	FBasicToken Token;
	if (!GetToken(Token, true))
	{
		return false;
	}
	UngetToken(Token);
	return Token.TokenType == FBasicToken::TOKEN_Identifier && Token.TokenName == Match;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::PeekIdentifier(const TCHAR* Match)
{
	FBasicToken Token;
	if (!GetToken(Token, true))
	{
		return false;
	}
	UngetToken(Token);
	return Token.TokenType == FBasicToken::TOKEN_Identifier && FCString::Stricmp(Token.Identifier, Match) == 0;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::MatchSymbol(const TCHAR* Match)
{
	FBasicToken Token;

	if (GetToken(Token, /*bNoConsts=*/ true))
	{
		if (Token.TokenType == FBasicToken::TOKEN_Symbol && !FCString::Stricmp(Token.Identifier, Match))
		{
			return true;
		}
		else
		{
			UngetToken(Token);
		}
	}

	return false;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::PeekSymbol(const TCHAR* Match)
{
	FBasicToken Token;
	if (!GetToken(Token, true))
	{
		return false;
	}
	UngetToken(Token);

	return Token.TokenType == FBasicToken::TOKEN_Symbol && FCString::Stricmp(Token.Identifier, Match) == 0;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::RequireIdentifier(FName Match, const TCHAR* ErrorContext)
{
	if (!MatchIdentifier(Match))
	{
		FText ErrorDesc = FText::Format(LOCTEXT("MissingRequirement", "Missing '{0}' in {1}"), FText::FromName(Match), FText::FromString(ErrorContext));
		SetError(FErrorState::RequireError, ErrorDesc);
	}
	return IsValid();
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::RequireIdentifier(const TCHAR* Match, const TCHAR* ErrorContext)
{
	if (!MatchIdentifier(Match))
	{
		FText ErrorDesc = FText::Format(LOCTEXT("MissingRequirement", "Missing '{0}' in {1}"), FText::FromString(Match), FText::FromString(ErrorContext));
		SetError(FErrorState::RequireError, ErrorDesc);
	}
	return IsValid();
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::RequireSymbol(const TCHAR* Match, const TCHAR* ErrorContext)
{
	if (!MatchSymbol(Match))
	{
		FText ErrorDesc = FText::Format(LOCTEXT("MissingRequirement", "Missing '{0}' in {1}"), FText::FromString(Match), FText::FromString(ErrorContext));
		SetError(FErrorState::RequireError, ErrorDesc);
	}
	return IsValid();
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::RequireSemi()
{
	if( !MatchSymbol(TEXT(";")) )
	{
		FText ErrorDesc = LOCTEXT("MissingSemiColon", "Missing ';'");

		FBasicToken Token;
		if( GetToken(Token) )
		{
			ErrorDesc = FText::Format(LOCTEXT("MissingSemiBefore", "Missing ';' before '{0}'"), FText::FromString(Token.Identifier));
		}		
		SetError(FErrorState::RequireError, ErrorDesc);
	}
	return IsValid();
}

//------------------------------------------------------------------------------
void FBasicTokenParser::SetError(TEnumAsByte<FErrorState::EErrorType> ErrorCode, FText Description, bool bLogFatal)
{
	CurrentError.State = ErrorCode;
	CurrentError.Description = Description;
	CurrentError.Throw(bLogFatal);
}

//------------------------------------------------------------------------------
const FBasicTokenParser::FErrorState& FBasicTokenParser::GetErrorState() const
{
	return CurrentError;
}

//------------------------------------------------------------------------------
bool FBasicTokenParser::IsValid() const
{
	return (CurrentError.State == FErrorState::NoError);
}

//------------------------------------------------------------------------------
void FBasicTokenParser::ClearErrorState()
{
	SetError(FErrorState::NoError, FText::GetEmpty());
}

#undef LOCTEXT_NAMESPACE
