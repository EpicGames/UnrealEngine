// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "UObject/NameTypes.h"
#include "UObject/Script.h"
#include "UObject/Stack.h"

/*******************************************************************************
 * FBasicToken
*******************************************************************************/

/** 
 * Information regarding a token that was parsed from some expression string 
 * (type, value, etc.).
 */
class FBasicToken //: public FPropertyBase
{
public:
	FBasicToken(); // invokes InitToken()

	/**
	 * Resets this token, clearing any details that were previously set (allows
	 * you to init the token to some predefined value type).
	 * 
	 * @param  InConstType	Optional param that lets you init this to some known value type. 
	 */
	void InitToken(EPropertyType InConstType = CPT_None);

	/**
	* Copies the properties from another token into this one.
	*
	* @param  Other		The token to copy properties from.
	*/
	void Clone(const FBasicToken& Other);

	//--------------------------------------
	// Queries
	//--------------------------------------
	
	/** Determines if this token matches the specified name.*/
	bool Matches(const FName& Name) const;

	/** Determines if this token matches the specified string. */
	bool Matches(const TCHAR* Str, ESearchCase::Type SearchCase = ESearchCase::IgnoreCase) const;

	/** Determines if this token has the specified string as a prefix. */
	bool StartsWith(const TCHAR* Str, bool bCaseSensitive = false) const;

	/** Determines if this token has a boolean ConstantType.*/
	bool IsBool() const;

	//--------------------------------------
	// Constant value setters
	//--------------------------------------
	
	void SetConstInt(int32 InInt);
	void SetConstBool(bool InBool);
	void SetConstDouble(double InDouble);
	void SetConstName(FName InName);
	void SetConstString(TCHAR* InString, int32 MaxLength=MAX_STRING_CONST_SIZE);
	void SetGuid(TCHAR* InString, int32 MaxLength=MAX_STRING_CONST_SIZE);

	//--------------------------------------
	// Constant value getters
	//--------------------------------------

	/**
	 * Retrieves an int value from this token (if it is a constant 
	 * int/byte/double/etc. type).
	 * 
	 * @param  ValOut	Will hold the integer value if this is successfully.
	 * @return True if this represents a constant that can be converted to an integer, otherwise false (ValOut is invalid).
	 */
	bool GetConstInt(int32& ValOut) const;

	/**
	 * If this represents a constant value, then this returns a string 
	 * representing the value of said constant (formated according to the type).
	 * 
	 * @return A default error string if this is not a constant (or an 
	 *         unsupported type), else a string containing the constant value.
	 */
	FString GetConstantValue() const;

public:
	enum ETokenType
	{
		TOKEN_None				= 0x00, // No token.
		TOKEN_Identifier		= 0x01,	// Alphanumeric identifier.
		TOKEN_Symbol			= 0x02,	// Symbol.
		TOKEN_Const				= 0x03,	// A constant.
		TOKEN_Guid				= 0x04, // A variable guid
		TOKEN_Max				= 0x0D
	};
	/** Type of this token. */
	ETokenType TokenType;

	/** Name of this token. */
	FName TokenName;

	/** Starting position in expression stream where this token came from. */
	int32 StartPos;

	/** Starting line in the expression. */
	int32 StartLine;

	/** Always valid. */
	TCHAR Identifier[NAME_SIZE];

	/** Only valid when TokenType is TOKEN_Const */
	EPropertyType ConstantType;

	/* TOKEN_Const values */
	union
	{
		uint8  Byte;							// If CPT_Byte.
		int32  Int;								// If CPT_Int.
		bool   NativeBool;						// If CPT_Bool
		double Double;							// If CPT_Double.
		uint8  NameBytes[sizeof(FScriptName)];	// If CPT_Name.
		TCHAR  String[MAX_STRING_CONST_SIZE];	// If CPT_String
	};
};

/*******************************************************************************
 * FBasicTokenParser
*******************************************************************************/

/**
 * Provides a base set of expression parsing functionality for sub-classes to 
 * utilize when tokenizing a TCHAR stream (discerns operators from literals/ 
 * variables, strips out comments, etc.).
 */
class FBasicTokenParser
{
protected:
	/** 
	 * FBasicTokenParser is conceptually abstract, and should only be instantiated 
	 * through a subclass.
	 */
	FBasicTokenParser() {}

protected:
	/** 
	 * Sets up this to start parsing a new expression string and resets any 
	 * state lingering from the previous one.
	 *
	 * @param  SourceBuffer			The expression string you wish to parse next.
	 * @param  StartingLineNumber	The base line number to start counting from as this parses.
	 */
	void ResetParser(const TCHAR* SourceBuffer, int32 StartingLineNumber = 1);

	/** 
	 * Clears out the most recent parsed comment. 
	 */
	void ClearCachedComment();

	//--------------------------------------
	// Tokenizing functions
	//--------------------------------------

	/**
	 * Gets the next token from the input stream, advancing the variables which 
	 * keep track of the current input position and line. 
	 * 
	 * @param  Token [out]	Receives the value of the parsed text; if Token is 
	 *						pre-initialized, special logic is performed to attempt
	 *						to evaluated Token in the context of that type. Useful
	 *						for distinguishing between ambiguous symbols like enum tags.
	 * @param  bNoConsts	Specify true to indicate that tokens representing 
	 *						literal const values are not allowed.
	 * @return True if a token was successfully processed, false otherwise.
	 */
	bool GetToken(FBasicToken& Token, bool bNoConsts = false);

	/**
	 * Put all text from the current position up to either EOL or the StopToken
	 * into Token. Advances the compiler's current position.
	 *
	 * @param  Token [out]	Will contain the text that was parsed.
	 * @param  StopChar		Stop processing when this character is reached.
	 *
	 * @return True if a token was successfully processed, false otherwise.
	 */
	bool GetRawToken(FBasicToken& Token, TCHAR StopChar = TCHAR('\n'));

	/** 
	 * Doesn't quit if StopChar is found inside a double-quoted string, but does 
	 * NOT support quote escapes.
	 *
	 * @return True if a token was successfully processed, false otherwise.
	 */
	bool GetRawTokenRespectingQuotes(FBasicToken& Token, TCHAR StopChar = TCHAR('\n'));

	/** 
	 * Parse out an identifier from the expression stream.
	 *
	 * @return True if a token was successfully processed, false otherwise.
	 */
	bool GetIdentifier(FBasicToken& Token, bool bNoConsts = false);

	/**
	 * Parse out a symbol from the expression stream.
	 *
	 * @return True if a token was successfully processed, false otherwise.
	 */
	bool GetSymbol(FBasicToken& Token);

	/**
	 * Parse out an int constant from the expression stream.
	 *
	 * @return True if a int was successfully processed, false otherwise.
	 */
	bool GetConstInt(int32& Result, const TCHAR* ErrorContext = NULL);

	/** 
	 * Rolls back the line number and moves the parsing pointer back to where 
	 * the specified token started.
	 */
	void UngetToken(FBasicToken& Token);

	//--------------------------------------
	// Low-level parsing functions
	//--------------------------------------

	/** 
	 * Look at a single character from the input stream and return it, or 0=end.
	 * Has no effect on the input stream.
	 */
	TCHAR PeekChar();

	/** 
	 * Get a single character from the input stream and return it, or 0=end. 
	 */
	TCHAR GetChar(bool bLiteral = false);

	/** 
	 * Skip past all spaces, tabs, and comments in the input stream. 
	 */
	TCHAR GetLeadingChar();

	/** 
	 * Unget the previous character retrieved with GetChar(). 
	 */
	void  UngetChar();

	//--------------------------------------
	// Match queries
	//--------------------------------------

	/**
	 * Determines if the next token in the stream is an identifier that matches 
	 * the specified FName  (and consumes it if it does). This is used primarily 
	 * for checking for required symbols during parsing.
	 * 
	 * @param  Match	The name you wish to check for.
	 * @return True if a match was consumed, false otherwise.
	 */
	bool MatchIdentifier(FName Match);

	/**
	 * Determines if the next token in the stream is an identifier that matches 
	 * the specified string (and consumes it if it does).
	 */
	bool MatchIdentifier(const TCHAR* Match);

	/**
	 * Determines if the next token in the stream is an identifier that matches 
	 * the specified name (does NOT consume it).
	 */
	bool PeekIdentifier(FName Match);

	/**
	 * Determines if the next token in the stream is an identifier that matches 
	 * the specified string (does NOT consume it).
	 */
	bool PeekIdentifier(const TCHAR* Match);

	/**
	 * Determines if the next token in the stream is a symbol that matches 
	 * the specified string (and consumes it if it does).
	 */
	bool MatchSymbol(const TCHAR* Match);

	/**
	 * Determines if the next token in the stream is a symbol that matches 
	 * the specified string (does NOT consume it).
	 */
	bool PeekSymbol(const TCHAR* Match);

	//--------------------------------------
	// Requiring checks
	//--------------------------------------

	/**
	 * Ensures that the next token in the stream is an identifier that matches 
	 * the specified name (and errors out if it isn't).
	 */
	bool RequireIdentifier(FName Match, const TCHAR* ErrorContext);

	/**
	 * Ensures that the next token in the stream is an identifier that matches 
	 * the specified string (and errors out if it isn't).
	 */
	bool RequireIdentifier(const TCHAR* Match, const TCHAR* ErrorContext);

	/**
	 * Ensures that the next token in the stream is a symbol that matches 
	 * the specified string (and errors out if it isn't).
	 */
	bool RequireSymbol(const TCHAR* Match, const TCHAR* ErrorContext);

	/**
	 * Ensures that the next token in the stream is a semi-colon character (and 
	 * errors out if it isn't).
	 */
	bool RequireSemi();

	//--------------------------------------
	// Error state 
	//--------------------------------------

public:
	/**
	 * Checks to see if an error has been caught by the internal error state.
	 */
	bool IsValid() const;

	/** A struct for easily describing a specific error that occurred while parsing */
	struct FErrorState
	{
		enum EErrorType
		{
			NoError,
			ParseError,   // generic error that occurred while tokenizing the expression
			RequireError, // explicit error that was invoked through a call to one of the Require methods

			SubClassErrorStart, // the starting value for a subclass's 
		};
		
		FErrorState() 
			: State(NoError) 
		{}

		/** Will take the current error and logs it, optionally fatally */
		void Throw(bool bLogFatal) const;

		/** Should match up with EErrorType, but is extensible by subclasses 
		    (comes from a subclass if >= SubClassErrorStart) */
		TEnumAsByte<EErrorType> State; 
		/** A detailed localized string, describing what exactly went wrong 
		   (meant to be user facing) */
		FText Description;
	};

	/**
	 * Retrieves the parser's internal error state (so that users can interpret 
	 * what might have gone wrong while tokenizing).
	 */
	const FErrorState& GetErrorState() const;

protected:
	/**
	 * Takes the provided error and throws it (if the ErrorCode isn't 
	 * FErrorState::NoError). 
	 * 
	 * @param  ErrorCode	Should match up with FErrorState::EErrorType, but is
	 *						meant to be extensible by subclasses (so it could be 
	 *						out of that range).
	 * @param  Description	A detailed localized string, describing what exactly 
	 *						went wrong (meant to be user facing).
	 * @param  bLogFatal	Optional param where you can make a fatal log along 
	 *						with the thrown error (in case error handling is turned off).
	 */
	void SetError(TEnumAsByte<FErrorState::EErrorType> ErrorCode, FText Description, bool bLogFatal = false);

	/**
	 * Resets the internal error state, such that the parser can continue on.
	 */
	void ClearErrorState();
	
protected:
	/** Input text */
	const TCHAR* Input;

	/** Length of input text */
	int32 InputLen;

	/** Current position in text */
	int32 InputPos;

	/** Current line in text */
	int32 InputLine;

	/** Position previous to last GetChar() call */
	int32 PrevPos;

	/** Line previous to last GetChar() call. */
	int32 PrevLine;

	/** Previous comment parsed by GetChar() call. */
	FString PrevComment;

	/** Keeps track of the last error to occur */
	FErrorState CurrentError;
};
