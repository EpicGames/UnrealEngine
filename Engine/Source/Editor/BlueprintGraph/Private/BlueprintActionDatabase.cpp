// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionDatabase.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintAssetHandler.h"
#include "BlueprintBoundEventNodeSpawner.h"
#include "BlueprintComponentNodeSpawner.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintEventNodeSpawner.h"
#include "BlueprintFieldNodeSpawner.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintTypePromotion.h"
#include "BlueprintVariableNodeSpawner.h"
#include "ComponentTypeRegistry.h"
#include "Components/ActorComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
// used below in BlueprintActionDatabaseImpl::GetNodeSpectificActions()
#include "EdGraph/EdGraphNode_Documentation.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Engine/MemberReference.h"
#include "Engine/World.h"
#include "EngineLogs.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_AddDelegate.h"
// used below in BlueprintActionDatabaseImpl::AddClassPropertyActions()
#include "K2Node_AssignDelegate.h"
#include "K2Node_CallDelegate.h"
// used below in BlueprintActionDatabaseImpl::AddClassCastActions()
#include "K2Node_ClassDynamicCast.h"
#include "K2Node_ClearDelegate.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_MacroInstance.h"
// used below in FBlueprintNodeSpawnerFactory::MakeMacroNodeSpawner()
// used below in FBlueprintNodeSpawnerFactory::MakeComponentBoundEventSpawner()/MakeActorBoundEventSpawner()
// used below in FBlueprintNodeSpawnerFactory::MakeMessageNodeSpawner()
#include "K2Node_Message.h"
#include "K2Node_RemoveDelegate.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/SlateRect.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "PropertyPermissionList.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "BlueprintActionDatabase"

/*******************************************************************************
 * FBlueprintNodeSpawnerFactory
 ******************************************************************************/

namespace FBlueprintNodeSpawnerFactory
{
	/**
	 * Constructs a UK2Node_MacroInstance spawner. Evolved from 
	 * FK2ActionMenuBuilder::AttachMacroGraphAction(). Sets up the spawner to 
	 * set spawned nodes with the supplied macro.
	 *
	 * @param  MacroGraph	The macro you want spawned nodes referencing.
	 * @return A new node-spawner, setup to spawn a UK2Node_MacroInstance.
	 */
	static UBlueprintNodeSpawner* MakeMacroNodeSpawner(UEdGraph* MacroGraph);

	/**
	 * Constructs a UK2Node_Message spawner. Sets up the spawner to set 
	 * spawned nodes with the supplied function.
	 *
	 * @param  InterfaceFunction	The function you want spawned nodes referencing.
	 * @return A new node-spawner, setup to spawn a UK2Node_Message.
	 */
	static UBlueprintNodeSpawner* MakeMessageNodeSpawner(UFunction* InterfaceFunction);

	/**
	 * Constructs a UEdGraphNode_Comment spawner. Since UEdGraphNode_Comment is
	 * not a UK2Node then we can't have it create a spawner for itself (using 
	 * UK2Node's GetMenuActions() method).
	 *
	 * @param  DocNodeType	The node class type that you want the spawner to be responsible for.
	 *
	 * @return A new node-spawner, setup to spawn a UEdGraphNode_Comment.
	 */
	template <class DocNodeType>
	static UBlueprintNodeSpawner* MakeDocumentationNodeSpawner();

	/**
	 * 
	 * 
	 * @return 
	 */
	static UBlueprintNodeSpawner* MakeCommentNodeSpawner();

	/**
	 * Constructs a delegate binding node along with a connected event that is  
	 * triggered from the specified delegate.
	 * 
	 * @param  DelegateProperty	The delegate the spawner will bind to.
	 * @return A new node-spawner, setup to spawn a UK2Node_AssignDelegate.
	 */
	static UBlueprintNodeSpawner* MakeAssignDelegateNodeSpawner(FMulticastDelegateProperty* DelegateProperty);

	/**
	 * 
	 * 
	 * @param  DelegateProperty	
	 * @return 
	 */
	static UBlueprintNodeSpawner* MakeComponentBoundEventSpawner(FMulticastDelegateProperty* DelegateProperty);

	/**
	 * 
	 * 
	 * @param  DelegateProperty	
	 * @return 
	 */
	static UBlueprintNodeSpawner* MakeActorBoundEventSpawner(FMulticastDelegateProperty* DelegateProperty);
};

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeMacroNodeSpawner(UEdGraph* MacroGraph)
{
	check(MacroGraph != nullptr);
	check(MacroGraph->GetSchema()->GetGraphType(MacroGraph) == GT_Macro);

	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_MacroInstance::StaticClass());
	check(NodeSpawner != nullptr);

	auto CustomizeMacroNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, TWeakObjectPtr<UEdGraph> InMacroGraph)
	{
		UK2Node_MacroInstance* MacroNode = CastChecked<UK2Node_MacroInstance>(NewNode);
		if (InMacroGraph.IsValid())
		{
			MacroNode->SetMacroGraph(InMacroGraph.Get());
		}
	};

	TWeakObjectPtr<UEdGraph> GraphPtr = MacroGraph;
	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeMacroNodeLambda, GraphPtr);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeMessageNodeSpawner(UFunction* InterfaceFunction)
{
	check(InterfaceFunction != nullptr);
	check(FKismetEditorUtilities::IsClassABlueprintInterface(CastChecked<UClass>(InterfaceFunction->GetOuter())));

	UBlueprintFunctionNodeSpawner* NodeSpawner = UBlueprintFunctionNodeSpawner::Create(UK2Node_Message::StaticClass(), InterfaceFunction);
	check(NodeSpawner != nullptr);

	auto SetNodeFunctionLambda = [](UEdGraphNode* NewNode, FFieldVariant FuncField)
	{
		UK2Node_Message* MessageNode = CastChecked<UK2Node_Message>(NewNode);
		MessageNode->FunctionReference.SetFromField<UFunction>(FuncField.Get<UField>(), /*bIsConsideredSelfContext =*/false);
	};
	NodeSpawner->SetNodeFieldDelegate = UBlueprintFunctionNodeSpawner::FSetNodeFieldDelegate::CreateStatic(SetNodeFunctionLambda);

	NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(LOCTEXT("MessageNodeMenuName", "{0} (Message)"), NodeSpawner->DefaultMenuSignature.MenuName);
	return NodeSpawner;
}

//------------------------------------------------------------------------------
template <class DocNodeType>
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeDocumentationNodeSpawner()
{
	UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(DocNodeType::StaticClass());
	check(NodeSpawner != nullptr);

	auto CustomizeMessageNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
	{
		DocNodeType* DocNode = CastChecked<DocNodeType>(NewNode);

		UEdGraph* OuterGraph = NewNode->GetGraph();
		check(OuterGraph != nullptr);
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(OuterGraph);
		check(Blueprint != nullptr);

		const float OldNodePosX = static_cast<float>(NewNode->NodePosX);
		const float OldNodePosY = static_cast<float>(NewNode->NodePosY);
		const float OldHalfHeight = NewNode->NodeHeight / 2.f;
		const float OldHalfWidth  = NewNode->NodeWidth  / 2.f;
		
		static const float DocNodePadding = 50.0f;
		FSlateRect Bounds(OldNodePosX - OldHalfWidth, OldNodePosY - OldHalfHeight, OldNodePosX + OldHalfWidth, OldNodePosY + OldHalfHeight);
		FKismetEditorUtilities::GetBoundsForSelectedNodes(Blueprint, Bounds, DocNodePadding);
		DocNode->SetBounds(Bounds);
	};

	NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeMessageNodeLambda);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeCommentNodeSpawner()
{
	UBlueprintNodeSpawner* NodeSpawner = MakeDocumentationNodeSpawner<UEdGraphNode_Comment>();
	NodeSpawner->DefaultMenuSignature.MenuName = LOCTEXT("AddCommentActionMenuName", "Add Comment...");

	auto OverrideMenuNameLambda = [](FBlueprintActionContext const& Context, IBlueprintNodeBinder::FBindingSet const& /*Bindings*/, FBlueprintActionUiSpec* UiSpecOut)
	{
		for (UBlueprint* Blueprint : Context.Blueprints)
		{
			if (FKismetEditorUtilities::GetNumberOfSelectedNodes(Blueprint) > 0)
			{
				UiSpecOut->MenuName = LOCTEXT("AddCommentFromSelectionMenuName", "Add Comment to Selection");
				break;
			}
		}
	};
	NodeSpawner->DynamicUiSignatureGetter = UBlueprintNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(OverrideMenuNameLambda);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeAssignDelegateNodeSpawner(FMulticastDelegateProperty* DelegateProperty)
{
	// @TODO: it'd be awesome to have both nodes spawned by this available for 
	//        context pin matching (the delegate inputs and the event outputs)
	return UBlueprintDelegateNodeSpawner::Create(UK2Node_AssignDelegate::StaticClass(), DelegateProperty);
}

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeComponentBoundEventSpawner(FMulticastDelegateProperty* DelegateProperty)
{
	return UBlueprintBoundEventNodeSpawner::Create(UK2Node_ComponentBoundEvent::StaticClass(), DelegateProperty);
}

//------------------------------------------------------------------------------
static UBlueprintNodeSpawner* FBlueprintNodeSpawnerFactory::MakeActorBoundEventSpawner(FMulticastDelegateProperty* DelegateProperty)
{
	return UBlueprintBoundEventNodeSpawner::Create(UK2Node_ActorBoundEvent::StaticClass(), DelegateProperty);
}

/*******************************************************************************
 * Static FBlueprintActionDatabase Helpers
 ******************************************************************************/

namespace BlueprintActionDatabaseImpl
{
	typedef FBlueprintActionDatabase::FActionList FActionList;

	/**
	 * Mimics UEdGraphSchema_K2::CanUserKismetAccessVariable(); however, this 
	 * omits the filtering that CanUserKismetAccessVariable() does (saves that 
	 * for later with FBlueprintActionFilter).
	 * 
	 * @param  Property		The property you want to check.
	 * @return True if the property can be seen from a blueprint.
	 */
	static bool IsPropertyBlueprintVisible(FProperty const* const Property);

	/**
	 * Checks to see if the specified function is a blueprint owned function
	 * that was inherited from an implemented interface.
	 *
	 * @param  Function	 The function to check.
	 * @return True if the function is owned by a blueprint, and some (implemented) interface has a matching function name.
	 */
	static bool IsBlueprintInterfaceFunction(const UFunction* Function);

	/**
	 * Checks to see if the specified function is a blueprint owned function
	 * that was inherited from the blueprint's parent.
	 * 
	 * @param  Function	 The function to check.
	 * @return True if the function is owned by a blueprint, and some parent has a matching function name.
	 */
	static bool IsInheritedBlueprintFunction(const UFunction* Function);

	/**
	 * Retrieves all the actions pertaining to a class and its fields (functions,
	 * variables, delegates, etc.). Actions that are conceptually owned by the 
	 * class.
	 * 
	 * @param  Class			The class you want actions for.
	 * @param  ActionListOut	The array you want filled with the requested actions.
	 */
	static void GetClassMemberActions(UClass* const Class, FActionList& ActionListOut);

	/**
	 * Loops over all of the class's functions and creates a node-spawners for
	 * any that are viable for blueprint use. Evolved from 
	 * FK2ActionMenuBuilder::GetFuncNodesForClass(), plus a series of other
	 * FK2ActionMenuBuilder methods (GetAllInterfaceMessageActions,
	 * GetEventsForBlueprint, etc). 
	 *
	 * Ideally, any node that is constructed from a UFunction should go in here 
	 * (so we only ever loop through the class's functions once). We handle
	 * UK2Node_CallFunction alongside UK2Node_Event.
	 *
	 * @param  Class			The class whose functions you want node-spawners for.
	 * @param  ActionListOut	The list you want populated with the new spawners.
	 */
	static void AddClassFunctionActions(UClass const* const Class, FActionList& ActionListOut);

	/**
	 * Loops over all of the class's properties and creates node-spawners for 
	 * any that are viable for blueprint use. Evolved from certain parts of 
	 * FK2ActionMenuBuilder::GetAllActionsForClass().
	 *
	 * @param  Class			The class whose properties you want node-spawners for.
	 * @param  ActionListOut	The list you want populated with the new spawners.
	 */
	static void AddClassPropertyActions(UClass const* const Class, FActionList& ActionListOut);

	/**
	 * Loops over all of the class's data object's properties and creates node-spawners for
	 * any that are viable for blueprint use.
	 *
	 * @param  Class			The class whose properties you want node-spawners for.
	 * @param  ActionListOut	The list you want populated with the new spawners.
	 */
	static void AddClassDataObjectActions(UClass const* const Class, FActionList& ActionListOut);

	/**
	 * Evolved from FClassDynamicCastHelper::GetClassDynamicCastNodes(). If the
	 * specified class is a viable blueprint variable type, then two cast nodes
	 * are added for it (UK2Node_DynamicCast, and UK2Node_ClassDynamicCast).
	 *
	 * @param  Class			The class who you want cast nodes for (they cast to this class).
	 * @param  ActionListOut	The list you want populated with the new spawners.
	 */
	static void AddClassCastActions(UClass* const Class, FActionList& ActionListOut);

	/**
	 * If the associated class is a blueprint generated class, then this will
	 * loop over the blueprint's graphs and create any node-spawners associated
	 * with those graphs (like UK2Node_MacroInstance spawners for macro graphs).
	 *
	 * @param  Blueprint		The blueprint which you want graph associated node-spawners for.
	 * @param  ActionListOut	The list you want populated with new spawners.
	 */
	static void AddBlueprintGraphActions(UBlueprint const* const Blueprint, FActionList& ActionListOut);

	/**
	 * Emulates UEdGraphSchema::GetGraphContextActions(). If the supplied class  
	 * is a node type, then it will query the node's CDO for any actions it 
	 * wishes to add. This helps us keep the code in this file paired down, and 
	 * makes it easily extensible for new node types.
	 * 
	 * @param  NodeClass		The class which you want node-spawners for.
	 * @param  ActionListOut	The list you want populated with new spawners.
	 */
	static void GetNodeSpecificActions(TSubclassOf<UEdGraphNode const> const NodeClass, FBlueprintActionDatabaseRegistrar& Registrar);

	/**
	 * Callback to refresh the database when a blueprint has been altered 
	 * (clears the database entries for the blueprint's classes and recreates 
	 * them... a property/function could have been added/removed).
	 * 
	 * @param  InBlueprint	The blueprint that has been altered.
	 */
	static void OnBlueprintChanged(UBlueprint* InBlueprint);

	/**
	 * Callback to refresh the database when a new object has just been loaded 
	 * (to catch blueprint classes that weren't in the initial set, etc.) 
	 * 
	 * @param  NewObject	The object that was just loaded.
	 */
	static void OnAssetLoaded(UObject* NewObject);

	/**
	 * Callback to refresh the database when a new object has just been created 
	 * (to catch new blueprint classes that weren't in the initial set, etc.) 
	 * 
	 * @param  NewAssetInfo	Data regarding the newly added asset.
	 */
	static void OnAssetAdded(FAssetData const& NewAssetInfo);

	/**
	 * Callback to clear out object references so that a object can be deleted 
	 * without resistance from the actions cached here. Objects passed here
	 * won't necessarily be deleted (the user could still choose to cancel).
	 * 
	 * @param  ObjectsForDelete	A list of objects that MIGHT be deleted.
	 */
	static void OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete);

	/**
	 * Callback to refresh the database when an object has been deleted (to  
	 * clear any related entries that were stored in the database) 
	 * 
	 * @param  AssetInfo	Data regarding the freshly removed asset.
	 */
	static void OnAssetRemoved(FAssetData const& AssetInfo);

	/**
	 * Callback to refresh the database when an object has been deleted/unloaded
	 * (to clear any related entries that were stored in the database) 
	 * 
	 * @param  AssetObject	The object being removed/deleted/unloaded.
	 */
	static void OnAssetRemoved(UObject* AssetObject);

	/**
	 * Callback to refresh the database when a blueprint has been unloaded
	 * (to clear any related entries that were stored in the database) 
	 * 
	 * @param  BlueprintObj	The blueprint being unloaded.
	 */
	static void OnBlueprintUnloaded(UBlueprint* BlueprintObj);

	/**
	 * Callback to refresh the database when an object has been renamed (to clear 
	 * any related classes that were stored in the database under the old name) 
	 * 
	 * @param  AssetInfo	Data regarding the freshly removed asset.
	 */
	static void OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldName);

	/**
	 * Callback to refresh/add all level blueprints owned by this world to the database
	 * 
	 * @param  NewWorld		The world that was added.
	 */
	static void OnWorldAdded(UWorld* NewWorld);

	/**
	 * Callback to clear all levels from the database when a world is destroyed 
	 * 
	 * @param  DestroyedWorld	The world that was destroyed
	 */
	static void OnWorldDestroyed(UWorld* DestroyedWorld);

	/**
	 * Callback to re-evaluate all level blueprints owned by the world when the layout has changed
	 * 
	 * @param  World			The owner of the level and the world to be re-evaluated.
	 */
	static void OnRefreshLevelScripts(UWorld* World);

	/**
	 * Returns TRUE if the Object is valid for the database
	 *
	 * @param Object		Object to check for validity
	 * @return				TRUE if the Blueprint is valid for the database
	 */
	static bool IsObjectValidForDatabase(UObject const* Object);

	/**
	 * Refreshes database after a module is loaded or unloaded.
	 */
	static void OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason);

	/**
	 * Refreshes database after project was hot-reloaded.
	 */
	static void OnReloadComplete(EReloadCompleteReason Reason);

	/** 
	 * Assets that we cleared from the database (to remove references, and make 
	 * way for a delete), but in-case the class wasn't deleted we need them 
	 * tracked here so we can add them back in.
	 */
	TSet<TWeakObjectPtr<UObject>> PendingDelete;

	/**
	 * Modules that were explicitly loaded at runtime but have not yet been
	 * registered into the database. These will be processed on the next tick.
	 */
	TSet<FName> PendingModules;

	/**
	 * Modules that were explicitly loaded at runtime and registered into
	 * the database. We keep track of them here so that in the off chance
	 * we allow it to be unloaded, we can then trigger a database refresh.
	 */
	TSet<FName> LoadedModules;

	/** */
	bool bIsInitializing = false;

	/** True if a RefreshAll has been requested for the next Tick */
	bool bRefreshAllRequested = false;
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnModulesChanged(FName InModuleName, EModuleChangeReason InModuleChangeReason)
{
	switch (InModuleChangeReason)
	{
	case EModuleChangeReason::ModuleLoaded:
		// If not already tracked, add it to the list of modules that need to be registered on the next tick.
		if (!LoadedModules.Contains(InModuleName))
		{
			PendingModules.Add(InModuleName);
		}
		break;

	case EModuleChangeReason::ModuleUnloaded:
		// If pending, it was unloaded in the same frame, so we just need to remove it, and no refresh is needed.
		if (!PendingModules.Remove(InModuleName))
		{
			// If already registered as a loaded module, then we need to remove it and do a full refresh on the next tick.
			if (LoadedModules.Remove(InModuleName))
			{
				bRefreshAllRequested = true;
			}
		}

		// Guard against the possibility of unloading a pending module that is also already registered.
		checkf(!LoadedModules.Contains(InModuleName), TEXT("Module %s was unloaded, but wasn't unregistered from the Blueprint action database."), *InModuleName.ToString());
		break;
	
	default:
		break;
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnReloadComplete(EReloadCompleteReason Reason)
{
	BlueprintActionDatabaseImpl::bRefreshAllRequested = true;
}

//------------------------------------------------------------------------------
static bool BlueprintActionDatabaseImpl::IsPropertyBlueprintVisible(FProperty const* const Property)
{
	bool const bIsAccessible = Property->HasAllPropertyFlags(CPF_BlueprintVisible);

	bool const bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
	bool const bIsAssignableOrCallable = Property->HasAnyPropertyFlags(CPF_BlueprintAssignable | CPF_BlueprintCallable);

	bool bVisible = !Property->HasAnyPropertyFlags(CPF_Parm) && (bIsAccessible || (bIsDelegate && bIsAssignableOrCallable));
	if (bVisible)
	{
		bVisible = FPropertyEditorPermissionList::Get().DoesPropertyPassFilter(Property->GetOwnerStruct(), Property->GetFName());
	}

	return bVisible;
}

//------------------------------------------------------------------------------
static bool BlueprintActionDatabaseImpl::IsBlueprintInterfaceFunction(const UFunction* Function)
{
	bool bIsBpInterfaceFunc = false;
	if (UClass* FuncClass = Function->GetOwnerClass())
	{
		if (UBlueprint* BpOuter = Cast<UBlueprint>(FuncClass->ClassGeneratedBy))
		{
			FName FuncName = Function->GetFName();

			for (int32 InterfaceIndex = 0; (InterfaceIndex < BpOuter->ImplementedInterfaces.Num()) && !bIsBpInterfaceFunc; ++InterfaceIndex)
			{
				FBPInterfaceDescription& InterfaceDesc = BpOuter->ImplementedInterfaces[InterfaceIndex];
				if(!InterfaceDesc.Interface)
				{
					continue;
				}

				bIsBpInterfaceFunc = (InterfaceDesc.Interface->FindFunctionByName(FuncName) != nullptr);
			}
		}
	}
	return bIsBpInterfaceFunc;
}

//------------------------------------------------------------------------------
static bool BlueprintActionDatabaseImpl::IsInheritedBlueprintFunction(const UFunction* Function)
{
	bool bIsBpInheritedFunc = false;
	if (UClass* FuncClass = Function->GetOwnerClass())
	{
		if (UBlueprint* BpOwner = Cast<UBlueprint>(FuncClass->ClassGeneratedBy))
		{
			FName FuncName = Function->GetFName();
			if (UClass* ParentClass = BpOwner->ParentClass)
			{
				bIsBpInheritedFunc = (ParentClass->FindFunctionByName(FuncName, EIncludeSuperFlag::IncludeSuper) != nullptr);
			}
		}
	}
	return bIsBpInheritedFunc;
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::GetClassMemberActions(UClass* const Class, FActionList& ActionListOut)
{
	// class field actions (nodes that represent and perform actions on
	// specific fields of the class... functions, properties, etc.)
	{
		AddClassFunctionActions(Class, ActionListOut);
		AddClassPropertyActions(Class, ActionListOut);
		AddClassDataObjectActions(Class, ActionListOut);
		// class UEnum actions are added by individual nodes via GetNodeSpecificActions()
		// class UScriptStruct actions are added by individual nodes via GetNodeSpecificActions()
	}

	AddClassCastActions(Class, ActionListOut);
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::AddClassFunctionActions(UClass const* const Class, FActionList& ActionListOut)
{
	using namespace FBlueprintNodeSpawnerFactory; // for MakeMessageNodeSpawner()
	check(Class != nullptr);

	// loop over all the functions in the specified class; exclude-super because 
	// we can always get the super functions by looking up that class separately 
	for (TFieldIterator<UFunction> FunctionIt(Class, EFieldIteratorFlags::ExcludeSuper); FunctionIt; ++FunctionIt)
	{
		UFunction* Function = *FunctionIt;

		bool const bIsInheritedFunction = BlueprintActionDatabaseImpl::IsInheritedBlueprintFunction(Function);
		if (bIsInheritedFunction)
		{
			// inherited functions will be captured when the parent class is ran
			// through this function (no need to duplicate)
			continue;
		}

		// Apply general filtering for functions
		if(!FBlueprintActionDatabase::IsFunctionAllowed(Function, FBlueprintActionDatabase::EPermissionsContext::Node))
		{
			continue;
		}

		bool const bIsBpInterfaceFunc = BlueprintActionDatabaseImpl::IsBlueprintInterfaceFunction(Function);
		if (UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function) && !bIsBpInterfaceFunc)
		{
			if (UBlueprintEventNodeSpawner* NodeSpawner = UBlueprintEventNodeSpawner::Create(Function))
			{
				ActionListOut.Add(NodeSpawner);
			}
		}

		// If this is a promotable function, and it has already been registered
		// than do NOT add it to the asset action database. We should
		// probably have some better logic for this, like adding our own node spawner
		const bool bIsRegisteredPromotionFunc =
			TypePromoDebug::IsTypePromoEnabled() &&
			FTypePromotion::IsFunctionPromotionReady(Function) &&
			FTypePromotion::IsOperatorSpawnerRegistered(Function);

		if (UEdGraphSchema_K2::CanUserKismetCallFunction(Function))
		{
			// @TODO: if this is a Blueprint, and this function is from a 
			//        Blueprint "implemented interface", then we don't need to 
			//        include it (the function is accounted for in from the 
			//        interface class).
			UBlueprintFunctionNodeSpawner* FuncSpawner = UBlueprintFunctionNodeSpawner::Create(Function);
			
			// Only add this action to the list of the operator function is not already registered. Otherwise we will 
			// get a bunch of duplicate operator actions
			if (!bIsRegisteredPromotionFunc)
			{
				ActionListOut.Add(FuncSpawner);
			}

			// If an interface class has specified "CannotGenerateMessageNodes=true" in its metadata, then don't create any message nodes for
			// this interface class.
			if (FKismetEditorUtilities::IsClassABlueprintInterface(Class) && !Class->GetBoolMetaData(TEXT("CannotGenerateMessageNodes")))
			{
				// Use the default function name
				ActionListOut.Add(MakeMessageNodeSpawner(Function));
			}
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::AddClassPropertyActions(UClass const* const Class, FActionList& ActionListOut)
{
	using namespace FBlueprintNodeSpawnerFactory; // for MakeDelegateNodeSpawner()

	bool const bIsComponent  = Class->IsChildOf<UActorComponent>();
	bool const bIsActorClass = Class->IsChildOf<AActor>();
	
	// loop over all the properties in the specified class; exclude-super because 
	// we can always get the super properties by looking up that class separately 
	for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (!IsPropertyBlueprintVisible(Property))
		{
			continue;
		}

 		bool const bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
 		if (bIsDelegate)
 		{
			FMulticastDelegateProperty* DelegateProperty = CastFieldChecked<FMulticastDelegateProperty>(Property);
			if (DelegateProperty->HasAnyPropertyFlags(CPF_BlueprintAssignable))
			{
				UBlueprintNodeSpawner* AddSpawner = UBlueprintDelegateNodeSpawner::Create(UK2Node_AddDelegate::StaticClass(), DelegateProperty);
				ActionListOut.Add(AddSpawner);
				
				UBlueprintNodeSpawner* AssignSpawner = MakeAssignDelegateNodeSpawner(DelegateProperty);
				ActionListOut.Add(AssignSpawner);
			}
			
			if (DelegateProperty->HasAnyPropertyFlags(CPF_BlueprintCallable))
			{
				UBlueprintNodeSpawner* CallSpawner = UBlueprintDelegateNodeSpawner::Create(UK2Node_CallDelegate::StaticClass(), DelegateProperty);
				ActionListOut.Add(CallSpawner);
			}
			
			UBlueprintNodeSpawner* RemoveSpawner = UBlueprintDelegateNodeSpawner::Create(UK2Node_RemoveDelegate::StaticClass(), DelegateProperty);
			ActionListOut.Add(RemoveSpawner);
			UBlueprintNodeSpawner* ClearSpawner = UBlueprintDelegateNodeSpawner::Create(UK2Node_ClearDelegate::StaticClass(), DelegateProperty);
			ActionListOut.Add(ClearSpawner);

			if (bIsComponent)
			{
				ActionListOut.Add(MakeComponentBoundEventSpawner(DelegateProperty));
			}
			else if (bIsActorClass)
			{
				ActionListOut.Add(MakeActorBoundEventSpawner(DelegateProperty));
			}
 		}
		else
		{
			UBlueprintVariableNodeSpawner* GetterSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), Property);
			ActionListOut.Add(GetterSpawner);
			UBlueprintVariableNodeSpawner* SetterSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableSet::StaticClass(), Property);
			ActionListOut.Add(SetterSpawner);
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::AddClassDataObjectActions(UClass const* const Class, FActionList& ActionListOut)
{
	using namespace FBlueprintNodeSpawnerFactory; // for MakeDelegateNodeSpawner()

	// loop over all the properties in the specified class; exclude-super because 
	// we can always get the super properties by looking up that class separately 
	const UStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
	const UStruct* ParentSparseDataStruct = Class->GetSuperClass() ? Class->GetSuperClass()->GetSparseClassDataStruct() : nullptr;
	if (ParentSparseDataStruct != SparseDataStruct)
	{
		for (TFieldIterator<FProperty> PropertyIt(SparseDataStruct, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (!IsPropertyBlueprintVisible(Property))
			{
				continue;
			}

			UClass* NonConstClass = const_cast<UClass*>(Class);
			UBlueprintVariableNodeSpawner* GetterSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), Property, nullptr, NonConstClass);
			ActionListOut.Add(GetterSpawner);
//			UBlueprintVariableNodeSpawner* SetterSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableSet::StaticClass(), Property);
//			ActionListOut.Add(SetterSpawner);
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::AddClassCastActions(UClass* Class, FActionList& ActionListOut)
{
	Class = Class->GetAuthoritativeClass();
	check(Class);

	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
	bool bIsCastPermitted  = UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Class) && FBlueprintActionDatabase::IsClassAllowed(Class, FBlueprintActionDatabase::EPermissionsContext::Node);

	if (bIsCastPermitted)
	{
		auto CustomizeCastNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, UClass* TargetType)
		{
			UK2Node_DynamicCast* CastNode = CastChecked<UK2Node_DynamicCast>(NewNode);
			CastNode->TargetType = TargetType;
		};

		UBlueprintNodeSpawner* CastObjNodeSpawner = UBlueprintNodeSpawner::Create<UK2Node_DynamicCast>();
		CastObjNodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCastNodeLambda, Class);
		ActionListOut.Add(CastObjNodeSpawner);

		UBlueprintNodeSpawner* CastClassNodeSpawner = UBlueprintNodeSpawner::Create<UK2Node_ClassDynamicCast>();
		CastClassNodeSpawner->CustomizeNodeDelegate = CastObjNodeSpawner->CustomizeNodeDelegate;
		ActionListOut.Add(CastClassNodeSpawner);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::AddBlueprintGraphActions(UBlueprint const* const Blueprint, FActionList& ActionListOut)
{
	using namespace FBlueprintNodeSpawnerFactory; // for MakeMacroNodeSpawner()

	for (UEdGraph* MacroGraph : Blueprint->MacroGraphs)
	{
		ActionListOut.Add(MakeMacroNodeSpawner(MacroGraph));
	}

	auto CreateEntriesForGraphLambda = [Blueprint, &ActionListOut](UEdGraph* FunctionGraph)
	{
		if (!FunctionGraph)
		{
			return;
		}

		TArray<UK2Node_FunctionEntry*> GraphEntryNodes;
		FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(GraphEntryNodes);

		for (UK2Node_FunctionEntry* FunctionEntry : GraphEntryNodes)
		{
			UFunction* SkeletonFunction = FindUField<UFunction>(Blueprint->SkeletonGeneratedClass, FunctionGraph->GetFName());

			// Create entries for function parameters
			if (SkeletonFunction != nullptr)
			{
				for (TFieldIterator<FProperty> ParamIt(SkeletonFunction); ParamIt && (ParamIt->PropertyFlags & CPF_Parm); ++ParamIt)
				{
					FProperty* Param = *ParamIt;
					const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_ReturnParm) && (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm));
					if (bIsFunctionInput)
					{
						UBlueprintNodeSpawner* GetVarSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), Param, FunctionGraph);
						ActionListOut.Add(GetVarSpawner);
					}
				}
			}

			// Create entries for local variables
			for (FBPVariableDescription const& LocalVar : FunctionEntry->LocalVariables)
			{
				// Create a member reference so we can safely resolve the FProperty
				FMemberReference Reference;
				Reference.SetLocalMember(LocalVar.VarName, FunctionGraph->GetName(), LocalVar.VarGuid);

				UBlueprintNodeSpawner* GetVarSpawner = UBlueprintVariableNodeSpawner::CreateFromLocal(UK2Node_VariableGet::StaticClass(), FunctionGraph, LocalVar, Reference.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass));
				ActionListOut.Add(GetVarSpawner);
				UBlueprintNodeSpawner* SetVarSpawner = UBlueprintVariableNodeSpawner::CreateFromLocal(UK2Node_VariableSet::StaticClass(), FunctionGraph, LocalVar, Reference.ResolveMember<FProperty>(Blueprint->SkeletonGeneratedClass));
				ActionListOut.Add(SetVarSpawner);
			}
		}
	};

	auto CreateEntriesLambda = [&](const TArray<UEdGraph*>& Graphs)
	{
		for (UEdGraph* const Graph : Graphs)
		{
			CreateEntriesForGraphLambda(Graph);
		}
	};

	// local variables and parameters for functions
	CreateEntriesLambda(Blueprint->FunctionGraphs);

	// local variables and parameters for interfaces
	for (const FBPInterfaceDescription& Interface : Blueprint->ImplementedInterfaces)
	{
		CreateEntriesLambda(Interface.Graphs);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::GetNodeSpecificActions(TSubclassOf<UEdGraphNode const> const NodeClass, FBlueprintActionDatabaseRegistrar& Registrar)
{
	using namespace FBlueprintNodeSpawnerFactory; // for MakeCommentNodeSpawner()/MakeDocumentationNodeSpawner()

	if (NodeClass->IsChildOf<UK2Node>() && !NodeClass->HasAnyClassFlags(CLASS_Abstract))
	{
		UK2Node const* const NodeCDO = NodeClass->GetDefaultObject<UK2Node>();
		check(NodeCDO != nullptr);
		NodeCDO->GetMenuActions(Registrar);
	}


	// unfortunately, UEdGraphNode_Comment is not a UK2Node and therefore cannot
	// leverage UK2Node's GetMenuActions() function, so here we HACK it in
	//
	// @TODO: DO NOT follow this example! Do as I say, not as I do! If we need
	//        to support other nodes in a similar way, then we should come up
	//        with a better (more generalized) solution.
	if (NodeClass == UEdGraphNode_Comment::StaticClass())
	{
		Registrar.AddBlueprintAction(MakeCommentNodeSpawner());
	}
	else if (NodeClass == UEdGraphNode_Documentation::StaticClass())
	{
		// @TODO: BOOOOOOO! (see comment above)
		UBlueprintNodeSpawner* DocumentationSpawner = MakeDocumentationNodeSpawner<UEdGraphNode_Documentation>();
		DocumentationSpawner->DefaultMenuSignature.Category = LOCTEXT("DocumentationNodeCategory", "Documentation");
		Registrar.AddBlueprintAction(DocumentationSpawner);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnBlueprintChanged(UBlueprint* Blueprint)
{
	if (IsObjectValidForDatabase(Blueprint))
	{
		FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
		ActionDatabase.RefreshAssetActions(Blueprint);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetLoaded(UObject* NewObject)
{
	if (UBlueprint* NewBlueprint = Cast<UBlueprint>(NewObject))
	{
		OnBlueprintChanged(NewBlueprint);
	}
	else
	{
		FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
		ActionDatabase.RefreshAssetActions(NewObject);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetAdded(FAssetData const& NewAssetInfo)
{
	if (NewAssetInfo.IsAssetLoaded())
	{
		UObject* AssetObject = NewAssetInfo.GetAsset();
		if (UBlueprint* NewBlueprint = Cast<UBlueprint>(AssetObject))
		{
			OnBlueprintChanged(NewBlueprint);			
		}
		else 
		{
			FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
			ActionDatabase.RefreshAssetActions(AssetObject);
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetsPendingDelete(TArray<UObject*> const& ObjectsForDelete)
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	for (UObject* DeletingObject : ObjectsForDelete)
	{
// 		if (!IsObjectValidForDatabase(DeletingObject))
// 		{
// 			continue;
// 		}

		// have to temporarily remove references (so that this delete isn't 
		// blocked by dangling references)
		if (ActionDatabase.ClearAssetActions(DeletingObject))
		{
			ensure(IsObjectValidForDatabase(DeletingObject));
			// in case they choose not to delete the object, we need to add 
			// these back in to the database, so we track them here
			PendingDelete.Add(DeletingObject);
		}

		// If this asset contains a blueprint, ensure that it's actions are also marked for pending delete
		else if (const IBlueprintAssetHandler* Handler = FBlueprintAssetHandler::Get().FindHandler(DeletingObject->GetClass()))
		{
			UBlueprint* Blueprint = Handler->RetrieveBlueprint(DeletingObject);
			if (Blueprint && ActionDatabase.ClearAssetActions(Blueprint))
			{
				PendingDelete.Add(Blueprint);
			}
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetRemoved(FAssetData const& AssetInfo)
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();

	if (AssetInfo.IsAssetLoaded())
	{
		UObject* AssetObject = AssetInfo.GetAsset();
		OnAssetRemoved(AssetObject);
	}
	else
	{
		ActionDatabase.ClearUnloadedAssetActions(AssetInfo.GetSoftObjectPath());
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetRemoved(UObject* AssetObject)
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
	ActionDatabase.ClearAssetActions(AssetObject);

	for (auto It(PendingDelete.CreateIterator()); It; ++It)
	{
		if ((*It).Get() == AssetObject)
		{
			// the delete went through, so we don't need to track these for re-add
			It.RemoveCurrent();
			break;
		}
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnBlueprintUnloaded(UBlueprint* BlueprintObj)
{
	OnAssetRemoved(BlueprintObj);
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldName)
{
	FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();

	if (!AssetInfo.IsAssetLoaded())
	{
		ActionDatabase.MoveUnloadedAssetActions(FSoftObjectPath(InOldName), AssetInfo.GetSoftObjectPath());
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnWorldAdded(UWorld* NewWorld)
{
	if (IsObjectValidForDatabase(NewWorld))
	{
		FBlueprintActionDatabase::Get().RefreshAssetActions((UObject*)NewWorld);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnWorldDestroyed(UWorld* DestroyedWorld)
{
	if (IsObjectValidForDatabase(DestroyedWorld))
	{
		FBlueprintActionDatabase::Get().ClearAssetActions((UObject*)DestroyedWorld);
	}
}

//------------------------------------------------------------------------------
static void BlueprintActionDatabaseImpl::OnRefreshLevelScripts(UWorld* World)
{
	if (IsObjectValidForDatabase(World))
	{
		FBlueprintActionDatabase::Get().RefreshAssetActions((UObject*)World);
	}
}

//------------------------------------------------------------------------------
static bool BlueprintActionDatabaseImpl::IsObjectValidForDatabase(UObject const* Object)
{
	bool bReturn = false;
	if( Object == nullptr )
	{
		bReturn = false;
	}
	else if(Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor | PKG_ForDiffing))
	{
		// Do not keep track of any PIE/diff objects as they will not exist after those processes finish
		bReturn = false;
	}
	else if(Object->IsAsset())
	{
		bReturn = true;
	}
	else if(Object->IsA<UBlueprint>())
	{
		// If this is a blueprint contained within an asset, we can include it in the action database
		UObject* PotentialAsset = Object->GetOuter();
		while (PotentialAsset)
		{
			if (PotentialAsset->IsAsset())
			{
				bReturn = true;
				break;
			}
			PotentialAsset = PotentialAsset->GetOuter();
		}
	}
	else if(UWorld const* World = Cast<UWorld>(Object))
	{
		// We now use worlds as databse keys to manage the level scripts they own, but we only want Editor worlds.
		if(World->WorldType == EWorldType::Editor )
		{
			bReturn = true;
		}
	}
	return bReturn;
}

/*******************************************************************************
 * FBlueprintActionDatabase
 ******************************************************************************/

static FBlueprintActionDatabase* DatabaseInst = nullptr;
//------------------------------------------------------------------------------
FBlueprintActionDatabase& FBlueprintActionDatabase::Get()
{
	if (DatabaseInst == nullptr)
	{
		DatabaseInst = new FBlueprintActionDatabase();
		// Run init after constructing to avoid potentially creating multiple 
		// instances of the BAD:
		DatabaseInst->Init();
	}

	return *DatabaseInst;
}

//------------------------------------------------------------------------------
FBlueprintActionDatabase* FBlueprintActionDatabase::TryGet()
{
	return DatabaseInst;
}

//------------------------------------------------------------------------------
FBlueprintActionDatabase::FBlueprintActionDatabase()
{
}

void FBlueprintActionDatabase::Init()
{
	RefreshAll();
	OnAssetLoadedDelegateHandle = FCoreUObjectDelegates::OnAssetLoaded.AddStatic(&BlueprintActionDatabaseImpl::OnAssetLoaded);

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	OnAssetAddedDelegateHandle = AssetRegistry.OnAssetAdded().AddStatic(&BlueprintActionDatabaseImpl::OnAssetAdded);
	OnAssetRemovedDelegateHandle = AssetRegistry.OnAssetRemoved().AddStatic(&BlueprintActionDatabaseImpl::OnAssetRemoved);
	OnAssetRenamedDelegateHandle = AssetRegistry.OnAssetRenamed().AddStatic(&BlueprintActionDatabaseImpl::OnAssetRenamed);

	OnAssetsPreDeleteDelegateHandle = FEditorDelegates::OnAssetsPreDelete.AddStatic(&BlueprintActionDatabaseImpl::OnAssetsPendingDelete);
	OnBlueprintUnloadedDelegateHandle = FKismetEditorUtilities::OnBlueprintUnloaded.AddStatic(&BlueprintActionDatabaseImpl::OnBlueprintUnloaded);

	OnWorldAddedDelegateHandle = GEngine->OnWorldAdded().AddStatic(&BlueprintActionDatabaseImpl::OnWorldAdded);
	OnWorldDestroyedDelegateHandle = GEngine->OnWorldDestroyed().AddStatic(&BlueprintActionDatabaseImpl::OnWorldDestroyed);
	RefreshLevelScriptActionsDelegateHandle = FWorldDelegates::RefreshLevelScriptActions.AddStatic(&BlueprintActionDatabaseImpl::OnRefreshLevelScripts);

	OnModulesChangedDelegateHandle = FModuleManager::Get().OnModulesChanged().AddStatic(&BlueprintActionDatabaseImpl::OnModulesChanged);

	OnReloadCompleteDelegateHandle = FCoreUObjectDelegates::ReloadCompleteDelegate.AddStatic(&BlueprintActionDatabaseImpl::OnReloadComplete);
}

//------------------------------------------------------------------------------
FBlueprintActionDatabase::~FBlueprintActionDatabase()
{
	FCoreUObjectDelegates::OnAssetLoaded.Remove(OnAssetLoadedDelegateHandle);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
	{
		IAssetRegistry* AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().Remove(OnAssetAddedDelegateHandle);
			AssetRegistry->OnAssetRemoved().Remove(OnAssetRemovedDelegateHandle);
			AssetRegistry->OnAssetRenamed().Remove(OnAssetRenamedDelegateHandle);
		}
	}

	FEditorDelegates::OnAssetsPreDelete.Remove(OnAssetsPreDeleteDelegateHandle);
	FKismetEditorUtilities::OnBlueprintUnloaded.Remove(OnBlueprintUnloadedDelegateHandle);

	if (GEngine)
	{
		GEngine->OnWorldAdded().Remove(OnWorldAddedDelegateHandle);
		GEngine->OnWorldDestroyed().Remove(OnWorldDestroyedDelegateHandle);
	}

	FWorldDelegates::RefreshLevelScriptActions.Remove(RefreshLevelScriptActionsDelegateHandle);
	FModuleManager::Get().OnModulesChanged().Remove(OnModulesChangedDelegateHandle);


	FCoreUObjectDelegates::ReloadCompleteDelegate.Remove(OnReloadCompleteDelegateHandle);
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TPair<FObjectKey, TArray<TObjectPtr<UBlueprintNodeSpawner>>>& ActionListIt : ActionRegistry)
	{
		TArray<TObjectPtr<UBlueprintNodeSpawner>>& ActionList = ActionListIt.Value;
		for (TObjectPtr<UBlueprintNodeSpawner>& Action : ActionList)
		{
			// We have some reports of invalid action ptrs during GC - try to catch that case here without crashing the editor while reference gathering.
			if (!Action || (GIsGarbageCollecting && !Action->IsValidLowLevel()))
			{
				const UObject* Key = ActionListIt.Key.ResolveObjectPtr();
				ensureMsgf(false, TEXT("Invalid action (0x%016llx) registered for object: %s"), (int64)(PTRINT)Action.Get(), Key ? *Key->GetName() : TEXT("NULL"));
				continue;
			}

			Collector.AddReferencedObject(Action);
		}
	}

	for (TPair<FSoftObjectPath, TArray<TObjectPtr<UBlueprintNodeSpawner>>>& UnloadedActionListIt : UnloadedActionRegistry)
	{
		TArray<TObjectPtr<UBlueprintNodeSpawner>>& ActionList = UnloadedActionListIt.Value;
		for (TObjectPtr<UBlueprintNodeSpawner>& Action : ActionList)
		{
			// Similar to above; however, we don't have any reports of failure here during GC. Nonetheless, we'll try and catch an invalid ptr value just in case.
			if (!Action || (GIsGarbageCollecting && !Action->IsValidLowLevel()))
			{
				ensureMsgf(false, TEXT("Invalid action (0x%016llx) registered for unloaded object path: %s"), (int64)(PTRINT)Action.Get(), *UnloadedActionListIt.Key.ToString());
				continue;
			}

			Collector.AddReferencedObject(Action);
		}
	}
}

FString FBlueprintActionDatabase::GetReferencerName() const
{
	return TEXT("FBlueprintActionDatabase");
}

int32 GBlueprintDatabasePrimingMaxPerFrame = 16;
static FAutoConsoleVariableRef CVarBlueprintDatabasePrimingMaxPerFrame(
	TEXT("bp.DatabasePrimingMaxPerFrame"),
	GBlueprintDatabasePrimingMaxPerFrame,
	TEXT("How many entries should be primed in to the database per frame."),
	ECVF_Default
);

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::Tick(float DeltaTime)
{
	const double DurationThreshold = FMath::Min(0.003, DeltaTime * 0.01);

	if (BlueprintActionDatabaseImpl::bRefreshAllRequested)
	{
		RefreshAll();
	}
	else if (!BlueprintActionDatabaseImpl::PendingModules.IsEmpty())
	{
		PreRefresh(false);
	}
	
	// Check for any modules that may have been loaded since the last tick. Even if we call RefreshAll() above, we still want to run
	// through this list in order to keep track of loaded modules containing native script types that are registered into the database.
	if(!BlueprintActionDatabaseImpl::PendingModules.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::ProcessLoadedModules);

		// Treat this as an initialization event so that the registrar is configured to register new types rather than to refresh existing ones. For
		// example, if this module contains a new node type that implements a GetMenuActions() override, most implementations assume that a NULL action
		// key filter indicates the initialization path. This ensures we get the same behavior as node types that are registered via the RefreshAll() API.
		TGuardValue<bool> ScopedInitialization(BlueprintActionDatabaseImpl::bIsInitializing, true);

		// Register actions for any new modules that were explicitly loaded prior to this tick. Note that native type objects defined within the module
		// may have already been registered prior to receiving the load event; in that case, action(s) associated with those objects are already present.
		for (const FName& LoadedModule : BlueprintActionDatabaseImpl::PendingModules)
		{
			if (BlueprintActionDatabaseImpl::LoadedModules.Contains(LoadedModule))
			{
				continue;
			}

			// Look for a script package that's associated with this module load. If there isn't one, we can skip it.
			const FName ModuleScriptPackageName = FPackageName::GetModuleScriptPackageName(LoadedModule);
			if (const UPackage* ModuleScriptPackage = FindPackage(nullptr, *ModuleScriptPackageName.ToString()))
			{
				TArray<UObject*> ObjectsToProcess;
				GetObjectsWithPackage(ModuleScriptPackage, ObjectsToProcess, EGetObjectsFlags::None, RF_ClassDefaultObject);
				for (UObject* Object : ObjectsToProcess)
				{
					UClass* ObjectAsClass = Cast<UClass>(Object);
					const bool bIsNativeTypeObject = ObjectAsClass != nullptr || Object->IsA<UScriptStruct>() || Object->IsA<UEnum>();
					
					// Only need to include native types and those not already added through the registrar at initialization time.
					if (!bIsNativeTypeObject || ActionRegistry.Contains(Object))
					{
						continue;
					}

					if (ObjectAsClass)
					{
						RefreshClassActions(ObjectAsClass);
					}
					else
					{
						RefreshAssetActions(Object);
					}
				}

				// We only need to track modules that contain a script package in the off-chance that it is ever unloaded, in which case we'd need
				// to refresh the database to account for any types that go missing as a result. Otherwise, we can ignore this module when unloaded.
				BlueprintActionDatabaseImpl::LoadedModules.Add(LoadedModule);
			}
		}

		BlueprintActionDatabaseImpl::PendingModules.Empty();
	}

	// entries that were removed from the database, in preparation for a delete
	// (but the user ended up not deleting the object)
	for (TWeakObjectPtr<UObject> AssetObj : BlueprintActionDatabaseImpl::PendingDelete)
	{
		if (AssetObj.IsValid())
		{
			RefreshAssetActions(AssetObj.Get());
		}
	}
	BlueprintActionDatabaseImpl::PendingDelete.Empty();

	
	// priming every database entry at once would cause a hitch, so we spread it 
	// out over several frames
	int32 PrimedCount = 0;

	while ((ActionPrimingQueue.Num() > 0) && (PrimedCount < GBlueprintDatabasePrimingMaxPerFrame))
	{
		auto ActionIndex = ActionPrimingQueue.CreateIterator();	
			
		const FObjectKey& ActionsKey = ActionIndex.Key();
		if (ActionsKey.ResolveObjectPtr())
		{
			// make sure this class is still listed in the database
			if (FActionList* ClassActionList = ActionRegistry.Find(ActionsKey))
			{
				int32& ActionListIndex = ActionIndex.Value();
				for (; (ActionListIndex < ClassActionList->Num()) && (PrimedCount < GBlueprintDatabasePrimingMaxPerFrame); ++ActionListIndex)
				{
					UBlueprintNodeSpawner* Action = (*ClassActionList)[ActionListIndex];
					Action->Prime();
					++PrimedCount;
				}

				if (ActionListIndex >= ClassActionList->Num())
				{
					ActionPrimingQueue.Remove(ActionsKey);
				}
			}
			else
			{
				ActionPrimingQueue.Remove(ActionsKey);
			}
		}
		else
		{
			ActionPrimingQueue.Remove(ActionsKey);
		}
	}

	// Handle deferred removals.
	while (ActionRemoveQueue.Num() > 0)
	{
		ClearAssetActions(ActionRemoveQueue.Pop());
	}
}

//------------------------------------------------------------------------------
TStatId FBlueprintActionDatabase::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FBlueprintActionDatabase, STATGROUP_Tickables);
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::DeferredRemoveEntry(FObjectKey const& InKey)
{
	ActionRemoveQueue.AddUnique(InKey);
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RefreshAll()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::RefreshAll);

	TGuardValue<bool> ScopedInitialization(BlueprintActionDatabaseImpl::bIsInitializing, true);
	BlueprintActionDatabaseImpl::bRefreshAllRequested = false;

	// Refresh other systems before the database is recreated
	PreRefresh(true);

	// Remove callbacks from blueprints
	for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		UBlueprint* Blueprint = *BlueprintIt;

		// Level script BPs are registered using the associated world context. This will clear any registered LSBP actions.
		const bool bIsLevelScript = FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint);
		if (bIsLevelScript)
		{
			const ULevelScriptBlueprint* LSBP = CastChecked<ULevelScriptBlueprint>(Blueprint);
			if (UWorld* World = LSBP->GetWorld())
			{
				ClearAssetActions(World);
			}
		}
		
		// Do this for all BP assets to both clear registered actions and remove callbacks. In the case of LSBPs, this will just remove callbacks.
		ClearAssetActions(Blueprint);
	}

	ActionRegistry.Empty();
	UnloadedActionRegistry.Empty();
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* const Class = (*ClassIt);
		RefreshClassActions(Class);
	}

	FComponentTypeRegistry::Get().SubscribeToComponentList(ComponentTypes).RemoveAll(this);

	// this handles creating entries for components that were loaded before the database was alive:
	FComponentTypeRegistry::Get().SubscribeToComponentList(ComponentTypes).AddRaw(this, &FBlueprintActionDatabase::RefreshComponentActions);
	RefreshComponentActions();

	// Refresh existing worlds
	RefreshWorlds();
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RefreshWorlds()
{
	// Add all level scripts from current world
	const TIndirectArray<FWorldContext>& WorldContexts = GEngine->GetWorldContexts();

	for (const FWorldContext& Context : WorldContexts)
	{
		if( Context.WorldType == EWorldType::Editor )
		{
			if( UWorld* CurrentWorld = Context.World())
			{
				RefreshAssetActions((UObject*)CurrentWorld);
			}
		}
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RefreshClassActions(UClass* const Class)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::RefreshClassActions);

	using namespace BlueprintActionDatabaseImpl;
	check(Class != nullptr);

	bool const bFilterClass      = !IsClassAllowed(Class, EPermissionsContext::Asset);
	bool const bOutOfDateClass   = Class->HasAnyClassFlags(CLASS_NewerVersionExists);
	bool const bHiddenClass		 = Class->HasAnyClassFlags(CLASS_Hidden);
	bool const bIsBlueprintClass = (Cast<UBlueprintGeneratedClass>(Class) != nullptr);
	bool const bIsLevelScript	 = Class->ClassGeneratedBy && Cast<UBlueprint>(Class->ClassGeneratedBy)->BlueprintType == EBlueprintType::BPTYPE_LevelScript;
	// transient non native classes can't be persistently referenced - carve out for blueprint classes is for skeleton classes
	// which are transient but serve as a proxy for the persistent generated class:
	bool const bIsTransientClass = Class->HasAnyFlags(RF_Transient) && !Class->IsNative() && !bIsBlueprintClass;

	if (bOutOfDateClass || bIsLevelScript || bHiddenClass || bIsTransientClass)
	{
		ActionRegistry.Remove(Class);
		return;
	}
	else if (bIsBlueprintClass)
	{
		// Early out if the class is filtered
		if (bFilterClass)
		{
			return;
		}

		UBlueprint* Blueprint = Cast<UBlueprint>(Class->ClassGeneratedBy);
		if ((Blueprint != nullptr) && BlueprintActionDatabaseImpl::IsObjectValidForDatabase(Blueprint))
		{
			// to prevent us from hitting this twice on init (once for the skel 
			// class, again for the generated class)
			bool const bRefresh = !bIsInitializing || (Blueprint->SkeletonGeneratedClass == nullptr) ||
				(Blueprint->SkeletonGeneratedClass == Class);

			if (bRefresh)
			{
				RefreshAssetActions(Blueprint);
			}
		}
	}
	// here we account for "autonomous" standalone nodes, and any nodes that
	// exist in a separate module; each UK2Node has a chance to append its
	// own actions (presumably ones that would spawn that node)...
	else if (Class->IsChildOf<UEdGraphNode>())
	{
		// Early out if the class is filtered
		if (bFilterClass)
		{
			return;
		}

		{
			FActionList& ClassActionList = ActionRegistry.FindOrAdd(Class);
			if (!bIsInitializing)
			{
				ClassActionList.Empty();
			}
		}

		FBlueprintActionDatabaseRegistrar Registrar(ActionRegistry, UnloadedActionRegistry, ActionPrimingQueue, Class);
		if (!bIsInitializing)
		{
			// if this a call to RefreshClassActions() from somewhere other than 
			// RefreshAll(), then we should only add actions for this class (the
			// node could be adding actions, probably duplicate ones for assets)
			Registrar.ActionKeyFilter = Class;
		}
	
		// also, should catch any actions dealing with global UFields (like
		// global structs, enums, etc.; elements that wouldn't be caught
		// normally when sifting through fields on all known classes)		
		GetNodeSpecificActions(Class, Registrar);
		// don't worry, the registrar marks new actions for priming

		// Filter out actions by node class
		if(HasClassFiltering())
		{
			FActionList& ClassActionList = ActionRegistry.FindOrAdd(Class);
			ClassActionList.RemoveAllSwap([this](UBlueprintNodeSpawner* InAction)
			{
				return !IsClassAllowed(InAction->NodeClass.Get(), EPermissionsContext::Node);
			});
		}
	}
	else if (Class->IsChildOf<UBlueprint>())
	{
		// Early out if the class is filtered
		if (bFilterClass)
		{
			return;
		}

		FBlueprintActionDatabaseRegistrar Registrar(ActionRegistry, UnloadedActionRegistry, ActionPrimingQueue);
		Cast<UBlueprint>(Class->GetDefaultObject(false))->GetTypeActions(Registrar);
	}
	else
	{
		FActionList& ClassActionList = ActionRegistry.FindOrAdd(Class);
		if (!bIsInitializing && !bFilterClass)
		{
			ClassActionList.Empty();
			// if we're only refreshing this class (and not init'ing the whole 
			// database), then we have to reach out to individual nodes in case 
			// they'd add entries for this as well
			FBlueprintActionDatabaseRegistrar Registrar(ActionRegistry, UnloadedActionRegistry, ActionPrimingQueue);
			Registrar.ActionKeyFilter = Class; // only want actions for this class

			RegisterAllNodeActions(Registrar);
		}
		// Note: We still run this if we're filtering the class, as the class itself may expose properties/functions/etc that derived non-filtered classes need access to
		GetClassMemberActions(Class, ClassActionList);

		// queue the newly added actions for priming
		if (ClassActionList.Num() > 0)
		{
			ActionPrimingQueue.Add(Class, 0);

			if (HasClassFiltering())
			{
				// Filter out actions by node class
				ClassActionList.RemoveAllSwap([this](UBlueprintNodeSpawner* InAction)
				{
					return !IsClassAllowed(InAction->NodeClass.Get(), EPermissionsContext::Node);
				});
			}
		}
		else
		{
			ActionRegistry.Remove(Class);
		}
	}

	// blueprints are handled in RefreshAssetActions()
	if (!bIsInitializing && !bIsBlueprintClass)
	{
		EntryRefreshDelegate.Broadcast(Class);
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RefreshAssetActions(UObject* const AssetObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::RefreshAssetActions);

	using namespace BlueprintActionDatabaseImpl;

	// this method is very expensive and is only for blueprint editor functionality
	// it should remain that way as *greatly* increases cook times, etc!
	if (IsRunningCommandlet())
	{
		return;
	}

	FActionList& AssetActionList = ActionRegistry.FindOrAdd(AssetObject);
	for (UBlueprintNodeSpawner* Action : AssetActionList)
	{
		// because some asserts expect everything to be cleaned up in a 
		// single GC pass, we need to ensure that any previously cached node templates
		// are cleaned up here before we add any new node spawners.
		Action->ClearCachedTemplateNode();
	}
	AssetActionList.Empty();

	if (!IsObjectValidForDatabase(AssetObject))
	{
		return;
	}

	if (!IsClassAllowed(AssetObject->GetClass(), EPermissionsContext::Asset))
	{
		return;
	}

	UBlueprint* BlueprintAsset = Cast<UBlueprint>(AssetObject);
	if (BlueprintAsset != nullptr)
	{
		AddBlueprintGraphActions(BlueprintAsset, AssetActionList);
		if (UClass* SkeletonClass = BlueprintAsset->SkeletonGeneratedClass)
		{
			GetClassMemberActions(SkeletonClass, AssetActionList);
		}

		FBlueprintActionDatabaseRegistrar Registrar(ActionRegistry, UnloadedActionRegistry, ActionPrimingQueue);
		if (!bIsInitializing)
		{
			// if this a call to RefreshAssetActions() from somewhere other than 
			// RefreshAll(), then we should only add actions for this class (the
			// node could be adding actions, probably duplicate ones for assets)
			Registrar.ActionKeyFilter = BlueprintAsset->GeneratedClass;
		}
		BlueprintAsset->GetInstanceActions(Registrar);

		UBlueprint::FChangedEvent& OnBPChanged = BlueprintAsset->OnChanged();
		UBlueprint::FCompiledEvent& OnBPCompiled = BlueprintAsset->OnCompiled();
		// have to be careful not to register this callback twice for the 
		// blueprint
		if (!OnBPChanged.IsBoundToObject(this))
		{
			OnBPChanged.AddRaw(this, &FBlueprintActionDatabase::OnBlueprintChanged);
		}
		if (!OnBPCompiled.IsBoundToObject(this))
		{
			OnBPCompiled.AddRaw(this, &FBlueprintActionDatabase::OnBlueprintChanged);
		}
	}

	UWorld* WorldAsset = Cast<UWorld>(AssetObject);
	if (WorldAsset && WorldAsset->WorldType == EWorldType::Editor)
	{
		for( ULevel* Level : WorldAsset->GetLevels() )
		{
			UBlueprint* LevelScript = Level->GetLevelScriptBlueprint(true);
			if (LevelScript != nullptr)
			{
				AddBlueprintGraphActions(LevelScript, AssetActionList);
				if (UClass* SkeletonClass = LevelScript->SkeletonGeneratedClass)
				{
					GetClassMemberActions(SkeletonClass, AssetActionList);
				}
				// Register for change and compilation notifications
				if (!LevelScript->OnChanged().IsBoundToObject(this))
				{
					LevelScript->OnChanged().AddRaw(this, &FBlueprintActionDatabase::OnBlueprintChanged);
				}
				if (!LevelScript->OnCompiled().IsBoundToObject(this))
				{
					LevelScript->OnCompiled().AddRaw(this, &FBlueprintActionDatabase::OnBlueprintChanged);
				}
			}
		}
	}

	FBlueprintActionDatabaseRegistrar Registrar(ActionRegistry, UnloadedActionRegistry, ActionPrimingQueue);
	Registrar.ActionKeyFilter = AssetObject; // make sure actions only associated with this asset get added
	// nodes may have actions they wish to add actions for this asset
	RegisterAllNodeActions(Registrar);

	// Will clear up any unloaded asset actions associated with this object, if any
	ClearUnloadedAssetActions(FSoftObjectPath(AssetObject));

	if (!IsValid(AssetObject))
	{
		ClearAssetActions(AssetObject);
	}
	else if (AssetActionList.Num() > 0)
	{
		// queue these assets for priming
		ActionPrimingQueue.Add(AssetObject, 0);
	}
	// we don't want to clear entries for blueprints, mainly because we 
	// use the presence of an entry to know if we've set the blueprint's 
	// OnChanged(), but also because most blueprints will have actions at some 
	// later point. Same goes for in-editor world assets because they are used to manage level script blueprints.
	else if (!BlueprintAsset && (!WorldAsset || WorldAsset->WorldType != EWorldType::Editor))
	{
		ClearAssetActions(AssetObject);
	}

	if (!bIsInitializing)
	{
		EntryRefreshDelegate.Broadcast(AssetObject);
	}

	if (HasClassFiltering())
	{
		// Filter out actions by node class
		AssetActionList.RemoveAllSwap([this](UBlueprintNodeSpawner* InAction)
		{
			return !IsClassAllowed(InAction->NodeClass.Get(), EPermissionsContext::Node);
		});
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RefreshComponentActions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::RefreshComponentActions);

	check(ComponentTypes);
	FActionList& ClassActionList = ActionRegistry.FindOrAdd(UBlueprintComponentNodeSpawner::StaticClass());
	ClassActionList.Empty(ComponentTypes->Num());
	for (const FComponentTypeEntry& ComponentType : *ComponentTypes)
	{
		if (!IsClassAllowed(ComponentType.ComponentClass, EPermissionsContext::Node))
		{
			continue;
		}

		if (UBlueprintComponentNodeSpawner* NodeSpawner = UBlueprintComponentNodeSpawner::Create(ComponentType))
		{
			ClassActionList.Add(NodeSpawner);
		}
	}

	if (HasClassFiltering())
	{
		// Filter out actions by node class
		ClassActionList.RemoveAllSwap([this](UBlueprintNodeSpawner* InAction)
		{
			return !IsClassAllowed(InAction->NodeClass.Get(), EPermissionsContext::Node);
		});
	}
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabase::ClearAssetActions(UObject* const AssetObject)
{
	FObjectKey ObjectKey(AssetObject);
	return ClearAssetActions(ObjectKey);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabase::ClearAssetActions(const FObjectKey& AssetObjectKey)
{
	FActionList* ActionList = ActionRegistry.Find(AssetObjectKey);

	bool const bHasEntry = (ActionList != nullptr);
	if (bHasEntry)
	{
		for (UBlueprintNodeSpawner* Action : *ActionList)
		{
			if (Action != nullptr)
			{
			// because some asserts expect everything to be cleaned up in a 
			// single GC pass, we can't wait for the GC'd Action to release its
			// template node from the cache
			Action->ClearCachedTemplateNode();
		}
		}
		ActionRegistry.Remove(AssetObjectKey);
	}

	if (UObject* AssetObject = AssetObjectKey.ResolveObjectPtr())
	{
		if (UBlueprint* BlueprintAsset = Cast<UBlueprint>(AssetObject))
		{
			BlueprintAsset->OnChanged().RemoveAll(this);
			BlueprintAsset->OnCompiled().RemoveAll(this);
		}

		if (bHasEntry && (ActionList->Num() > 0) && !BlueprintActionDatabaseImpl::bIsInitializing)
		{
			EntryRemovedDelegate.Broadcast(AssetObject);
		}
	}
	
	return bHasEntry;
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::ClearUnloadedAssetActions(const FSoftObjectPath& ObjectPath)
{
	// Check if the asset can be found in the unloaded action registry, if it can, we need to remove it
	if(auto* UnloadedActionList = UnloadedActionRegistry.Find(ObjectPath))
	{
		for(UBlueprintNodeSpawner* NodeSpawner : *UnloadedActionList)
		{
			FActionList* ActionList = ActionRegistry.Find(NodeSpawner->NodeClass.Get());

			// Remove the NodeSpawner from the main registry, it will be replaced with the loaded version of the action
			ActionList->Remove(NodeSpawner);
		}

		// Remove the asset's path from the unloaded registry, it is no longer needed
		UnloadedActionRegistry.Remove(ObjectPath);
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::MoveUnloadedAssetActions(const FSoftObjectPath& SourceObjectPath, const FSoftObjectPath& TargetObjectPath)
{
	// Check if the asset can be found in the unloaded action registry, if it can, we need to remove it and re-add under the new name
	if(auto* UnloadedActionList = UnloadedActionRegistry.Find(SourceObjectPath))
	{
		check(!UnloadedActionRegistry.Find(TargetObjectPath));

		// Add the entire array to the database under the new path
		UnloadedActionRegistry.Add(TargetObjectPath, *UnloadedActionList);

		// Remove the old asset's path from the unloaded registry, it is no longer needed
		UnloadedActionRegistry.Remove(SourceObjectPath);
	}
}

//------------------------------------------------------------------------------
FBlueprintActionDatabase::FActionRegistry const& FBlueprintActionDatabase::GetAllActions()
{
	// if this is the first time that we're querying for actions, generate the
	// list before returning it
	if (ActionRegistry.Num() == 0)
	{
		RefreshAll();
	}
	return ActionRegistry;
}

//------------------------------------------------------------------------------
void FBlueprintActionDatabase::RegisterAllNodeActions(FBlueprintActionDatabaseRegistrar& Registrar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionDatabase::RegisterAllNodeActions);

	// nodes may have actions they wish to add for this asset
	TArray<UClass*> NodeClassList;
	GetDerivedClasses(UK2Node::StaticClass(), NodeClassList);

	for (UClass* NodeClass : NodeClassList)
	{
		TGuardValue< TSubclassOf<UEdGraphNode> > ScopedNodeClass(Registrar.GeneratingClass, NodeClass);
		BlueprintActionDatabaseImpl::GetNodeSpecificActions(NodeClass, Registrar);
	}
}

void FBlueprintActionDatabase::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	if( InBlueprint->BlueprintType == BPTYPE_LevelScript )
	{
		// Levelscript blueprints are managed through their owning worlds.
		RefreshWorlds();
	}
	else
	{
		BlueprintActionDatabaseImpl::OnBlueprintChanged(InBlueprint);
	}
}

void FBlueprintActionDatabase::PreRefresh(bool bRefreshAll)
{
	// Refresh other systems as necessary, doing it here avoids redundant work
	FTypePromotion::RefreshPromotionTables();
}

bool FBlueprintActionDatabase::IsClassAllowed(UClass const* InClass, EPermissionsContext InContext)
{
	if (InClass == nullptr)
	{
		return false;
	}

	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
	
	switch(InContext)
	{
	case EPermissionsContext::Property:
	case EPermissionsContext::Node:
	case EPermissionsContext::Asset:
		if(BlueprintEditorSettings->HasClassFiltering())
		{
			return BlueprintEditorSettings->IsClassAllowed(InClass);
		}
		break;
	case EPermissionsContext::Pin:
		if(BlueprintEditorSettings->HasClassOnPinFiltering())
		{
			return BlueprintEditorSettings->IsClassAllowedOnPin(InClass);
		}
		break;
	default:
		break;
	}

	return true;
}

bool FBlueprintActionDatabase::IsClassAllowed(const FTopLevelAssetPath& InClassPath, EPermissionsContext InContext)
{
	if(!InClassPath.IsValid())
	{
		return false;
	}

	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
	
	switch(InContext)
	{
	case EPermissionsContext::Property:
	case EPermissionsContext::Node:
	case EPermissionsContext::Asset:
		if (BlueprintEditorSettings->HasClassPathFiltering())
		{
			return BlueprintEditorSettings->IsClassPathAllowed(InClassPath);
		}
		break;
	case EPermissionsContext::Pin:
		if(BlueprintEditorSettings->HasClassPathOnPinFiltering())
		{
			return BlueprintEditorSettings->IsClassPathAllowedOnPin(InClassPath);
		}
		break;
	default:
		break;
	}

	return true;
}

bool FBlueprintActionDatabase::HasClassFiltering()
{
	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
	return	BlueprintEditorSettings->HasClassFiltering() || 
			BlueprintEditorSettings->HasClassOnPinFiltering() || 
			BlueprintEditorSettings->HasClassPathFiltering() ||
			BlueprintEditorSettings->HasClassPathOnPinFiltering();
}

bool FBlueprintActionDatabase::IsFieldAllowed(UField const* InField, EPermissionsContext InContext)
{
	if (UFunction const* Function = Cast<UFunction>(InField))
	{
		return IsFunctionAllowed(Function, InContext);
	}
	else if (UEnum const* Enum = Cast<UEnum>(InField))
	{
		return IsEnumAllowed(Enum, InContext);
	}
	else if (UScriptStruct const* ScriptStruct = Cast<UScriptStruct>(InField))
	{
		return IsStructAllowed(ScriptStruct, InContext);
	}
	else if (UClass const* Class = Cast<UClass>(InField))
	{
		return IsClassAllowed(Class, InContext);
	}

	return true;
}

bool FBlueprintActionDatabase::IsFunctionAllowed(UFunction const* InFunction, EPermissionsContext InContext)
{
	UBlueprintEditorSettings* BlueprintEditorSettings = GetMutableDefault<UBlueprintEditorSettings>();
	const FPathPermissionList& FunctionPermissions = BlueprintEditorSettings->GetFunctionPermissions();
	const FPathPermissionList& EnumPermissions = BlueprintEditorSettings->GetEnumPermissions();
	const FPathPermissionList& StructPermissions = BlueprintEditorSettings->GetStructPermissions();
	const FNamePermissionList& PinCategoryPermissions = BlueprintEditorSettings->GetPinCategoryPermissions();

	// Apply general filtering for functions
	if (FunctionPermissions.HasFiltering())
	{
		TStringBuilder<256> ResultBuilder;
		InFunction->GetPathName(nullptr, ResultBuilder);
		if (!FunctionPermissions.PassesFilter(ResultBuilder.ToView()))
		{
			return false;
		}
	}

	if (UClass* Class = InFunction->GetOuterUClass())
	{
		if (!IsClassAllowed(Class, EPermissionsContext::Asset))
		{
			UE_LOGF(LogBlueprint, Warning, "Function %ls was filtered because its class (%ls) was filtered", *InFunction->GetPathName(), *Class->GetName());
			return false;
		}
	}

	// If we have filtering for other fields we need to check function parameters that might reference them
	if (EnumPermissions.HasFiltering() || StructPermissions.HasFiltering() || PinCategoryPermissions.HasFiltering())
	{
		for (TFieldIterator<FProperty> PropertyIt(InFunction); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (StructProperty->Struct)
				{
					if (!IsStructAllowed(StructProperty->Struct, EPermissionsContext::Pin))
					{
						UE_LOGF(LogBlueprint, Warning, "Function %ls was filtered because one of its parameters %ls (struct %ls) was filtered", *InFunction->GetPathName(), *Property->GetName(), *StructProperty->Struct->GetPathName());
						return false;
					}
				}
			}
			else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
			{
				if (UEnum* Enum = EnumProperty->GetEnum())
				{
					if (!IsEnumAllowed(Enum, EPermissionsContext::Pin))
					{
						UE_LOGF(LogBlueprint, Warning, "Function %ls was filtered because one of its parameters %ls (enum %ls) was filtered", *InFunction->GetPathName(), *Property->GetName(), *Enum->GetPathName());
						return false;
					}
				}
			}
			else
			{
				FEdGraphPinType PinType;
				if(GetDefault<UEdGraphSchema_K2>()->ConvertPropertyToPinType(Property, PinType))
				{
					if(!IsPinTypeAllowed(PinType))
					{
						UE_LOGF(LogBlueprint, Warning, "Function %ls was filtered because one of its parameters %ls (%ls) was filtered", *InFunction->GetPathName(), *Property->GetName(), *Property->GetCPPType());
						return false;
					}
				}
				else
				{
					UE_LOGF(LogBlueprint, Warning, "Function %ls was filtered because one of its parameters %ls (%ls) was not able to be converted to a pin", *InFunction->GetPathName(), *Property->GetName(), *Property->GetCPPType());
					return false;
				}
			}
		}
	}

	return true;
}

bool FBlueprintActionDatabase::IsEnumAllowed(UEnum const* InEnum, EPermissionsContext InContext)
{
	const FPathPermissionList& EnumPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetEnumPermissions();
	if(EnumPermissions.HasFiltering())
	{
		TStringBuilder<256> ResultBuilder;
		InEnum->GetPathName(nullptr, ResultBuilder);
		return EnumPermissions.PassesFilter(ResultBuilder.ToView());
	}
	return true;
}

bool FBlueprintActionDatabase::IsEnumAllowed(const FTopLevelAssetPath& InEnumPath, EPermissionsContext InContext)
{
	const FPathPermissionList& EnumPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetEnumPermissions();
	if(EnumPermissions.HasFiltering())
	{
		return EnumPermissions.PassesFilter(InEnumPath.ToString());
	}
	return true;
}

bool FBlueprintActionDatabase::IsStructAllowed(UScriptStruct const* InStruct, EPermissionsContext InContext)
{
	const FPathPermissionList& StructPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetStructPermissions();
	if(StructPermissions.HasFiltering())
	{
		TStringBuilder<256> ResultBuilder;
		InStruct->GetPathName(nullptr, ResultBuilder);
		return StructPermissions.PassesFilter(ResultBuilder.ToView());
	}
	return true;
}

bool FBlueprintActionDatabase::IsStructAllowed(const FTopLevelAssetPath& InStructPath, EPermissionsContext InContext)
{
	const FPathPermissionList& StructPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetStructPermissions();
	if(StructPermissions.HasFiltering())
	{
		return StructPermissions.PassesFilter(InStructPath.ToString());
	}
	return true;
}

bool FBlueprintActionDatabase::IsPinTypeAllowed(const FEdGraphPinType& InPinType, const FTopLevelAssetPath& InUnloadedAssetPath)
{
	// First check if the pin's category is allowed
	const FNamePermissionList& PinCategoryPermissions = GetMutableDefault<UBlueprintEditorSettings>()->GetPinCategoryPermissions();
	if(PinCategoryPermissions.HasFiltering())
	{
		if(!PinCategoryPermissions.PassesFilter(InPinType.PinCategory))
		{
			return false;
		}

		if(InPinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (InUnloadedAssetPath.IsValid())
			{
				return IsStructAllowed(InUnloadedAssetPath, EPermissionsContext::Pin);
			}
			else if(const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InPinType.PinSubCategoryObject))
			{
				return IsStructAllowed(ScriptStruct, EPermissionsContext::Pin);
			}
		}
		else if(InPinType.PinCategory == UEdGraphSchema_K2::PC_Enum || InPinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
		{
			if (InUnloadedAssetPath.IsValid())
			{
				return IsEnumAllowed(InUnloadedAssetPath, EPermissionsContext::Pin);
			}
			else if(const UEnum* Enum = Cast<UEnum>(InPinType.PinSubCategoryObject))
			{
				return IsEnumAllowed(Enum, EPermissionsContext::Pin);
			}
		}
		else if(InPinType.PinCategory == UEdGraphSchema_K2::AllObjectTypes ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_Class ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_Interface ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
		{
			if (InUnloadedAssetPath.IsValid())
			{
				return IsClassAllowed(InUnloadedAssetPath, EPermissionsContext::Pin);
			}
			else if(const UClass* Class = Cast<UClass>(InPinType.PinSubCategoryObject))
			{
				return IsClassAllowed(Class, EPermissionsContext::Pin);
			}
		}
		else if(InPinType.PinCategory == UEdGraphSchema_K2::PC_Delegate ||
				InPinType.PinCategory == UEdGraphSchema_K2::PC_MCDelegate)
		{
			if(const UFunction* PinSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(InPinType.PinSubCategoryMemberReference))
			{
				return IsFunctionAllowed(PinSignature, EPermissionsContext::Pin);
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
