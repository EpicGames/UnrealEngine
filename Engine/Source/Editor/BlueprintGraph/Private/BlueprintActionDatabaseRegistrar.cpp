// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionDatabaseRegistrar.h"

#include "AssetRegistry/AssetData.h"
#include "AssetToolsModule.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintNodeSpawnerUtils.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/PlatformCrt.h"
#include "IAssetTools.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NamePermissionList.h"
#include "Modules/ModuleManager.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

class UEdGraphNode;

/*******************************************************************************
 * BlueprintActionDatabaseRegistrarImpl
 ******************************************************************************/

namespace BlueprintActionDatabaseRegistrarImpl
{
	static UObject const* ResolveClassKey(UClass const* ClassKey);
	static UObject const* ResolveActionKey(UObject const* UserPassedKey);
}

//------------------------------------------------------------------------------
static UObject const* BlueprintActionDatabaseRegistrarImpl::ResolveClassKey(UClass const* ClassKey)
{
	UObject const* ResolvedKey = ClassKey;
	if (UBlueprintGeneratedClass const* BlueprintClass = Cast<UBlueprintGeneratedClass>(ClassKey))
	{
		ResolvedKey = CastChecked<UBlueprint>(BlueprintClass->ClassGeneratedBy.Get(), ECastCheckedType::NullAllowed);
	}
	return ResolvedKey;
}

//------------------------------------------------------------------------------
static UObject const* BlueprintActionDatabaseRegistrarImpl::ResolveActionKey(UObject const* UserPassedKey)
{
	UObject const* ResolvedKey = nullptr;
	if (UClass const* Class = Cast<UClass>(UserPassedKey))
	{
		ResolvedKey = ResolveClassKey(Class);
	}
	// both handled in the IsAsset() case
// 	else if (UUserDefinedEnum const* EnumAsset = Cast<UUserDefinedEnum>(UserPassedKey))
// 	{
// 		ResolvedKey = EnumAsset;
// 	}
// 	else if (UUserDefinedStruct const* StructAsset = Cast<UUserDefinedStruct>(StructOwner))
// 	{
// 		ResolvedKey = StructAsset;
// 	}
	else if (UserPassedKey->IsAsset())
	{
		ResolvedKey = UserPassedKey;
	}
	else if (UField const* MemberField = Cast<UField>(UserPassedKey))
	{
		ResolvedKey = ResolveClassKey(MemberField->GetOwnerClass());
	}

	return ResolvedKey;
}

/*******************************************************************************
 * FBlueprintActionDatabaseRegistrar
 ******************************************************************************/

//------------------------------------------------------------------------------
FBlueprintActionDatabaseRegistrar::FBlueprintActionDatabaseRegistrar(FActionRegistry& Database, FUnloadedActionRegistry& UnloadedDatabase, FPrimingQueue& PrimingQueue, TSubclassOf<UEdGraphNode> DefaultKey)
	: GeneratingClass(DefaultKey)
	, ActionDatabase(Database)
	, UnloadedActionDatabase(UnloadedDatabase)
	, ActionKeyFilter(nullptr)
	, ActionPrimingQueue(PrimingQueue)
{
	AssetTools = &FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner)
{
	UField const* ActionKey = GeneratingClass;
	// if this spawner wraps some member function/property, then we want it 
	// recorded under that class (so we can update the action if the class 
	// changes... like, if the member is deleted, or if one is added)
	const UField* MemberField = FBlueprintNodeSpawnerUtils::GetAssociatedFunction(NodeSpawner);
	if (!MemberField)
	{
		const FProperty* MemberProperty = FBlueprintNodeSpawnerUtils::GetAssociatedProperty(NodeSpawner);
		if (MemberProperty)
		{
			MemberField = MemberProperty->GetOwnerUField();
		}
	}
	if (MemberField)
	{
		ActionKey = MemberField;
	}

	if (!FBlueprintActionDatabase::IsFieldAllowed(ActionKey, FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		return false;
	}

	return AddActionToDatabase(ActionKey, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UClass const* ClassOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	if (!FBlueprintActionDatabase::IsClassAllowed(ClassOwner, FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		return false;
	}

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)

	// ResolveActionKey() is used on ClassOwner (in AddActionToDatabase), to 
	// convert it into a proper key
	return AddActionToDatabase(ClassOwner, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UEnum const* EnumOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	if (!FBlueprintActionDatabase::IsEnumAllowed(EnumOwner, FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		return false;
	}

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)

	// ResolveActionKey() is used on EnumOwner (in AddActionToDatabase), to 
	// convert it into a proper key
	return AddActionToDatabase(EnumOwner, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UScriptStruct const* StructOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	if (!FBlueprintActionDatabase::IsStructAllowed(StructOwner, FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		return false;
	}

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)

	// ResolveActionKey() is used on StructOwner (in AddActionToDatabase), to 
	// convert it into a proper key
	return AddActionToDatabase(StructOwner, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UField const* FieldOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	if (!FBlueprintActionDatabase::IsFieldAllowed(FieldOwner, FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		return false;
	}

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)

	// ResolveActionKey() is used on FieldOwner (in AddActionToDatabase), to 
	// convert it into a proper key
	return AddActionToDatabase(FieldOwner, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(UObject const* AssetOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	if (AssetTools && !AssetTools->IsAssetClassSupported(AssetOwner->GetClass()))
	{
		return false;
	}

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)

	// cannot record an action under any ol' object (we want to associate them 
	// with asset/class outers that are subject to change; so that we can 
	// refresh/rebuild corresponding actions when that happens).
	check(AssetOwner->IsAsset());
	return AddActionToDatabase(AssetOwner, NodeSpawner);
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddBlueprintAction(FAssetData const& AssetDataOwner, UBlueprintNodeSpawner* NodeSpawner)
{
	bool bReturnResult = false;

	// @TODO: assert that AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner) 
	//        wouldn't come up with a different key (besides GeneratingClass)
	if(AssetDataOwner.IsAssetLoaded())
	{
		bReturnResult = AddBlueprintAction(AssetDataOwner.GetAsset(), NodeSpawner);
	}
	else
	{
		bReturnResult = AddBlueprintAction(NodeSpawner->NodeClass, NodeSpawner);
		if(bReturnResult)
		{
			auto& ActionList = UnloadedActionDatabase.FindOrAdd(AssetDataOwner.GetSoftObjectPath());
			ActionList.Add(NodeSpawner);
		}
	}
	return bReturnResult;
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::IsOpenForRegistration(UObject const* OwnerKey)
{
	UObject const* ActionKey = BlueprintActionDatabaseRegistrarImpl::ResolveActionKey(OwnerKey);
	if (ActionKey == nullptr)
	{
		ActionKey = GeneratingClass;
	}
	return (ActionKey != nullptr) && ((ActionKeyFilter == nullptr) || (ActionKeyFilter == ActionKey) || (ActionKeyFilter == OwnerKey));
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::IsOpenForRegistration(FAssetData const& AssetKey)
{
	UObject const* OwnerKey = GeneratingClass;
	if (AssetKey.IsAssetLoaded())
	{
		OwnerKey = AssetKey.GetAsset();
	}
	return IsOpenForRegistration(OwnerKey);
}

//------------------------------------------------------------------------------
int32 FBlueprintActionDatabaseRegistrar::RegisterStructActions(const FMakeStructSpawnerDelegate& MakeActionCallback)
{
	int32 RegisteredCount = 0;

	const FPathPermissionList& Permissions = GetMutableDefault<UBlueprintEditorSettings>()->GetStructPermissions();

	auto RegisterStruct = [this, &MakeActionCallback, &RegisteredCount, &Permissions](const UScriptStruct* InStruct)
	{
		if ((InStruct->StructFlags & STRUCT_NewerVersionExists) == 0)
		{
			if (Permissions.HasFiltering())
			{
				TStringBuilder<256> ResultBuilder;
				InStruct->GetPathName(nullptr, ResultBuilder);
				if (!Permissions.PassesFilter(ResultBuilder.ToView()))
				{
					return;
				}
			}

			if (UBlueprintNodeSpawner* NewAction = MakeActionCallback.Execute(InStruct))
			{
				RegisteredCount += (int32)AddActionToDatabase(InStruct, NewAction);
			}
		}
	};

	// to keep from needlessly looping through UScriptStructs, first check to 
	// see if the registrar is looking for only certain actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (const UObject* RegistrarTarget = GetActionKeyFilter())
	{
		if (const UScriptStruct* StructAsset = Cast<UScriptStruct>(RegistrarTarget))
		{
			check(IsOpenForRegistration(StructAsset));
			RegisterStruct(StructAsset);
		}
		// else, the target is a class or a different asset type... not something pertaining to a struct
	}
	else
	{
		for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
		{
			UScriptStruct const* Struct = (*StructIt);
			RegisterStruct(Struct);
		}
	}
	return RegisteredCount;
}

//------------------------------------------------------------------------------
int32 FBlueprintActionDatabaseRegistrar::RegisterEnumActions(const FMakeEnumSpawnerDelegate& MakeActionCallback)
{
	int32 RegisteredCount = 0;

	const FPathPermissionList& Permissions = GetMutableDefault<UBlueprintEditorSettings>()->GetEnumPermissions();

	auto RegisterEnum = [this, &MakeActionCallback, &RegisteredCount, &Permissions](const UEnum* InEnum)
	{
		if (!InEnum->HasAnyEnumFlags(EEnumFlags::NewerVersionExists) && UEdGraphSchema_K2::IsAllowableBlueprintVariableType(InEnum))
		{
			if (Permissions.HasFiltering())
			{
				TStringBuilder<256> ResultBuilder;
				InEnum->GetPathName(nullptr, ResultBuilder);
				if (!Permissions.PassesFilter(ResultBuilder.ToView()))
				{
					return;
				}
			}

			if (UBlueprintNodeSpawner* NewAction = MakeActionCallback.Execute(InEnum))
			{
				RegisteredCount += (int32)AddActionToDatabase(InEnum, NewAction);
			}
		}
	};

	if (const UObject* RegistrarTarget = GetActionKeyFilter())
	{
		if (const UClass* TargetClass = Cast<UClass>(RegistrarTarget))
		{
			for (TFieldIterator<UEnum> EnumIt(TargetClass, EFieldIteratorFlags::ExcludeSuper); EnumIt; ++EnumIt)
			{
				RegisterEnum(*EnumIt);
			}
		}
		else if (const UEnum* TargetEnum = Cast<UEnum>(RegistrarTarget))
		{
			RegisterEnum(TargetEnum);
		}
	}
	else
	{
		for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
		{
			RegisterEnum(*EnumIt);
		}
	}

	return RegisteredCount;
}

//------------------------------------------------------------------------------
int32 FBlueprintActionDatabaseRegistrar::RegisterClassFactoryActions(const UClass* TargetType, const FMakeFuncSpawnerDelegate& MakeActionCallback)
{
	struct RegisterClassFactoryActions_Utils
	{
		static bool IsFactoryMethod(const UFunction* Function, const UClass* InTargetType)
		{
			if (!Function->HasAnyFunctionFlags(FUNC_Static))
			{
				return false;
			}

			if (!Function->GetOwnerClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) &&
				(!Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction) || GetDefault<UBlueprintEditorSettings>()->bExposeDeprecatedFunctions))
			{
				FObjectProperty* ReturnProperty = CastField<FObjectProperty>(Function->GetReturnProperty());
				// see if the function is a static factory method
				bool const bIsFactoryMethod = (ReturnProperty != nullptr) && (ReturnProperty->PropertyClass != nullptr) &&
					ReturnProperty->PropertyClass->IsChildOf(InTargetType);

				return bIsFactoryMethod;
			}
			else
			{
				return false;
			}
		}
	};

	int32 RegisteredCount = 0;
	if (const UObject* RegistrarTarget = GetActionKeyFilter())
	{
		if (const UClass* TargetClass = Cast<UClass>(RegistrarTarget))
		{
			if (!TargetClass->HasAnyClassFlags(CLASS_Abstract) && !TargetClass->IsChildOf(TargetType))
			{
				for (TFieldIterator<UFunction> FuncIt(TargetClass, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
				{
					UFunction* Function = *FuncIt;
					if (!RegisterClassFactoryActions_Utils::IsFactoryMethod(Function, TargetType))
					{
						continue;
					}
					else if (UBlueprintNodeSpawner* NewAction = MakeActionCallback.Execute(Function))
					{
						RegisteredCount += (int32)AddBlueprintAction(Function, NewAction);
					}
				}
			}
		}
	}
	else
	{
		// these nested loops are combing over the same classes/functions the
		// FBlueprintActionDatabase does; ideally we save on perf and fold this in
		// with FBlueprintActionDatabase, but we want to give separate modules
		// the opportunity to add their own actions per class func
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* Class = *ClassIt;
			if (Class->HasAnyClassFlags(CLASS_Abstract) || !Class->IsChildOf(TargetType))
			{
				continue;
			}
			
			for (TFieldIterator<UFunction> FuncIt(Class, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
			{
				UFunction* Function = *FuncIt;
				if (!RegisterClassFactoryActions_Utils::IsFactoryMethod(Function, TargetType))
				{
					continue;
				}
				else if (UBlueprintNodeSpawner* NewAction = MakeActionCallback.Execute(Function))
				{
					RegisteredCount += (int32)AddBlueprintAction(Function, NewAction);
				}
			}
		}
	}
	return 0;
}

//------------------------------------------------------------------------------
bool FBlueprintActionDatabaseRegistrar::AddActionToDatabase(UObject const* ActionKey, UBlueprintNodeSpawner* NodeSpawner)
{
	ensureMsgf(GeneratingClass == nullptr || NodeSpawner->NodeClass == GeneratingClass, TEXT("We expect a nodes to add only spawners for its own type... Maybe a sub-class is adding nodes it shouldn't?"));
	if (IsOpenForRegistration(ActionKey))
	{
		ActionKey = BlueprintActionDatabaseRegistrarImpl::ResolveActionKey(ActionKey);
		if (ActionKey == nullptr)
		{
			ActionKey = GeneratingClass;
		}
		FBlueprintActionDatabase::FActionList& ActionList = ActionDatabase.FindOrAdd(ActionKey);
		
		int32* QueuedIndex = ActionPrimingQueue.Find(ActionKey);
		if (QueuedIndex == nullptr)
		{
			int32 PrimingIndex = ActionList.Num();
			ActionPrimingQueue.Add(ActionKey, PrimingIndex);
		}

		ActionList.Add(NodeSpawner);
		return true;
	}
	return false;
}
