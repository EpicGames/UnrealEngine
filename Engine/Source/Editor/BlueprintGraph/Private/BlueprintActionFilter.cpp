// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionFilter.h"
#include "Algo/Unique.h"
#include "UObject/Interface.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "Modules/ModuleManager.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimBlueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "K2Node.h"
#include "K2Node_Event.h"
#include "K2Node_CallFunction.h"
#include "K2Node_BaseAsyncTask.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MakeArray.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintNodeSpawnerUtils.h"
#include "BlueprintVariableNodeSpawner.h"
#include "BlueprintEventNodeSpawner.h"
#include "BlueprintBoundEventNodeSpawner.h"
#include "BlueprintBoundNodeSpawner.h"
#include "BlueprintAssetNodeSpawner.h"
#include "Algo/Transform.h"
// "impure" node types (utilized in BlueprintActionFilterImpl::IsImpure)
#include "K2Node_MultiGate.h"
#include "K2Node_Message.h"
#include "K2Node_PromotableOperator.h"
#include "EditorCategoryUtils.h"
#include "BlueprintEditorModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Editor/EditorEngine.h"
#include "BlueprintEditorSettings.h"
#include "AnimNotifyEventNodeSpawner.h"

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
#include "HAL/PlatformStackWalk.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

/*******************************************************************************
 * Static BlueprintActionFilter Helpers
 ******************************************************************************/

namespace BlueprintActionFilterImpl
{
	static FString const ConfigSection("BlueprintEditor.Menu");

	/**
	 * Blueprints have several classes associated with them (the skeleton, for
	 * UI reflection, and the full generated class). This retrieves the 
	 * "authoritative" one for comparisons and queries.
	 * 
	 * @param  Blueprint	The blueprint you want a class for.
	 * @return The authoritative class for the specified blueprint.
	 */
	static UClass* GetAuthoritativeBlueprintClass(UBlueprint const* const Blueprint);

	/**
	 * Since objects (like a blueprint) can have multiple classes (skeleton, 
	 * generated, etc.), this utility method provides a way to compare two 
	 * classes and ensure both are using the same class type.
	 * 
	 * @param  ClassToTest		The class you're querying for.
	 * @param  TypeToCheckFor	The type you want to test for (this is asking "is ClassToTest a TypeToCheckFor class?")
	 * @param  bNeedsExactMatch	Optional parameter. If false, will check IsChildOf() and ImplementsInterface(), otherwise it just checks for equivalence.
	 * @return True if the ClassToTest is the same type as (or inherited from) TypeToCheckFor.
	 */
	static bool IsClassOfType(UClass const* ClassToTest, UClass const* TypeToCheckFor, bool const bNeedsExactMatch = false);

	/**
	 * This not only tells you if an interface is implemented by a class, but 
	 * walks the class inheritance chain to find out which super class actually 
	 * adds the interface.
	 * 
	 * @param  SubClass		The class you're querying for.
	 * @param  Interface	The interface you want to look for.
	 * @return Null if the class doesn't implement the function, otherwise the super class that adds it (could be the class passed in).
	 */
	static UClass const* FindInheritedInterfaceClass(UClass const* SubClass, TSubclassOf<UInterface> Interface);

	/**
	 * 
	 * 
	 * @return 
	 */
	static const TArray<FString>& GetHiddenFieldPaths();

	/**
	 * 
	 * 
	 * @return 
	 */
	static const TArray< TSubclassOf<UEdGraphNode> >& GetHiddenNodeTypes();
	
	/**
	 * Utility method to check and see if the specified node-spawner would 
	 * produce an impure node.
	 * 
	 * @param  NodeSpawner	The action you want to query.
	 * @return True if the action will spawn an impure node, false if not (or unknown).
	 */
	static bool IsImpure(FBlueprintActionInfo& NodeSpawner);

	/**
	 * Utility method to check and see if the specified node-spawner would 
	 * produce a latent node.
	 * 
	 * @param  NodeSpawner	The action you want to query.
	 * @return True if the action will spawn a latent node, false if not (or unknown).
	 */
	static bool IsLatent(FBlueprintActionInfo& NodeSpawner);
	
	/**
	 * Utility method to check and see if the specified field is a public global
	 * or static field (that is has a persistent extent that span's the 
	 * program's lifetime).
	 *
	 * @param  Field	The field you want to check.
	 * @return True if the field is global/static (and public), false if it has a limited extent (or is private/protected).
	 */
	static bool IsGloballyAccessible(FFieldVariant Field);
	
	/**
	 * Rejection test that checks to see if the supplied node-spawner would
	 * produce an event that does NOT belong to the specified blueprint.
	 * 
	 * @param  Filter			Holds the blueprint context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the node-spawner would produce an event incompatible with the specified blueprint(s).
	 */
	static bool IsEventUnimplementable(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner has an 
	 * associated field that is not accessible by the blueprint (it's private or
	 * protected).
	 * 
	 * @param  Filter			Holds the blueprint context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the node-spawner is associated with a private field.
	 */
	static bool IsFieldInaccessible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner has an 
	 * associated class that is "restricted" and thusly, hidden from the 
	 * blueprint.
	 * 
	 * @param  Filter			Holds the blueprint context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the node-spawner belongs to a class that is restricted to certain blueprints (not this one).
	 */
	static bool IsRestrictedClassMember(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);
	
	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce an asset-related action that cannot be accessed according to 
	 * asset permissions
	 * 
	 * @param  Filter			Holds the blueprint context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn a variable-set node for a read-only property.
	 */
	static bool IsAssetPermissionNotGranted(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce a variable-set node when the property is read-only (in this
	 * blueprint).
	 * 
	 * @param  Filter			Holds the blueprint context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn a variable-set node for a read-only property.
	 */
	static bool IsPermissionNotGranted(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce a node (or comes from an associated class) that is deprecated.
	 * 
	 * @param  Filter			Filter context (unused) for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn a node that is deprecated.
	 */
	static bool IsDeprecated(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);
	
	/**
	* Rejection test that checks to see if the supplied node-spawner would 
	* produce a node (or comes from an associated class) that is deprecated.
	* 
	* @param  Filter			Filter context (unused) for this test.
	* @param  BlueprintAction	The action you wish to query.
	* @return True if the action would spawn a node that is deprecated.
	*/
	static bool IsPropertyAccessorNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	* Rejection test that checks to see if the supplied node-spawner would produce a node with thread safety
	* characteristics that are incompatible with the specification of the containing graph. 
	* 
	* @param  Filter			Filter context (unused) for this test.
	* @param  BlueprintAction	The action you wish to query.
	* @return True if the action would spawn a node that has incompatible thread safety.
	*/
	static bool IsThreadSafetyIncompatible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce an impure node, incompatible with the specified graphs.
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn an impure node (and the graph doesn't allow them).
	 */
	static bool IsIncompatibleImpureNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);
	
	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce a latent node, incompatible with the specified graph.
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn a node (and the graph wouldn't allow it).
	 */
	static bool IsIncompatibleLatentNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the supplied node-spawner would 
	 * produce a node incompatible with the specified graph.
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would spawn a node (and the graph wouldn't allow it).
	 */
	static bool IsIncompatibleWithGraphType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the node-spawner has any associated
	 * "non-target" fields that are global/static.
	 * 
	 * @param  Filter			Holds the TagetClass context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action should be rejected.
	 */
	static bool IsRejectedGlobalField(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the node-spawner represents an object
	 * that does NOT exist within the scope of the editor's current import set.
	 *
	 * @param  Filter			The filter context for this test.
	 * @param  InObject			The associated object you wish to query.
	 * @return True if the action is associated with an object that is not included in any imported or global namespace.
	 */
	static bool IsNonImportedObject(FBlueprintActionFilter const& Filter, const UObject* InObject);

	/**
	 * Rejection test that checks to see if the node-spawner is associated with 
	 * a field that belongs to a class that is not allowed (ignores global/
	 * static fields).
	 * 
	 * @param  Filter					Holds the class context for this test.
	 * @param  BlueprintAction			The action you wish to query.
	 * @return True if the action is associated with a non-allowed class member.
	 */
	static bool IsNonTargetMember(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the node-spawner is associated with 
	 * a field that is hidden from the specified blueprint (via metadata).
	 * 
	 * @param  Filter			Holds the class context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action is associated with a hidden field.
	 */
	static bool IsFieldCategoryHidden(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter    
	 * @param  BlueprintAction    
	 * @return 
	 */
	static bool IsActionHiddenByConfig(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the node-spawner would produce a 
	 * node type that isn't allowed
	 * 
	 * @param  Filter				Holds the class context for this test.
	 * @param  BlueprintAction		The action you wish to query.
	 * @return True if the action would produce a denied node.
	 */
	static bool IsFilteredNodeType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks to see if the node-spawner is tied to a 
	 * specific object that is not currently selected.
	 * 
	 * 
	 * @param  Filter			Holds the selected object context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action would produce a bound node, tied to an object that isn't selected.
	 */
	static bool IsUnBoundBindingSpawner(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsOutOfScopeLocalVariable(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsLevelScriptActionValid(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsSchemaIncompatible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  BlueprintAction	
	 * @param  Pin	
	 * @return 
	 */
	static bool HasMatchingPin(FBlueprintActionInfo& BlueprintAction, UEdGraphPin const* Pin);

	/**
	 * 
	 * 
	 * @param  Pin	
	 * @param  MemberNodeSpawner
	 * @return 
	 */
	static bool IsPinCompatibleWithTargetSelf(UEdGraphPin const* Pin, FBlueprintActionInfo& MemberNodeSpawner);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsFunctionMissingPinParam(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 */
	static bool ArrayFunctionHasParamOfType(const UFunction* ArrayFunction, UEdGraph const* InGraph, const FEdGraphPinType& DesiredPinType, bool bWantOutput);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsMissmatchedPropertyType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsMissingMatchingPinParam(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Dynamic casts should only show results for casting to classes that the 
	 * context pin is a child of (and not itself).
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsNotSubClassCast(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks if the NodeTemplate (if any)
	 * decides to filter itself out of the action list
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return
	 */
	static bool IsNodeTemplateSelfFiltered(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks if the skeleton associated with the current blueprint
	 * will raise any of the available anim notification events
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action is an animnotification that is incompatible with the current skeleton
	 */
	static bool IsIncompatibleAnimNotification( FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction); 
	
	/**
	 * 
	 * 
	 * @param  Filter	
	 * @param  BlueprintAction	
	 * @return 
	 */
	static bool IsExtraneousInterfaceCall(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test that checks if a macro instance is incompatible with the current graph context
	 * 
	 * @param  Filter			Holds the graph context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the macro instance is incompatible with the current graph context
	 */
	static bool IsIncompatibleMacroInstance(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	 * Rejection test to help unblock common crashes, where programmers forget 
	 * to refresh the BlueprintActionDatabase when a blueprint has been re-compiled.
	 * 
	 * @param  Filter			Holds the action/field context for this test.
	 * @param  BlueprintAction	The action you wish to query.
	 * @return True if the action is stale (associated with a TRASH or REINST class).
	 */
	static bool IsStaleFieldAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	/**
	* Rejection test whether editor module functionality should be filtered out for 
	* this specific blueprint (relies on whether or not base class is part of an editor module).
	*
	* @param  Filter			Holds the action/field context for this test.
	* @param  BlueprintAction	The action you wish to query.
	* @return True if the action is invalid to use in the current blueprint
	*/
	static bool IsHiddenInNonEditorBlueprint(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction);

	//------------------------------------------------------------------------------
#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	static TAutoConsoleVariable<bool> CVarBPEnableActionMenuFilterTestTraceLogging(
		TEXT("BP.EnableActionMenuFilterTestTraceLogging"),
		false,
		TEXT("If enabled, each filter test will be individually profiled for Insights.")
	);

	static TAutoConsoleVariable<bool> CVarBPEnableActionMenuFilterTestStatsLogging(
		TEXT("BP.EnableActionMenuFilterTestStatsLogging"),
		false,
		TEXT("If enabled, filter test stats will be logged for each menu section each time a Blueprint action menu is built for display.")
	);

	static FName FilterTestProfileEventName(TEXT("BlueprintActionFilterImpl::FilterTestProfileEvent"));

	static const FString& FilterTestFuncPtrToFuncName(uint64 TestFuncPtr)
	{
		static TMap<uint64, FString> FuncPtrToFuncNameMap;

		const FString* FuncNamePtr = FuncPtrToFuncNameMap.Find(TestFuncPtr);
		if (!FuncNamePtr)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FilterTestProfile_FuncPtrToFuncName);

			FProgramCounterSymbolInfo SymbolInfo;
			FPlatformStackWalk::ProgramCounterToSymbolInfo(TestFuncPtr, SymbolInfo);

			FuncNamePtr = &FuncPtrToFuncNameMap.Add(TestFuncPtr, SymbolInfo.FunctionName);
		}

		return *FuncNamePtr;
	}

	static uint32 FilterTestFuncPtrToCpuSpecId(uint64 TestFuncPtr)
	{
		static TMap<uint64, uint32> FuncPtrToCpuSpecIdMap;

		uint32* CpuSpecIdPtr = FuncPtrToCpuSpecIdMap.Find(TestFuncPtr);
		if (!CpuSpecIdPtr)
		{
			check(UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel));
			CpuSpecIdPtr = &FuncPtrToCpuSpecIdMap.Add(TestFuncPtr, FCpuProfilerTrace::OutputEventType(*FilterTestFuncPtrToFuncName(TestFuncPtr)));
		}

		return *CpuSpecIdPtr;
	}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
};

//------------------------------------------------------------------------------
UClass* BlueprintActionFilterImpl::GetAuthoritativeBlueprintClass(UBlueprint const* const Blueprint)
{
	UClass* BpClass = (Blueprint->SkeletonGeneratedClass != nullptr) ? Blueprint->SkeletonGeneratedClass :
		Blueprint->GeneratedClass;

	if (BpClass == nullptr)
	{
		BpClass = Blueprint->ParentClass;
	}

	UClass* AuthoritativeClass = BpClass;
	if (BpClass != nullptr)
	{
		if (ensureMsgf(!BpClass->GetClass()->IsChildOf<UBlueprintGeneratedClass>() || BpClass->ClassGeneratedBy
			, TEXT("Ambiguous Blueprint: '%s'; with skeleton: %s (%s), class: %s (%s), and parent: %s (%s) - authoratative class: %s (%s)")
			, *Blueprint->GetPathName()
			, Blueprint->SkeletonGeneratedClass ? *Blueprint->SkeletonGeneratedClass->GetName() : TEXT("[NULL]")
			, Blueprint->SkeletonGeneratedClass ? *Blueprint->SkeletonGeneratedClass->GetClass()->GetName() : TEXT("N/A")
			, Blueprint->GeneratedClass ? *Blueprint->GeneratedClass->GetName() : TEXT("[NULL]")
			, Blueprint->GeneratedClass ? *Blueprint->GeneratedClass->GetClass()->GetName() : TEXT("N/A")
			, Blueprint->ParentClass ? *Blueprint->ParentClass->GetName() : TEXT("[NULL]")
			, Blueprint->ParentClass ? *Blueprint->ParentClass->GetClass()->GetName() : TEXT("N/A")
			, BpClass ? *BpClass->GetName() : TEXT("[NULL]")
			, BpClass ? *BpClass->GetClass()->GetName() : TEXT("N/A")) )
		{
			AuthoritativeClass = BpClass->GetAuthoritativeClass();
		}		
	}
	return AuthoritativeClass;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsClassOfType(UClass const* ClassToTest, UClass const* TypeToCheckFor, bool const bNeedsExactMatch)
{
	// @TODO: get rid of these nasty const_casts ... ew! (spent a while 
	//        following the const rabbit down the hole, and eventually gave up)
	//        should make a const version of GetAuthoritativeClass(), somehow
	//        without duplicating functionality
	UClass const* AuthoritativeTestClass = const_cast<UClass*>(ClassToTest)->GetAuthoritativeClass();
	UClass const* AuthoritativeToTestFor = const_cast<UClass*>(TypeToCheckFor)->GetAuthoritativeClass();

	if (bNeedsExactMatch)
	{
		return (AuthoritativeTestClass == AuthoritativeToTestFor);
	}
	else
	{
		return AuthoritativeTestClass->IsChildOf(AuthoritativeToTestFor) ||
			AuthoritativeTestClass->ImplementsInterface(AuthoritativeToTestFor);
	}
}

//------------------------------------------------------------------------------
static UClass const* BlueprintActionFilterImpl::FindInheritedInterfaceClass(UClass const* SubClass, TSubclassOf<UInterface> Interface)
{
	UClass const* ImplementsInterface = nullptr;

	UClass const* ClassToCheck = SubClass;
	while ((ClassToCheck != nullptr) && (ImplementsInterface == nullptr))
	{
		for (FImplementedInterface const& ClassInterface : ClassToCheck->Interfaces)
		{
			UClass const* InterfaceClass = ClassInterface.Class;
			if (InterfaceClass->IsChildOf(Interface))
			{
				ImplementsInterface = ClassToCheck;
			}
		}
		ClassToCheck = ClassToCheck->GetSuperClass();
	}
	return ImplementsInterface;
}

//------------------------------------------------------------------------------
static const TArray<FString>& BlueprintActionFilterImpl::GetHiddenFieldPaths()
{
	static TArray<FString> HiddenFields;

	static bool bInited = false;
	if (!bInited)
	{
		FString const HiddenFieldsId("BlueprintHiddenFields");
		GConfig->GetArray(*ConfigSection, *HiddenFieldsId, HiddenFields, GEditorIni);

		bInited = true;
	}

	return HiddenFields;
}

//------------------------------------------------------------------------------
static const TArray< TSubclassOf<UEdGraphNode> >& BlueprintActionFilterImpl::GetHiddenNodeTypes()
{
	static TArray< TSubclassOf<UEdGraphNode> > HiddenNodeTypes;

	static bool bInited = false;
	if (!bInited)
	{
		TArray<FString> HiddenClassNames;

		FString const HiddenFieldsId("BlueprintHiddenNodes");
		GConfig->GetArray(*ConfigSection, *HiddenFieldsId, HiddenClassNames, GEditorIni);

		HiddenNodeTypes.Reserve(HiddenClassNames.Num());
		for (FString const& ClassName : HiddenClassNames)
		{
			if (UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName))
			{
				HiddenNodeTypes.Add(FoundClass);
			}
		}

		bInited = true;
	}

	return HiddenNodeTypes;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsImpure(FBlueprintActionInfo& BlueprintAction)
{
	bool bIsImpure = false;
	
	if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		bIsImpure = (Function->HasAnyFunctionFlags(FUNC_BlueprintPure) == false);
	}
	else
	{
		UClass const* NodeClass = BlueprintAction.GetNodeClass();
		check(NodeClass != nullptr);

		// TODO: why are some of these "impure"?... we shouldn't have hardcoded 
		//       node types here (game modules cannot add their node types here,
		//       so we should find another way of identifying "pure" node types
		//       ... maybe look for an exec pin?)
		bIsImpure = (NodeClass == UK2Node_IfThenElse::StaticClass()) ||
			(NodeClass == UK2Node_MultiGate::StaticClass()) ||
			(NodeClass == UK2Node_MakeArray::StaticClass()) ||
			(NodeClass == UK2Node_Message::StaticClass()) ||
			(NodeClass == UK2Node_ExecutionSequence::StaticClass());
	}
	
	return bIsImpure;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsLatent(FBlueprintActionInfo& BlueprintAction)
{
	bool bIsLatent = false;

	UClass const* NodeClass = BlueprintAction.GetNodeClass();
	if ((NodeClass != nullptr) && NodeClass->IsChildOf<UK2Node_BaseAsyncTask>())
	{
		bIsLatent = true;
	}
	else if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		bIsLatent = (Function->HasMetaData(FBlueprintMetadata::MD_Latent) != false);
	}

	return bIsLatent;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsGloballyAccessible(FFieldVariant Field)
{
	bool bHasPersistentExtents = false; // is global or static
	// todo: Consider using !HasAnyFlags(CPF_NativeAccessSpecifierProtected|CPF_NativeAccessSpecifierPrivate) for non-UObject fields
	// (i.e. FProperties) to match the documentation of this function.
	bool bIsPublic = Field.IsUObject() ? Field.ToUObject()->HasAnyFlags(RF_Public) : true; 
	
	UClass* ClassOuter = Cast<UClass>(Field.IsUObject() ? Field.ToUObject()->GetOuter() : Field.ToField()->GetOwner<UObject>());
	// For UObjects like global enums, structs, etc., the outer may be a UPackage, treat these a globally accessible.
	// For FProperties the outer may be a UClass or a UStruct, and we are currently
	// treating all UStruct properties (including private and non-static ones) as globally accessible.
	if (ClassOuter == nullptr)
	{
		bHasPersistentExtents = true;
	}
	else if (UFunction const* Function = Cast<UFunction>(Field.ToUObject()))
	{
		bIsPublic |= !Function->HasMetaData(FBlueprintMetadata::MD_Protected) &&
			!Function->HasMetaData(FBlueprintMetadata::MD_Private);
		
		bHasPersistentExtents = Function->HasAnyFunctionFlags(FUNC_Static);
	}
	
	return bIsPublic && bHasPersistentExtents;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsEventUnimplementable(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;
	
	if (UBlueprintEventNodeSpawner const* EventSpawner = Cast<UBlueprintEventNodeSpawner>(BlueprintAction.NodeSpawner))
	{
		if (UFunction const* EventFunc = EventSpawner->GetEventFunction())
		{
			UClass* FuncOwner = EventFunc->GetOuterUClass();
			for (UBlueprint const* Blueprint : FilterContext.Blueprints)
			{
				UClass const* BpClass = GetAuthoritativeBlueprintClass(Blueprint);
				if (!ensureMsgf(BpClass != nullptr
					, TEXT("Unable to resolve IsEventUnimplementable() - Blueprint (%s) missing an authoratative class (skel: %s, generated: %s, parent: %s)")
					, *Blueprint->GetName()
					, Blueprint->SkeletonGeneratedClass ? *Blueprint->SkeletonGeneratedClass->GetName() : TEXT("[NULL]")
					, Blueprint->GeneratedClass ? *Blueprint->GeneratedClass->GetName() : TEXT("[NULL]")
					, Blueprint->ParentClass ? *Blueprint->ParentClass->GetName() : TEXT("[NULL]")) )
				{
					continue;
				}

				// if this function belongs directly to this blueprint, then it is
				// already implemented here (this action however is valid for sub-
				// classes, as they can override the event's functionality)
				if (IsClassOfType(BpClass, FuncOwner, /*bNeedsExactMatch =*/true))
				{
					bIsFilteredOut = true;
					break;
				}

				// you can only implement events that you inherit; so if this
				// blueprint is not a subclass of the event's owner, then we're not
				// allowed to implement it
				if (!IsClassOfType(BpClass, FuncOwner))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsFieldInaccessible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;
	
	FFieldVariant Field = BlueprintAction.GetAssociatedMemberField();
	bool const bIsMemberAction = Field.IsValid() && Field.GetOwnerClass() != nullptr;

	if (bIsMemberAction)
	{
		bool const bIsProtected = Field.HasMetaData(FBlueprintMetadata::MD_Protected);
		bool const bIsPrivate   = Field.HasMetaData(FBlueprintMetadata::MD_Private);
		bool const bIsPublic    = !bIsPrivate && !bIsProtected;

		// @TODO: Trying to respect the "editable"/DisableEditOnInstance toggle
		//        was a bad idea that lead to confusion amongst users (also this 
		//        created a discrepancy between native and blueprint variables),
		//        until we make this concept more understandable: hold off
// 		if (FProperty const* Property = CastField<FProperty>(Field))
// 		{
// 			// default to assuming that this property is a native one (the
// 			// accessibility there is a little more lax)
// 			bool bIsNativeProperty = true;
// 			if (UClass* PropertyClass = Property->GetOwnerClass())
// 			{
// 				bIsNativeProperty = (Cast<UBlueprintGeneratedClass>(PropertyClass) == nullptr);
// 			}
// 
// 			// if this is a native property, then the user only should have to 
// 			// specify BlueprintReadWrite or BlueprintReadOnly for it to be 
// 			// accessible
// 			if (bIsNativeProperty)
// 			{
// 				bIsPublic &= Property->HasAnyPropertyFlags(CPF_BlueprintVisible);
// 			}
// 			else
// 			{
// 				// however, if this is a blueprint variable, then the
// 				// CPF_DisableEditOnInstance corresponds to the eyeball/editable 
// 				// checkbox in the editor, and we don't want the variable to be 
// 				// public until the user exposes it (delegates don't have an
// 				// 'editable' toggle, so they should default to public)
// 				bIsPublic &= !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance) ||
// 					(CastField<FMulticastDelegateProperty>(Property) != nullptr);
// 
// 				// @TODO: allow users to choose "read-only" for blueprint 
// 				//        variables ("private" might have provided this in the 
// 				//        past, but that is just wrong, or needs to be renamed)
// 			}
// 
// 			// If the variable is not public, and is not tagged as private, it should be protected
// 			// by default. This branch handles variables declared in blueprints:
// 			if (!bIsNativeProperty && !bIsPublic && !bIsPrivate)
// 			{
// 				bIsProtected = true;
// 			}
// 		}

		if( !bIsPublic )
		{
			UClass const* ActionOwner = BlueprintAction.GetOwnerClass();
			for (UBlueprint const* Blueprint : FilterContext.Blueprints)
			{
				UClass const* BpClass = GetAuthoritativeBlueprintClass(Blueprint);
				if (!ensureMsgf(BpClass != nullptr
					, TEXT("Unable to resolve IsFieldInaccessible() - Blueprint (%s) missing an authoratative class (skel: %s, generated: %s, parent: %s)")
					, *Blueprint->GetName()
					, Blueprint->SkeletonGeneratedClass ? *Blueprint->SkeletonGeneratedClass->GetName() : TEXT("[NULL]")
					, Blueprint->GeneratedClass ? *Blueprint->GeneratedClass->GetName() : TEXT("[NULL]")
					, Blueprint->ParentClass ? *Blueprint->ParentClass->GetName() : TEXT("[NULL]")))
				{
					continue;
				}
			
				// private functions are only accessible from the class they belong to
				if (bIsPrivate && !IsClassOfType(BpClass, ActionOwner, /*bNeedsExactMatch =*/true))
				{
					bIsFilteredOut = true;
					break;
				}
				else if (bIsProtected && !IsClassOfType(BpClass, ActionOwner))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsRestrictedClassMember(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;
	
	if (UClass const* ActionClass = BlueprintAction.GetOwnerClass())
	{
		if (ActionClass->HasMetaData(FBlueprintMetadata::MD_RestrictedToClasses))
		{
			FString const& ClassRestrictions = ActionClass->GetMetaData(FBlueprintMetadata::MD_RestrictedToClasses);
			
			// Parse the the metadata into an array that is delimited by ',' and trim whitespace
			TArray<FString> ParsedClassRestrictions;
			ClassRestrictions.ParseIntoArray(ParsedClassRestrictions, TEXT(","));
			for (FString& ValidClassName : ParsedClassRestrictions)
			{
				ValidClassName = ValidClassName.TrimStartAndEnd();
			}

			for (UBlueprint const* TargetContext : FilterContext.Blueprints)
			{
				UClass* TargetClass = TargetContext->GeneratedClass;
				if(!TargetClass)
				{
					// Skip possible null classes (e.g. macros, etc)
					continue;
				}

				bool bIsClassListed = false;
				
				UClass const* QueryClass = TargetClass;
				// walk the class inheritance chain to see if this class is one
				// of the allowed
				while (!bIsClassListed && (QueryClass != nullptr))
				{
					FString const ClassName = QueryClass->GetName();
					// If this class is on the list of valid classes
					for (const FString& ValidClassName : ParsedClassRestrictions)
					{
						bIsClassListed = (ClassName == ValidClassName);
						if (bIsClassListed)
						{
							break;
						}
					}
					
					QueryClass = QueryClass->GetSuperClass();
				}
				
				// if the blueprint class wasn't listed as one of the few
				// classes that this can be accessed from, then filter it out
				if (!bIsClassListed)
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsAssetPermissionNotGranted(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	// This handles actions with loaded assets
	UObject const* const Owner = BlueprintAction.GetActionOwner();
	if (Owner && Owner->IsAsset())
	{
		if (Filter.AssetReferenceFilter.IsValid())
		{
			bool const bIsAccessible = Filter.AssetReferenceFilter->PassesFilter(FAssetData(Owner, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering));
			if (!bIsAccessible)
			{
				bIsFilteredOut = true;
			}
		}
	}
	// Handle unloaded asset spawners
	else if(UBlueprintAssetNodeSpawner const* AssetNodeSpawner = Cast<UBlueprintAssetNodeSpawner>(BlueprintAction.NodeSpawner))
	{
		const FAssetData& AssetData = AssetNodeSpawner->GetAssetData();
		if (Filter.AssetReferenceFilter.IsValid() && AssetData.IsValid())
		{
			bool const bIsAccessible = Filter.AssetReferenceFilter->PassesFilter(AssetData);
			if (!bIsAccessible)
			{
				bIsFilteredOut = true;
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsPermissionNotGranted(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	if (FProperty const* const Property = BlueprintAction.GetAssociatedProperty())
	{
		UClass const* const NodeClass = BlueprintAction.GetNodeClass();
		for (UBlueprint const* Blueprint : FilterContext.Blueprints)
		{
			bool const bIsWritable = (FBlueprintEditorUtils::IsPropertyWritableInBlueprint(Blueprint, Property) == FBlueprintEditorUtils::EPropertyWritableState::Writable);
			if (!bIsWritable && NodeClass && NodeClass->IsChildOf<UK2Node_VariableSet>())
			{
				bIsFilteredOut = true;
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsDeprecated(FBlueprintActionFilter const& /*Filter*/, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	ensure(BlueprintAction.GetNodeClass() != nullptr);
	if (UClass const* NodeClass = BlueprintAction.GetNodeClass())
	{
		bIsFilteredOut |= NodeClass->HasAnyClassFlags(CLASS_Deprecated);
	}
	
	if (UClass const* ActionClass = BlueprintAction.GetOwnerClass())
	{
		bIsFilteredOut |= ActionClass->HasAnyClassFlags(CLASS_Deprecated);
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsNonImportedObject(FBlueprintActionFilter const& Filter, const UObject* InObject)
{
	// All objects are considered to be imported if there is no context.
	bool bIsFilteredOut = false;

	if (InObject)
	{
		TSharedPtr<IBlueprintEditor> BlueprintEditor = Filter.Context.EditorPtr.Pin();
		if (BlueprintEditor.IsValid())
		{
			bIsFilteredOut = BlueprintEditor->IsNonImportedObject(InObject);
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsRejectedGlobalField(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	const bool bRejectAllGlobalFields = Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectGlobalFields);
	const bool bRejectNonImportedFields = Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectNonImportedFields);

	bool bIsFilteredOut = false;
	FFieldVariant Field = BlueprintAction.GetAssociatedMemberField();
	if (Field.IsValid() && IsGloballyAccessible(Field))
	{
		const UStruct* OuterType = nullptr;
		if (const UField* FieldAsUObject = Cast<UField>(Field.ToUObject()))
		{
			OuterType = FieldAsUObject->GetOwnerStruct();
		}
		else
		{
			OuterType = Field.ToField()->GetOwnerStruct();
		}

		bIsFilteredOut = bRejectAllGlobalFields || (bRejectNonImportedFields && OuterType && IsNonImportedObject(Filter, OuterType));
		
		UClass* FieldClass = Field.GetOwnerClass();
		if (bIsFilteredOut && (FieldClass != nullptr))
		{
			for (const auto& ClassData : Filter.TargetClasses)
			{
				bool const bIsInternalMemberField = IsClassOfType(ClassData.TargetClass, FieldClass);
				if (bIsInternalMemberField)
				{
					bIsFilteredOut = false;
					break;
				}
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsNonTargetMember(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	// Determines if this test should pass for external global/static fields.
	const bool bPermitNonTargetGlobals = !Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectGlobalFields);

	FFieldVariant ClassField = BlueprintAction.GetAssociatedMemberField();
	bool const bIsMemberAction = ClassField.IsValid() && ClassField.GetOwnerClass() != nullptr;

	if (bIsMemberAction)
	{
		UClass const* ActionClass = BlueprintAction.GetOwnerClass();
		checkSlow(ActionClass != nullptr);

		// global (and static library) fields can stay (unless explicitly
		// excluded... save that for a separate test)
		bool const bSkip = (bPermitNonTargetGlobals && IsGloballyAccessible(ClassField)) || BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_Message>();
		if (!bSkip)
		{
			bIsFilteredOut = Filter.TargetClasses.Num() > 0;

			for (const auto& ClassData : Filter.TargetClasses)
			{
				bool const bIsTargetOwnedField = IsClassOfType(ClassData.TargetClass, ActionClass);
				if (bIsTargetOwnedField)
				{
					bIsFilteredOut = false;
					break;
				}
			}
		}
	}
	
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsFieldCategoryHidden(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	const UFunction* NodeFunction = BlueprintAction.GetAssociatedFunction();
	if ((NodeFunction != nullptr) && NodeFunction->HasAnyFunctionFlags(FUNC_Static))
	{
		bIsFilteredOut = false;
	}
	else
	{
		bIsFilteredOut = (Filter.TargetClasses.Num() > 0);

		for (const auto& ClassData : Filter.TargetClasses)
		{
			// Use the UiSpec to get the category
			FBlueprintActionUiSpec UiSpec = BlueprintAction.NodeSpawner->GetUiSpec(Filter.Context, BlueprintAction.GetBindings());
			if (!FEditorCategoryUtils::IsCategoryHiddenFromClass(ClassData.HiddenCategories, ClassData.TargetClass, UiSpec.Category.ToString()))
			{
				bIsFilteredOut = false;
				break;
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsActionHiddenByConfig(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	FFieldVariant ActionField = BlueprintAction.GetAssociatedMemberField();
	if (ActionField.IsValid())
	{
		const TArray<FString>& HiddenFields = GetHiddenFieldPaths();

		FString const FieldPath = ActionField.GetPathName();
		if (HiddenFields.Contains(FieldPath))
		{
			bIsFilteredOut = true;
		}
	}

	if (!bIsFilteredOut)
	{
		const TArray< TSubclassOf<UEdGraphNode> >& HiddenNodes = GetHiddenNodeTypes();
		if (HiddenNodes.Contains(BlueprintAction.GetNodeClass()))
		{
			bIsFilteredOut = true;
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsPropertyAccessorNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsAccessor = false;

	if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		bIsAccessor = (Function->HasMetaData(FBlueprintMetadata::MD_PropertySetFunction) || Function->HasMetaData(FBlueprintMetadata::MD_PropertyGetFunction));
	}

	return bIsAccessor;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsThreadSafetyIncompatible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{	
		bool bAllowNonThreadSafeNodes = true;

		FBlueprintActionContext const& FilterContext = Filter.Context;

		for (UEdGraph* Graph : FilterContext.Graphs)
		{
			if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema()))
			{
				bAllowNonThreadSafeNodes &= !K2Schema->IsGraphMarkedThreadSafe(Graph);
			}
		}

		if(!bAllowNonThreadSafeNodes)
		{
			return !FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Function);
		}
	}
	
	return false;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsIncompatibleImpureNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bAllowImpureNodes = true;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UEdGraph* Graph : FilterContext.Graphs)
	{
		if (const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema()))
		{
			bAllowImpureNodes &= K2Schema->DoesGraphSupportImpureFunctions(Graph);
		}
	}
	
	bool bIsFilteredOut = !bAllowImpureNodes && IsImpure(BlueprintAction);
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsIncompatibleLatentNode(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bAllowLatentNodes = true;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UEdGraph* Graph : FilterContext.Graphs)
	{
		if (const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>())
		{
			if(K2Schema->GetGraphType(Graph) == GT_Function)
			{
				bAllowLatentNodes = false;
			}
		}
	}

	bool bIsFilteredOut = !bAllowLatentNodes && IsLatent(BlueprintAction);
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsIncompatibleWithGraphType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;
	
	if (UClass const* NodeClass = BlueprintAction.GetNodeClass())
	{
		if (UEdGraphNode const* NodeCDO = CastChecked<UEdGraphNode>(NodeClass->GetDefaultObject(false)))
		{
			for (UEdGraph* Graph : FilterContext.Graphs)
			{
				if (!NodeCDO->IsCompatibleWithGraph(Graph))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsFilteredNodeType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = (Filter.PermittedNodeTypes.Num() > 0);

	// When true, the node's class doesn't have to perfectly match a class to be passed (it can be a sub-class).
	const bool bPermitChildClasses = !Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectPermittedSubClasses);

	// When true, the node's class doesn't have to perfectly match a class to be rejected (it can be a sub-class).
	const bool bRejectChildClasses = !Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_PermitRejectionSubClasses);

	UClass const* NodeClass = BlueprintAction.GetNodeClass();
	if (NodeClass != nullptr)
	{
		for (TSubclassOf<UEdGraphNode> AllowedClass : Filter.PermittedNodeTypes)
		{
			if ((AllowedClass == NodeClass) || 
				(bPermitChildClasses && NodeClass->IsChildOf(AllowedClass)))
			{
				bIsFilteredOut = false;
				break;
			}
		}
		
		for (int32 ClassIndx = 0; !bIsFilteredOut && (ClassIndx < Filter.RejectedNodeTypes.Num()); ++ClassIndx)
		{
			TSubclassOf<UEdGraphNode> ExcludedClass = Filter.RejectedNodeTypes[ClassIndx];
			if ((ExcludedClass == NodeClass) ||
				(bRejectChildClasses && NodeClass->IsChildOf(ExcludedClass)))
			{
				bIsFilteredOut = true;
				break;
			}
		}
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsUnBoundBindingSpawner(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool const bIsBindingSpecificSpawner = (Cast<UBlueprintBoundEventNodeSpawner>(BlueprintAction.NodeSpawner) != nullptr || Cast<UBlueprintBoundNodeSpawner>(BlueprintAction.NodeSpawner) != nullptr);

	bool bIsFilteredOut = false;
	if (bIsBindingSpecificSpawner)
	{
		bIsFilteredOut = (BlueprintAction.GetBindings().Num() <= 0);
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsOutOfScopeLocalVariable(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	if (UBlueprintVariableNodeSpawner const* VarSpawner = Cast<UBlueprintVariableNodeSpawner>(BlueprintAction.NodeSpawner))
	{
		if (VarSpawner->IsLocalVariable())
		{
			bIsFilteredOut = (Filter.Context.Graphs.Num() == 0);

			UEdGraph const* VarOuter = Cast<UEdGraph const>(VarSpawner->GetVarOuter().ToUObject());
			for (UEdGraph* Graph : Filter.Context.Graphs)
			{
				if (FBlueprintEditorUtils::GetTopLevelGraph(Graph) != VarOuter)
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsLevelScriptActionValid(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;
	UClass* OuterClass = nullptr;

	if( UFunction const* NodeFunction = BlueprintAction.GetAssociatedFunction() )
	{
		OuterClass = Cast<UClass>( NodeFunction->GetOuter() );
	}
	else if( FProperty const* NodeProperty = BlueprintAction.GetAssociatedProperty() )
	{
		OuterClass = NodeProperty->GetOwner<UClass>();
	}

	if( OuterClass )
	{
		UBlueprint const* BPOwner = Cast<UBlueprint>( OuterClass->ClassGeneratedBy );
		if( BPOwner && BPOwner->BlueprintType == BPTYPE_LevelScript )
		{
			bIsFilteredOut = true;
			for( auto Blueprint : FilterContext.Blueprints )
			{
				if( BPOwner->GetBlueprintGuid() == Blueprint->GetBlueprintGuid() )
				{
					bIsFilteredOut = false;
					break;
				}
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsSchemaIncompatible(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	UClass const* NodeClass = BlueprintAction.GetNodeClass();
	UEdGraphNode const* NodeCDO = CastChecked<UEdGraphNode>(NodeClass->GetDefaultObject(false));
	checkSlow(NodeCDO != nullptr);


	
	if (FilterContext.Graphs.Num() > 0)
	{
		auto IsSchemaIncompatibleLambda = [NodeCDO](TArray<UEdGraph*> const& GraphList)->bool
		{
			bool bIsCompatible = true;
			for (UEdGraph const* Graph : GraphList)
			{
				if (!NodeCDO->CanCreateUnderSpecifiedSchema(Graph->GetSchema()))
				{
					bIsCompatible = false;
					break;
				}
			}
			return !bIsCompatible;
		};

		bIsFilteredOut = IsSchemaIncompatibleLambda(FilterContext.Graphs);
	}
	else
	{
		// When we are in a non-graph context, we may need to account for some graphs 
		// being incompatible. In this case the code to place a node will take care of
		// any issues, but we dont filter here if a schema is rejected, only if all schemas
		// are incompatible.
		auto AreAnySchemasCompatibleLambda = [NodeCDO](TArray<UEdGraph*> const& GraphList)->bool
		{
			bool bIsCompatible = false;
			for (UEdGraph const* Graph : GraphList)
			{
				if (NodeCDO->CanCreateUnderSpecifiedSchema(Graph->GetSchema()))
				{
					bIsCompatible = true;
					break;
				}
			}
			return bIsCompatible;
		};

		bIsFilteredOut = true;
		for (UBlueprint const* Blueprint : FilterContext.Blueprints)
		{
			TArray<UEdGraph*> BpGraphList;
			Blueprint->GetAllGraphs(BpGraphList);

			if (AreAnySchemasCompatibleLambda(BpGraphList))
			{
				bIsFilteredOut = false;
				break;
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::HasMatchingPin(FBlueprintActionInfo& BlueprintAction, UEdGraphPin const* Pin)
{
	bool bHasCompatiblePin = false;

	UEdGraph* OuterGraph = Pin->GetOwningNode()->GetGraph();
	if (UEdGraphNode* TemplateNode = BlueprintAction.NodeSpawner->GetTemplateNode(OuterGraph, BlueprintAction.GetBindings()))
	{
		if (TemplateNode->Pins.Num() == 0)
		{
			TemplateNode->AllocateDefaultPins();
		}

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(OuterGraph);
		check(Blueprint != nullptr);
		UEdGraphSchema const* Schema = OuterGraph->GetSchema();

		UClass const* CallingContext = GetAuthoritativeBlueprintClass(Blueprint);
		UK2Node* K2TemplateNode = Cast<UK2Node>(TemplateNode);
		UK2Node* OwningK2Node = Cast<UK2Node>(Pin->GetOwningNode());
		
		for (int32 PinIndex = 0; !bHasCompatiblePin && (PinIndex < TemplateNode->Pins.Num()); ++PinIndex)
		{
			UEdGraphPin* TemplatePin = TemplateNode->Pins[PinIndex];
			if (!Schema->ArePinsCompatible(Pin, TemplatePin, CallingContext))
			{
				continue;
			}
			bHasCompatiblePin = true;

			if (K2TemplateNode != nullptr)
			{
				FString DisallowedReason;
				// to catch wildcard connections that are prevented
				bHasCompatiblePin = !K2TemplateNode->IsConnectionDisallowed(TemplatePin, Pin, DisallowedReason)
					&& (!OwningK2Node || !OwningK2Node->IsConnectionDisallowed(Pin, TemplatePin, DisallowedReason));
			}
		}
	}	

	return bHasCompatiblePin;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsPinCompatibleWithTargetSelf(UEdGraphPin const* Pin, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsCompatible = false;
	UClass const* TargetClass = BlueprintAction.GetOwnerClass();
	if (BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_Message>())
	{
		// message nodes are a special case, they are intended to call a certain 
		// function, but will take any arbitrary object (and invokes the function
		// if that object implements the interface, otherwise the node is passed
		// through)
		TargetClass = UObject::StaticClass();
	}

	if ((Pin->Direction == EGPD_Output) && (TargetClass != nullptr))
	{
		FEdGraphPinType const& PinType = Pin->PinType;
		UEdGraphSchema const* const PinSchema = Pin->GetSchema();
		checkSlow(PinSchema != nullptr);

		UClass const* PinObjClass = nullptr;
		if (PinSchema->IsSelfPin(*Pin))
		{
			UBlueprint const* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Pin->GetOwningNode());
			PinObjClass = GetAuthoritativeBlueprintClass(Blueprint);
		}
		else if (PinType.PinSubCategoryObject.IsValid() && (PinType.PinCategory == UEdGraphSchema_K2::PC_Object || 
			PinType.PinCategory == UEdGraphSchema_K2::PC_Interface))
		{
			PinObjClass = Cast<UClass>(PinType.PinSubCategoryObject.Get());
		}
		
		if (PinObjClass != nullptr)
		{
			if (IsClassOfType(PinObjClass, TargetClass))
			{
				bIsCompatible = true;
				if (PinType.IsArray())
				{
					if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
					{
						bIsCompatible = UK2Node_CallFunction::CanFunctionSupportMultipleTargets(Function);
					}
					else
					{
						UEdGraph* OuterGraph = Pin->GetOwningNode()->GetGraph();
						if (UK2Node* TemplateNode = Cast<UK2Node>(BlueprintAction.NodeSpawner->GetTemplateNode(OuterGraph)))
						{
							bIsCompatible = TemplateNode->AllowMultipleSelfs(/*bInputAsArray =*/true);
						}	
					}
				}
			}
			else if (!PinType.IsArray() && (BlueprintAction.GetNodeClass() == UK2Node_CallFunction::StaticClass()))
			{
				// if this is a bound CallFunction action, then we make the 
				// assumption that it will be turned into a UK2Node_CallFunctionOnMember
				// node, therefore transforming the target from the function's
				// outer to the owner of the bound property
				IBlueprintNodeBinder::FBindingSet const& Bindings = BlueprintAction.GetBindings();
				// UK2Node_CallFunction is only transformed into a UK2Node_CallFunctionOnMember
				// when there is only one binding being applied ...
				if (Bindings.Num() == 1) 
				{
					FProperty* MemberBinding = Bindings.CreateConstIterator()->Get<FProperty>();

					if ((MemberBinding != nullptr) && BlueprintAction.NodeSpawner->IsBindingCompatible(MemberBinding))
					{
						const UClass* BindingOwner = MemberBinding->GetOwnerClass();
						bIsCompatible = IsClassOfType(PinObjClass, BindingOwner);
					}
				}
			}
		}
	}

	return bIsCompatible;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsFunctionMissingPinParam(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	if (UFunction const* AssociatedFunc = BlueprintAction.GetAssociatedFunction())
	{
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		bool const bIsEventSpawner  = BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_Event>();
		bool const bIsArrayFunction = BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_CallArrayFunction>();
		
		if (BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_PromotableOperator>())
		{
			return false;
		}

		for (int32 PinIndex = 0; !bIsFilteredOut && (PinIndex < Filter.Context.Pins.Num()); ++PinIndex)
		{
			UEdGraphPin const* ContextPin = Filter.Context.Pins[PinIndex];

			FEdGraphPinType const& PinType = ContextPin->PinType;
			UK2Node const* const K2Node = CastChecked<UK2Node const>(ContextPin->GetOwningNode());
			EEdGraphPinDirection const PinDir = ContextPin->Direction;

			if (K2Schema->IsExecPin(*ContextPin))
			{
				bIsFilteredOut = (bIsEventSpawner && (PinDir == EGPD_Output)) || !IsImpure(BlueprintAction);
			}
			else
			{
				// event nodes have their parameters as outputs (even though
				// the function signature would have them as inputs), so we 
				// want to flip the connotation here
				bool const bWantsOutputConnection = (PinDir == EGPD_Input) ^ bIsEventSpawner;
				
				// we don't support direct 'containers of containers, hence the !IsContainer() check here:
				if ( K2Schema->FunctionHasParamOfType(AssociatedFunc, K2Node->GetGraph(), PinType, bWantsOutputConnection) || 
					(bIsArrayFunction && ArrayFunctionHasParamOfType(AssociatedFunc, K2Node->GetGraph(), PinType, bWantsOutputConnection) && !PinType.IsContainer()) )
				{
					bIsFilteredOut = false;
				}
				else
				{
					// need to take "Target" self pins into consideration for objects
					bIsFilteredOut = bIsEventSpawner || !IsPinCompatibleWithTargetSelf(ContextPin, BlueprintAction);
				}
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::ArrayFunctionHasParamOfType(const UFunction* ArrayFunction, UEdGraph const* InGraph, const FEdGraphPinType& DesiredPinType, bool bWantOutput)
{
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

	TSet<FName> HiddenPins;
	FBlueprintEditorUtils::GetHiddenPinsForFunction(InGraph, ArrayFunction, HiddenPins);

	FName ParamTag = FBlueprintMetadata::MD_ArrayDependentParam;
	if (DesiredPinType.IsArray())
	{
		ParamTag = FBlueprintMetadata::MD_ArrayParam;
	}
	const FString& FlaggedParamMetaData = ArrayFunction->GetMetaData(ParamTag);

	TArray<FString> WildcardPinNameStrs;
	TArray<FName> WildcardPinNames;
	FlaggedParamMetaData.ParseIntoArray(WildcardPinNameStrs, TEXT(","), /*CullEmpty =*/true);
	Algo::Transform(WildcardPinNameStrs, WildcardPinNames, [](const FString& NameStr) { return FName(*NameStr); });

	for (TFieldIterator<FProperty> PropIt(ArrayFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* FuncParam = *PropIt;

		const bool bIsFunctionInput = !FuncParam->HasAnyPropertyFlags(CPF_OutParm) || FuncParam->HasAnyPropertyFlags(CPF_ReferenceParm);
		if (bWantOutput == bIsFunctionInput)
		{
			continue;
		}

		const FName ParamName = FuncParam->GetFName();

		if (HiddenPins.Contains(ParamName) || !WildcardPinNames.Contains(ParamName))
		{
			continue;
		}

		FEdGraphPinType ParamPinType;
		if (K2Schema->ConvertPropertyToPinType(FuncParam, ParamPinType))
		{
			ParamPinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;

			const FEdGraphPinType& InputType  = (bIsFunctionInput) ? ParamPinType : DesiredPinType;
			const FEdGraphPinType& OutputType = (bIsFunctionInput) ? DesiredPinType : ParamPinType;

			if (K2Schema->ArePinTypesCompatible(OutputType, InputType))
			{
				return true;
			}
		}
	}

	return false;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsMissmatchedPropertyType(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;
	if (FProperty const* Property = BlueprintAction.GetAssociatedProperty())
	{
		TArray<UEdGraphPin*> const& ContextPins = Filter.Context.Pins;
		if (ContextPins.Num() > 0)
		{
			bool const bIsDelegate = Property->IsA<FMulticastDelegateProperty>();
			bool const bIsGetter   = BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_VariableGet>();
			bool const bIsSetter   = BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_VariableSet>();

			for (int32 PinIndex = 0; !bIsFilteredOut && PinIndex < ContextPins.Num(); ++PinIndex)
			{
				UEdGraphPin const* ContextPin = ContextPins[PinIndex];
				FEdGraphPinType const& ContextPinType = ContextPin->PinType;
				UEdGraphSchema_K2 const* K2Schema = CastChecked<UEdGraphSchema_K2 const>(ContextPin->GetSchema());

				// have to account for "self" context pin
				if (IsPinCompatibleWithTargetSelf(ContextPin, BlueprintAction))
				{
					continue;
				}
				else if (bIsDelegate)
				{
					// there are a lot of different delegate nodes, so let's  
					// just iterate over all the pins
					bIsFilteredOut = !HasMatchingPin(BlueprintAction, ContextPin);
				}
				else if (ContextPinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
				{
					// setters are impure, and therefore should have exec pins
					bIsFilteredOut = bIsGetter;
				}
				else if (bIsGetter || bIsSetter)
				{
					bIsFilteredOut = true;

					EEdGraphPinDirection const PinDir = ContextPin->Direction;
					if ((PinDir == EGPD_Input) && bIsGetter)
					{
						FEdGraphPinType OutputPinType;
						K2Schema->ConvertPropertyToPinType(Property, OutputPinType);
						bIsFilteredOut = !K2Schema->ArePinTypesCompatible(OutputPinType, ContextPinType);
					}
					else if ((PinDir == EGPD_Output) && bIsSetter)
					{
						FEdGraphPinType InputPinType;
						K2Schema->ConvertPropertyToPinType(Property, InputPinType);
						bIsFilteredOut = !K2Schema->ArePinTypesCompatible(ContextPinType, InputPinType);
					}	
				}
				else
				{
					ensureMsgf(false, TEXT("Unhandled property/node pair, we've probably made some bad assuptions."));
				}
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsMissingMatchingPinParam(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	// we have a separate pin tests for function/property nodes (IsFunctionMissingPinParam/IsMissmatchedPropertyType). Note that we only skip
	// this test for functions with bindings (because it does not handle getting templates for all binding nodes). By running this for 
	// other functions we ensure that IsConnectionDisallowed is honored.
	bool const bTestPinCompatibility = (BlueprintAction.GetAssociatedProperty() == nullptr) && BlueprintAction.GetBindings().Num() == 0 && !BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_PromotableOperator>();

	if (bTestPinCompatibility)
	{
		for (UEdGraphPin const* ContextPin : Filter.Context.Pins)
		{
			if (!HasMatchingPin(BlueprintAction, ContextPin))
			{
				bIsFilteredOut = true;
				break;
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsNotSubClassCast(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	if (BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_DynamicCast>())
	{
		for(UEdGraphPin const* ContextPin : Filter.Context.Pins)
		{
			// Only worry about removing cast nodes when dragging off output pins
			if (ContextPin->Direction != EGPD_Output)
			{
				break;
			}
			
			if (ContextPin->PinType.PinSubCategoryObject.IsValid())
			{
				UClass const* CastClass = BlueprintAction.GetOwnerClass();
				checkSlow(CastClass != nullptr);

				UClass const* ContextPinClass = Cast<UClass>(ContextPin->PinType.PinSubCategoryObject.Get());
				// could be a delegate (or some other) pin, with a non-class
				// PinSubCategoryObject
				if (ContextPinClass == nullptr)
				{
					continue;
				}

				// Interface casts are always enabled
				if (CastClass->IsChildOf<UInterface>())
				{
					continue;
				}

				if ((ContextPinClass == CastClass) || !IsClassOfType(CastClass, ContextPinClass))
				{
					bIsFilteredOut = true;
					break;
				}
			}
		}
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsNodeTemplateSelfFiltered(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	return BlueprintAction.NodeSpawner->IsTemplateNodeFilteredOut(Filter);
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsIncompatibleAnimNotification(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	if ( BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_Event>() )
	{
		if (UAnimNotifyEventNodeSpawner const* const AnimNotifyEventNodeSpawner = Cast<UAnimNotifyEventNodeSpawner>(BlueprintAction.NodeSpawner))
		{
			// The event is owned by a skeleton. Only show if it the current anim blueprint is targetting
			// that skeleton:
			const FSoftObjectPath& SkeletonObjectPath = AnimNotifyEventNodeSpawner->GetSkeletonObjectPath();
			FBlueprintActionContext const& FilterContext = Filter.Context;
			bool bFoundInAllBlueprints = true;
			const bool bUseSkeletonCheck = Filter.TargetClasses.Num() != 0;	// If we are 'context sensitive' then we will have classes here

			for (const UBlueprint* Blueprint : FilterContext.Blueprints)
			{
				bool bFoundInCurrentBlueprint = false;
				if (const UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Blueprint))
				{
					if( !bUseSkeletonCheck || FSoftObjectPath(AnimBlueprint->TargetSkeleton) == SkeletonObjectPath )
					{
						bFoundInCurrentBlueprint = true;
						break;
					}
				}
				bFoundInAllBlueprints &= bFoundInCurrentBlueprint;
			}
			// If all of the blueprints selected aren't animblueprints targetting this skeleton then we need to 
			// filter it out:
			bIsFilteredOut = !bFoundInAllBlueprints; 
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsExtraneousInterfaceCall(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	const UClass* NodeClass = BlueprintAction.GetNodeClass();
	if (NodeClass->IsChildOf<UK2Node_Message>())
	{
		UFunction const* Function = BlueprintAction.GetAssociatedFunction();
		checkSlow(Function != nullptr);

		UClass* InterfaceClass = Function->GetOwnerClass();
		checkSlow(InterfaceClass->IsChildOf<UInterface>());

		bool const bCanBeAddedToBlueprints = FKismetEditorUtilities::IsClassABlueprintImplementableInterface(InterfaceClass);

		bIsFilteredOut = (Filter.TargetClasses.Num() > 0);
		for (const auto& ClassData : Filter.TargetClasses)
		{
			bool const bImplementsInterface = IsClassOfType(ClassData.TargetClass, InterfaceClass);
			// if this is a blueprint class, and "CannotImplementInterfaceInBlueprint" 
			// is set on the interface, then we know sub-classes cannot have
			// the interface either (so there's no point to offering a message node)
			bool const bIsBlueprintClass = (Cast<UBlueprintGeneratedClass>(ClassData.TargetClass) != nullptr);

			// if the class doesn't directly implement the interface (and it is 
			// a possibility that some sub-class does), then we want to offer 
			// the message node (in case the Target object is actually an 
			// instance of a sub-class)
			if (!bImplementsInterface && (!bIsBlueprintClass || bCanBeAddedToBlueprints))
			{
				bIsFilteredOut = false;
				break;
			}
		}
	}
	else if (UFunction const* Function = BlueprintAction.GetAssociatedFunction())
	{
		UClass* FuncClass = Function->GetOwnerClass();
		bool const bIsInterfaceAction = FuncClass->IsChildOf<UInterface>();

		if (bIsInterfaceAction && !NodeClass->IsChildOf<UK2Node_Event>())
		{
			bIsFilteredOut = (Filter.TargetClasses.Num() > 0);
			for (const auto& ClassData : Filter.TargetClasses)
			{
				UClass const* InterfaceImplementingClass = FindInheritedInterfaceClass(ClassData.TargetClass, FuncClass);
				// interfaces that are added directly to a Blueprint (even in 
				// the case of an interface on a parent blueprint) have their 
				// functions stubbed-out/added to the blueprint class directly;
				// in that case, we want to favor a call to the blueprint 
				// version (not this interface call) because we can circumvent 
				// the extra work converting from a interface to an object.
				//
				// however, if the interface belongs to a native class, then the
				// blueprint doesn't get those extra functions, so this is is 
				// our only way of calling the interface methods.
				bool const bImplementedByBlueprint = (InterfaceImplementingClass != nullptr) && (Cast<UBlueprint>(InterfaceImplementingClass->ClassGeneratedBy) != nullptr);
				if (!bImplementedByBlueprint)
				{
					bIsFilteredOut = false;
					break;
				}
			}
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsIncompatibleMacroInstance(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool bIsFilteredOut = false;

	const bool bRejectNonImportedMacros = Filter.HasAllFlags(FBlueprintActionFilter::BPFILTER_RejectNonImportedFields);

	if(BlueprintAction.GetNodeClass()->IsChildOf<UK2Node_MacroInstance>())
	{
		if(const UBlueprint* MacroBP = Cast<const UBlueprint>(BlueprintAction.GetActionOwner()))
		{
			if (bRejectNonImportedMacros && IsNonImportedObject(Filter, MacroBP))
			{
				return true;
			}

			if (!ensure(MacroBP->ParentClass != nullptr))
			{
				return true;
			}

			for(auto BlueprintIt = Filter.Context.Blueprints.CreateConstIterator(); BlueprintIt && !bIsFilteredOut; ++BlueprintIt)
			{
				const UBlueprint* Blueprint = *BlueprintIt;
				if (!ensure(Blueprint != nullptr && Blueprint->ParentClass != nullptr))
				{
					return true;
				}

				bIsFilteredOut = (Blueprint != MacroBP) && (MacroBP->BlueprintType != BPTYPE_MacroLibrary || !Blueprint->ParentClass || !Blueprint->ParentClass->IsChildOf(MacroBP->ParentClass));
			}

			// Note: The rest is handled by IsNodeTemplateSelfFiltered() - the check above is a "fast path" in that we don't have to instance the node template (see UK2Node_MacroInstance::IsActionFilteredOut())
		}
	}

	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsStaleFieldAction(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	bool const bIsFilteredOut = FBlueprintNodeSpawnerUtils::IsStaleFieldAction(BlueprintAction.NodeSpawner);
	ensureMsgf(!bIsFilteredOut, TEXT("Invalid BlueprintActionDatabase entry (for %s). Was the database properly updated when this class was compiled?"), *BlueprintAction.GetOwnerClass()->GetName());
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
static bool BlueprintActionFilterImpl::IsHiddenInNonEditorBlueprint(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& BlueprintAction)
{
	const UFunction* Function = BlueprintAction.GetAssociatedFunction();

	bool bVisible = true;

	if (Function)
	{
		 // Leaving this check here (even though its done in CanEditorOnlyFunctionBeCalled) as an early exit
		const bool bIsEditorOnlyFunction = IsEditorOnlyObject(Function) || Function->HasAnyFunctionFlags(FUNC_EditorOnly);
		const bool bIsUncookedOnlyFunction = Function->GetOutermost()->HasAnyPackageFlags(PKG_UncookedOnly);

		if (bIsEditorOnlyFunction)
		{
			for (const UBlueprint* Blueprint : Filter.Context.Blueprints)
			{
				bVisible &= UK2Node_CallFunction::CanEditorOnlyFunctionBeCalled(Function, Blueprint->ParentClass);
			}
		}

		if (bIsUncookedOnlyFunction)
		{
			for (const UBlueprint* Blueprint : Filter.Context.Blueprints)
			{
				const UClass* BlueprintClass = Blueprint->ParentClass;
				const bool bIsEditorBlueprintClass = (BlueprintClass != nullptr) && IsEditorOnlyObject(BlueprintClass);
				const bool bIsUncookedBlueprintClass = (BlueprintClass != nullptr) && BlueprintClass->GetOutermost()->HasAnyPackageFlags(PKG_UncookedOnly);
				bVisible &= (bIsEditorBlueprintClass || bIsUncookedBlueprintClass);
			}
		}
	}
	
	return !bVisible;
}

/*******************************************************************************
 * FBlueprintActionInfo
 ******************************************************************************/

/** */
namespace EBlueprintActionInfoFlags
{
	enum 
	{
		CachedClass    = (1 << 0),
		CachedField    = (1 << 1),
		CachedProperty = (1 << 2),
		CachedFunction = (1 << 3),
	};
}

//------------------------------------------------------------------------------
FBlueprintActionInfo::FBlueprintActionInfo(UObject const* ActionOwnerIn, UBlueprintNodeSpawner const* Action)
	: NodeSpawner(Action)
	, ActionOwner(ActionOwnerIn)
	, CacheFlags(0)
	, CachedOwnerClass(nullptr)
	, CachedActionField((UObject*)nullptr)
	, CachedActionProperty(nullptr)
	, CachedActionFunction(nullptr)
{
	checkSlow(NodeSpawner != nullptr);
}

//------------------------------------------------------------------------------
FBlueprintActionInfo::FBlueprintActionInfo(FBlueprintActionInfo const& Rhs, IBlueprintNodeBinder::FBindingSet const& InBindings)
	: NodeSpawner(Rhs.NodeSpawner)
	, ActionOwner(Rhs.ActionOwner)
	, CacheFlags(Rhs.CacheFlags)
	, CachedOwnerClass(Rhs.CachedOwnerClass)
	, CachedActionField(Rhs.CachedActionField)
	, CachedActionProperty(Rhs.CachedActionProperty)
	, CachedActionFunction(Rhs.CachedActionFunction)
	, Bindings(InBindings)
{
	checkSlow(NodeSpawner != nullptr);
}

//------------------------------------------------------------------------------
UObject const* FBlueprintActionInfo::GetActionOwner() const
{
	return ActionOwner.Get();
}

//------------------------------------------------------------------------------
IBlueprintNodeBinder::FBindingSet const& FBlueprintActionInfo::GetBindings() const
{
	return Bindings;
}

//------------------------------------------------------------------------------
UClass const* FBlueprintActionInfo::GetOwnerClass()
{
	if ((CacheFlags & EBlueprintActionInfoFlags::CachedClass) == 0)
	{
		CachedOwnerClass = Cast<UClass>(ActionOwner.Get());
		if (CachedOwnerClass == GetNodeClass())
		{
			CachedOwnerClass = nullptr;
		}
		else if (const UBlueprint* AsBlueprint = Cast<UBlueprint>(ActionOwner.Get()))
		{
			CachedOwnerClass = AsBlueprint->SkeletonGeneratedClass.Get();
		}

		if (CachedOwnerClass == nullptr)
		{
			FFieldVariant AssociatedMemberField = GetAssociatedMemberField();
			if (AssociatedMemberField.IsValid())
			{
				CachedOwnerClass = AssociatedMemberField.GetOwnerClass();
			}
		}

		CacheFlags |= EBlueprintActionInfoFlags::CachedClass;
	}
	return CachedOwnerClass.Get();
}

//------------------------------------------------------------------------------
UClass const* FBlueprintActionInfo::GetNodeClass()
{
	UClass const* NodeClass = NodeSpawner->NodeClass;
	checkSlow(NodeClass != nullptr);
	return NodeClass;
}

//------------------------------------------------------------------------------
FFieldVariant FBlueprintActionInfo::GetAssociatedMemberField()
{
	if ((CacheFlags & EBlueprintActionInfoFlags::CachedField) == 0)
	{
		if (const UFunction* AssociatedFunction = FBlueprintNodeSpawnerUtils::GetAssociatedFunction(NodeSpawner))
		{
			CachedActionField = const_cast<UFunction*>(AssociatedFunction);
		}
		else if (const FProperty* AssociatedProperty = FBlueprintNodeSpawnerUtils::GetAssociatedProperty(NodeSpawner))
		{
			CachedActionField = const_cast<FProperty*>(AssociatedProperty);
		}
		CacheFlags |= EBlueprintActionInfoFlags::CachedField;
	}
	return CachedActionField;
}

//------------------------------------------------------------------------------
FProperty const* FBlueprintActionInfo::GetAssociatedProperty()
{
	if ((CacheFlags & EBlueprintActionInfoFlags::CachedProperty) == 0)
	{
		if (CacheFlags & EBlueprintActionInfoFlags::CachedField)
		{
			CachedActionProperty = CastField<FProperty>(CachedActionField.ToField());
		}
		else
		{
			CachedActionProperty = FBlueprintNodeSpawnerUtils::GetAssociatedProperty(NodeSpawner);
			if (CachedActionProperty != nullptr)
			{
				CachedActionField = const_cast<FProperty*>(CachedActionProperty);
				CacheFlags |= EBlueprintActionInfoFlags::CachedProperty;
			}
		}
	}
	return CachedActionProperty;
}

//------------------------------------------------------------------------------
UFunction const* FBlueprintActionInfo::GetAssociatedFunction()
{
	if ((CacheFlags & EBlueprintActionInfoFlags::CachedFunction) == 0)
	{
		if (CacheFlags & EBlueprintActionInfoFlags::CachedField)
		{
			CachedActionFunction = Cast<UFunction>(CachedActionField.ToUObject());
		}
		else
		{
			CachedActionFunction = FBlueprintNodeSpawnerUtils::GetAssociatedFunction(NodeSpawner);
			if (CachedActionFunction != nullptr)
			{
				CachedActionField = const_cast<UFunction*>(CachedActionFunction);
				CacheFlags |= EBlueprintActionInfoFlags::CachedProperty;
			}
		}
		CacheFlags |= EBlueprintActionInfoFlags::CachedFunction;
	}
	return CachedActionFunction;
}

/*******************************************************************************
 * FActionFilterTest
 ******************************************************************************/

TAutoConsoleVariable<bool> FActionFilterTest::CVarEnableCaching{
		TEXT("BP.EnableActionMenuFilterCaching"),
		false,
		TEXT("If enabled, action filter tests with the CacheResults flag set will have their results cached")
	};

FActionFilterTest::FActionFilterTest(const FRejectionDelegate Delegate, FString Name, EActionFilterTestFlags Flags) :
	RejectionDelegate(Delegate),
	Name(Name),
	Flags(Flags)
{
}

bool FActionFilterTest::Call(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& ActionInfo)
{
	check(RejectionDelegate.IsBound());

	return RejectionDelegate.Execute(Filter, ActionInfo);
}

bool FActionFilterTest::operator==(const FActionFilterTest& other) const
{
	return Name == other.Name && Flags == other.Flags;
}

/*******************************************************************************
 * FBlueprintActionFilter
 ******************************************************************************/

FBlueprintActionFilter::FBlueprintActionFilter(const EFlags InFlags /*= BPFILTER_NoFlags*/)
	: FilterFlags(InFlags)
{
	using namespace BlueprintActionFilterImpl;

	// Disable non-imported type filtering if namespace filtering features are disabled.
	if (!GetDefault<UBlueprintEditorSettings>()->bEnableNamespaceFilteringFeatures)
	{
		FilterFlags &= ~BPFILTER_RejectNonImportedFields;
	}

	BluprintGraphModule = &FModuleManager::LoadModuleChecked<FBlueprintGraphModule>("BlueprintGraph");

	//
	// NOTE: The order of these tests can have perf implications, the more one
	//       rejects on average the later it should be added (they're executed
	//       in reverse order, so user added tests are ran first and the ones 
	//       here are ran last)
	//

	// add first the most expensive tests (they will be ran last, and therefore
	// should be operating on a smaller subset of node-spawners)
	//
	// this test in-particular spawns a template-node and then calls 
	// AllocateDefaultPins() which is costly, so it should be very last!
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsNodeTemplateSelfFiltered));

	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsIncompatibleAnimNotification));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsAssetPermissionNotGranted));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsMissingMatchingPinParam));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsMissmatchedPropertyType));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsFunctionMissingPinParam));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsIncompatibleLatentNode));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsIncompatibleImpureNode));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsPropertyAccessorNode));

	if (HasAnyFlags(BPFILTER_RejectIncompatibleThreadSafety))
	{
		AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsThreadSafetyIncompatible));
	}

	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsActionHiddenByConfig));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsFieldCategoryHidden));

	if (HasAnyFlags(BPFILTER_RejectGlobalFields | BPFILTER_RejectNonImportedFields))
	{
		AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsRejectedGlobalField));
	}

	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsFieldInaccessible));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsNotSubClassCast));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsEventUnimplementable));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsPermissionNotGranted));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsRestrictedClassMember));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsIncompatibleWithGraphType));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsSchemaIncompatible));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsExtraneousInterfaceCall));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsIncompatibleMacroInstance));

	if (!HasAnyFlags(BPFILTER_PermitDeprecated))
	{
		AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsDeprecated));
	}

	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsFilteredNodeType));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsNonTargetMember));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsUnBoundBindingSpawner));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsOutOfScopeLocalVariable));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsLevelScriptActionValid));
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsHiddenInNonEditorBlueprint));

	// added as the first rejection test, so that we don't operate on stale 
	// (TRASH/REINST) class fields
	AddRejectionTest(FRejectionTestDelegate::CreateStatic(IsStaleFieldAction));
}

//------------------------------------------------------------------------------
void FBlueprintActionFilter::AddUnique(TArray<FTargetClassFilterData>& ToArray, UClass* TargetClass)
{
	for (const FTargetClassFilterData& ClassData : ToArray)
	{
		if (ClassData.TargetClass == TargetClass)
		{
			return;
		}
	}
	FBlueprintActionFilter::Add(ToArray, TargetClass);
}

//------------------------------------------------------------------------------
void FBlueprintActionFilter::Add(TArray<FTargetClassFilterData>& ToArray, UClass* TargetClass)
{
	TArray<FString> ClassHideCategories;
	FEditorCategoryUtils::GetClassHideCategories(TargetClass, ClassHideCategories);
	ClassHideCategories.Sort();
	ClassHideCategories.SetNum(Algo::Unique(ClassHideCategories));
	FTargetClassFilterData Data = { TargetClass, MoveTemp(ClassHideCategories) };
	ToArray.Add(MoveTemp(Data));
}

//------------------------------------------------------------------------------
void FBlueprintActionFilter::AddRejectionTest(FRejectionTestDelegate RejectionTestDelegate)
{
	if (ensureMsgf(RejectionTestDelegate.IsBound(), TEXT("Cannot add a rejection test without a bound delegate")))
	{
		FilterTests.Add(RejectionTestDelegate);

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
		FFilterTestProfileRecord FilterTestProfileRecord;
		FilterTestProfileRecord.TestFuncPtr = RejectionTestDelegate.GetBoundProgramCounterForTimerManager();

		FilterTestProfiles.Add(FilterTests.Num() - 1, MoveTemp(FilterTestProfileRecord));
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	}
}

//------------------------------------------------------------------------------
void FBlueprintActionFilter::AddRejectionTest(TSharedRef<FActionFilterTest> RejectionTest)
{
	AddRejectionTest(RejectionTest->RejectionDelegate);
}

//------------------------------------------------------------------------------
bool FBlueprintActionFilter::IsFiltered(FBlueprintActionInfo& BlueprintAction)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionFilter::IsFiltered);

	bool bIsFiltered = IsFilteredByThis(BlueprintAction);
	if (!bIsFiltered)
	{
		for (FBlueprintActionFilter& OrFilter : OrFilters)
		{
			if (OrFilter.IsFiltered(BlueprintAction))
			{
				bIsFiltered = true;
				break;
			}
		}
	}

	if (bIsFiltered)
	{
		for (FBlueprintActionFilter& AndFilter : AndFilters)
		{
			bIsFiltered &= AndFilter.IsFiltered(BlueprintAction);
		}
	}

	return bIsFiltered;
}

//------------------------------------------------------------------------------
FBlueprintActionFilter const& FBlueprintActionFilter::operator|=(FBlueprintActionFilter const& Rhs)
{
	OrFilters.Add(Rhs);
	return *this;
}

//------------------------------------------------------------------------------
FBlueprintActionFilter const& FBlueprintActionFilter::operator&=(FBlueprintActionFilter const& Rhs)
{
	AndFilters.Add(Rhs);
	return *this;
}

//------------------------------------------------------------------------------
bool FBlueprintActionFilter::IsFilteredByThis(FBlueprintActionInfo& BlueprintAction) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintActionFilter::IsFilteredByThis);

	FBlueprintActionFilter const& FilterRef = *this;

	// for debugging purposes:
// 	FBlueprintActionUiSpec UiSpec = BlueprintAction.NodeSpawner->GetUiSpec(Context, BlueprintAction.GetBindings());
// 	bool bDebugBreak = false;
// 	if (UiSpec.MenuName.ToString().Contains("Print String"))
// 	{
// 		bDebugBreak = true;
// 	}

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	const bool bIsFilterTestProfilingEnabled = IsFilterTestTraceLoggingEnabled() || IsFilterTestStatsLoggingEnabled();
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

	bool bIsFiltered = false;
	// iterate backwards so that custom user test are ran first (and the slow
	// internal tests are ran last).
	for (int32 TestIndex = FilterTests.Num() - 1; TestIndex >= 0; --TestIndex)
	{
		FRejectionTestDelegate const& RejectionTestDelegate = FilterTests[TestIndex];
		checkSlow(RejectionTestDelegate.IsBound());

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
		bool bTraceEventEmitted = false;
		double StartTime = 0;
		FFilterTestProfileRecord* FilterTestProfileRecord = nullptr;
		if (bIsFilterTestProfilingEnabled)
		{
			if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
			{
				FCpuProfilerTrace::OutputBeginDynamicEvent(BlueprintActionFilterImpl::FilterTestProfileEventName);
				bTraceEventEmitted = true;
			}
			StartTime = FPlatformTime::Seconds();
			FilterTestProfileRecord = BeginFilterTestProfileEvent(TestIndex, RejectionTestDelegate.GetBoundProgramCounterForTimerManager());
		}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

		bIsFiltered = RejectionTestDelegate.Execute(FilterRef, BlueprintAction);

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
		if (FilterTestProfileRecord)
		{
			EndFilterTestProfileEvent(FilterTestProfileRecord, bIsFiltered, StartTime);
			if (bTraceEventEmitted)
			{
				FCpuProfilerTrace::OutputEndEvent();
			}
		}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

		if (bIsFiltered)
		{
			break;
		}
	}

	if (!bIsFiltered)
	{
#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
		int32 ExtraTestProfileRecordIndex = FilterTests.Num();
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

		for (const FBlueprintGraphModule::FActionMenuRejectionTest& ExtraRejectionTest : BluprintGraphModule->GetExtendedActionMenuFilters())
		{
#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
			bool bTraceEventEmitted = false;
			double StartTime = 0;
			FFilterTestProfileRecord* FilterTestProfileRecord = nullptr;
			if (bIsFilterTestProfilingEnabled)
			{
				if (UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
				{
					FCpuProfilerTrace::OutputBeginDynamicEvent(BlueprintActionFilterImpl::FilterTestProfileEventName);
					bTraceEventEmitted = true;
				}
				StartTime = FPlatformTime::Seconds();
				FilterTestProfileRecord = BeginFilterTestProfileEvent(ExtraTestProfileRecordIndex++, ExtraRejectionTest.GetBoundProgramCounterForTimerManager());
			}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

			bIsFiltered = ExtraRejectionTest.Execute(FilterRef, BlueprintAction);

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
			if (FilterTestProfileRecord)
			{
				EndFilterTestProfileEvent(FilterTestProfileRecord, bIsFiltered, StartTime);
				if (bTraceEventEmitted)
				{
					FCpuProfilerTrace::OutputEndEvent();
				}
			}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

			if (bIsFiltered)
			{
				break;
			}
		}
	}

	return bIsFiltered;
}

//------------------------------------------------------------------------------
#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
bool FBlueprintActionFilter::IsFilterTestTraceLoggingEnabled()
{
	return BlueprintActionFilterImpl::CVarBPEnableActionMenuFilterTestTraceLogging.GetValueOnGameThread();
}

bool FBlueprintActionFilter::IsFilterTestStatsLoggingEnabled()
{
	return BlueprintActionFilterImpl::CVarBPEnableActionMenuFilterTestStatsLogging.GetValueOnGameThread();
}

TArray<FString> FBlueprintActionFilter::GetFilterTestProfile()
{
	TArray<FString> Result;

	auto AddFilterTestProfile = [&Result](int32 DisplayIndex, const FFilterTestProfileRecord& FilterTestProfile)
	{
		const FString& FilterTestFunctionName = BlueprintActionFilterImpl::FilterTestFuncPtrToFuncName(FilterTestProfile.TestFuncPtr);
		Result.Add(FString::Printf(TEXT("[%02d]: %s | %d tested | %d failed | %.02f ms | %.02f us (per test)"),
			DisplayIndex,
			*FilterTestFunctionName,
			FilterTestProfile.NumIterations,
			FilterTestProfile.NumFilteredOut,
			FilterTestProfile.TotalTimeMs,
			FilterTestProfile.NumIterations > 0 ? FilterTestProfile.TotalTimeMs / FilterTestProfile.NumIterations * 1000.0f : 0.0f)
		);
	};

	for (int32 FilterTestIndex = FilterTests.Num() - 1; FilterTestIndex >= 0; --FilterTestIndex)
	{
		AddFilterTestProfile(FilterTests.Num() - FilterTestIndex, FilterTestProfiles[FilterTestIndex]);
	}

	for (int32 ExtendedFilterTestIndex = FilterTests.Num(); ExtendedFilterTestIndex < FilterTests.Num() + BluprintGraphModule->GetExtendedActionMenuFilters().Num(); ++ExtendedFilterTestIndex)
	{
		AddFilterTestProfile(ExtendedFilterTestIndex, FilterTestProfiles[ExtendedFilterTestIndex]);
	}

	return Result;
}

FBlueprintActionFilter::FFilterTestProfileRecord* FBlueprintActionFilter::BeginFilterTestProfileEvent(int32 FilterTestIndex, uint64 FilterTestFuncPtr) const
{
	FFilterTestProfileRecord* FilterTestProfileRecord = FilterTestProfiles.Find(FilterTestIndex);
	if (!FilterTestProfileRecord)
	{
		FilterTestProfileRecord = &FilterTestProfiles.Add(FilterTestIndex);
		FilterTestProfileRecord->TestFuncPtr = FilterTestFuncPtr;
	}

	if (IsFilterTestTraceLoggingEnabled() && UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
	{
		FCpuProfilerTrace::OutputBeginEvent(BlueprintActionFilterImpl::FilterTestFuncPtrToCpuSpecId(FilterTestFuncPtr));
		check(FilterTestProfileRecord->bOutputBeginEventEmitted == false);
		FilterTestProfileRecord->bOutputBeginEventEmitted = true;
	}

	return FilterTestProfileRecord;
}

void FBlueprintActionFilter::EndFilterTestProfileEvent(FFilterTestProfileRecord* InEvent, bool bWasFiltered, double StartTime) const
{
	if (InEvent)
	{
		if (InEvent->bOutputBeginEventEmitted)
		{
			FCpuProfilerTrace::OutputEndEvent();
			InEvent->bOutputBeginEventEmitted = false;
		}

		if (IsFilterTestStatsLoggingEnabled())
		{
			InEvent->NumIterations += 1;
			InEvent->TotalTimeMs += (FPlatformTime::Seconds() - StartTime) * 1000.0f;
			if (bWasFiltered)
			{
				InEvent->NumFilteredOut += 1;
			}
		}
	}
}
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
