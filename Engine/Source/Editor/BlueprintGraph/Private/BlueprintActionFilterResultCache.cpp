// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintActionFilterResultCache.h"
#include "Concepts/EqualityComparable.h"
#include "EdGraph/EdGraphPin.h"

/*******************************************************************************
 * FActionFilterCacheKey
 ******************************************************************************/

FActionFilterCacheKey::FActionFilterCacheKey(const FBlueprintActionInfo& InAction)
{
	KeyHash = GetTypeHash(InAction.NodeSpawner);
	KeyHash = HashCombine(KeyHash, GetTypeHash(InAction.GetActionOwner()));
	KeyHash = HashCombine(KeyHash, InAction.GetBindings().Num());
}

bool operator==(const FActionFilterCacheKey& A, const FActionFilterCacheKey& B)
{
	return A.KeyHash == B.KeyHash;
}

uint32 GetTypeHash(const FActionFilterCacheKey& InKey)
{
	return InKey.KeyHash;
}

/*******************************************************************************
 * TActionFilterCacheKeyNode
 ******************************************************************************/

template<typename TKeyType>
class TActionFilterCacheKeyNode : public FActionFilterCacheNode
{
public:
	explicit TActionFilterCacheKeyNode(const TKeyType& Key, TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr) :
		Key(MakeShared<TKeyType>(Key)),
		Comparison(Comparison)
	{}

	virtual ~TActionFilterCacheKeyNode() override = default;

	virtual void UnCacheSelf() override
	{
		if (Parent.IsValid())
		{
			const FSharedNode SharedParent = Parent.Pin();

			// remove this node from parent
			SharedParent->Children.RemoveSingleSwap(this->AsShared(), EAllowShrinking::No);

			if (SharedParent->Children.IsEmpty())
			{
				SharedParent->UnCacheSelf();
			}
		}
	}

	TSharedPtr<TKeyType> Key = nullptr;
protected:
	virtual bool KeysMatch(const void* RawKey) const override
	{
		if (Comparison.IsBound())
		{
			return Comparison.Execute(*Key, *StaticCast<const TKeyType*>(RawKey));
		}
		else
		{
			return *Key == *StaticCast<const TKeyType*>(RawKey);
		}
	}

	TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr;
};

/*******************************************************************************
 * TActionFilterCacheLeaf
 ******************************************************************************/

template<typename TKeyType>
class TActionFilterCacheLeaf : public TActionFilterCacheKeyNode<TKeyType>
{
public:
	explicit TActionFilterCacheLeaf(const TKeyType& Key, TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr) :
		TActionFilterCacheKeyNode<TKeyType>(Key, Comparison),
		CachedFilterResults(MakeShared<TMap<FActionFilterCacheKey, bool>>())
	{}

	virtual ~TActionFilterCacheLeaf() override = default;

	virtual uint32 GetLeafAccessCount() const override
	{
		return AccessCount;
	}


	virtual void DecrementLeafAccessCount() override
	{
		if (AccessCount > 0)
		{
			--AccessCount;
		}
	}

	uint32 AccessCount = 1;
	TSharedPtr<TMap<FActionFilterCacheKey, bool>> CachedFilterResults;
};

/*******************************************************************************
 * FActionFilterCache
 ******************************************************************************/

FActionFilterCache::FActionFilterCache()
{
	if (!ReloadCompleteDelegate.IsValid())
	{
		ReloadCompleteDelegate = FCoreUObjectDelegates::ReloadCompleteDelegate.AddStatic(&FActionFilterCache::OnReloadComplete);
	}
	if (!ModuleChangedDelegate.IsValid())
	{
		ModuleChangedDelegate = FModuleManager::Get().OnModulesChanged().AddStatic(&FActionFilterCache::OnModuleChanged);
	}
}

void FActionFilterCache::OnReloadComplete(EReloadCompleteReason Reason)
{
	ClearAllCache();
}

void FActionFilterCache::OnModuleChanged(FName ModuleName, EModuleChangeReason ChangeReason)
{
	ClearAllCache();
}

void FActionFilterCache::ClearAllCache()
{
	SharedCache = MakeShared<FActionFilterCacheNode>();
	CacheLeafs.Empty();
}

void FActionFilterCache::CacheFilteredResult(const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction, bool bWasFilteredOut)
{
	const TSharedPtr<TMap<FActionFilterCacheKey, bool>> Cache = GetCachedFilterResults(Filter);
	Cache->Add(FActionFilterCacheKey(BlueprintAction), bWasFilteredOut);
}

bool FActionFilterCache::IsFilteredByCachedResult(const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction)
{
	const TSharedPtr<TMap<FActionFilterCacheKey, bool>> Cache = GetCachedFilterResults(Filter);
	if (const bool* Result = Cache->Find(FActionFilterCacheKey(BlueprintAction)))
	{
		return *Result;
	}

	return false;
}

TSharedPtr<TMap<FActionFilterCacheKey, bool>> FActionFilterCache::GetCachedFilterResults(const FBlueprintActionFilter& Filter)
{
	// if the leaf for this context has already been looked up, just grab it rather than researching the tree
	if (CachedFilterResults.IsValid())
	{
		return CachedFilterResults;
	}

	TSharedPtr<FActionFilterCacheNode> Current = SharedCache;

	/** update Current to it's child that matches Key. Create one if neccesary */
#define LOOKUP_CACHE_KEY(Key) \
	FindOrCacheChildNode(Current, Key, #Key)

/** update Current to it's child that matches Key as a leaf. Create one if neccesary */
#define LOOKUP_CACHE_KEY_LEAF(Key) \
	FindOrCacheLeaf(Current, Key, #Key)

/** update Current to it's child that matches Key as a leaf. Create one if neccesary
 *  uses Comparison lambda to check for key "equality"
 */
#define LOOKUP_CACHE_KEY_PREDICATE(Key, Comparison) \
	FindOrCacheChildNode<decltype(Key)>(Current, Key, #Key, TDelegate<bool(const decltype(Key)&, const decltype(Key)&)>::CreateLambda(Comparison))

 /** update Current to it's child that matches Key as a leaf. Create one if neccesary
  *  uses Comparison lambda to check for key "equality"
  */
#define LOOKUP_CACHE_KEY_LEAF_PREDICATE(Key, Comparison) \
	FindOrCacheLeaf<decltype(Key)>(Current, Key, #Key, TDelegate<bool(const decltype(Key)&, const decltype(Key)&)>::CreateLambda(Comparison))

  // traverse/build the cache tree looking for the cached results

  // cache based on what filter tests are being applied
  // @todo - this is no longer a struct type, so we would need some way to infer delegate equality (or maybe we can generate a filter GUID instead of the stuff below)
//	LOOKUP_CACHE_KEY(Filter.GetFilterTests());

	// cache based on which blueprints/graphs/editor we're generating actions for
	LOOKUP_CACHE_KEY(Filter.Context.Blueprints);
	LOOKUP_CACHE_KEY(Filter.Context.Graphs);
	LOOKUP_CACHE_KEY(Filter.Context.EditorPtr);

	// cache based on the filter's target classes
	LOOKUP_CACHE_KEY(Filter.TargetClasses);

	// cache based on what node types are permitted or rejected
	LOOKUP_CACHE_KEY(Filter.RejectedNodeTypes);
	LOOKUP_CACHE_KEY(Filter.PermittedNodeTypes);

	// cache based on what flags are active
//	LOOKUP_CACHE_KEY(Filter.GetFilterFlags());

	// cache based on the relevant pin types and directions
	LOOKUP_CACHE_KEY_LEAF_PREDICATE(Filter.Context.Pins, [](const TArray<UEdGraphPin*>& A, const TArray<UEdGraphPin*>& B)
	{
		if (A.Num() != B.Num())
		{
			return false;
		}
		for (int32 Index = 0; Index < A.Num(); ++Index)
		{
			if (A[Index]->Direction != B[Index]->Direction)
			{
				return false;
			}
			if (A[Index]->PinType != B[Index]->PinType)
			{
				return false;
			}
		}
		return true;
	});

	// at this point, IsFilteredByThisCache has been updated to a shared memory space that can be retrieved again
	// by following the above procedure with the same keys values.
	return CachedFilterResults;
}

void FActionFilterCache::PopCache()
{
	int32 MinIndex = 0;
	uint32 MinAccessCount = TNumericLimits<uint32>::Max();
	for (int32 LeafIndex = 0; LeafIndex < CacheLeafs.Num(); ++LeafIndex)
	{
		TWeakPtr<FActionFilterCacheNode>& Leaf = CacheLeafs[LeafIndex];
		// if there's invalid cache, remove it first
		if (!Leaf.IsValid())
		{
			MinIndex = LeafIndex;
			break;
		}

		const uint32 AccessCount = Leaf.Pin()->GetLeafAccessCount();
		if (AccessCount < MinAccessCount)
		{
			MinAccessCount = AccessCount;
			MinIndex = LeafIndex;
		}

		// decrement access count of all leafs so that recently used leafs get prioritized more
		Leaf.Pin()->DecrementLeafAccessCount();
	}

	TWeakPtr<FActionFilterCacheNode> RemovedElement = CacheLeafs[MinIndex];
	if (RemovedElement.IsValid())
	{
		RemovedElement.Pin()->UnCacheSelf();
	}

	CacheLeafs.RemoveAt(MinIndex, EAllowShrinking::No);
}

TSharedPtr<FActionFilterCacheNode> FActionFilterCache::SharedCache = MakeShared<FActionFilterCacheNode>();

TAutoConsoleVariable<int32> FActionFilterCache::CVarCacheLeafCapacity{
	TEXT("BP.ActionMenuFilterCacheLeafCapacity"),
	32,
	TEXT("The number of action menu contexts to cache simultaniously. raising this number will increase the memory footprint but decrease how often the cache is blown")
};

TArray<TWeakPtr<FActionFilterCacheNode>> FActionFilterCache::CacheLeafs;

FDelegateHandle FActionFilterCache::ReloadCompleteDelegate;
FDelegateHandle FActionFilterCache::ModuleChangedDelegate;