// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "HAL/IConsoleManager.h"

template<typename TKeyType>
class TActionFilterCacheKeyNode;
template<typename TKeyType>
class TActionFilterCacheLeaf;

/*******************************************************************************
 * FActionFilterCacheKey
 ******************************************************************************/

// Wrapper for a hashed action key
struct FActionFilterCacheKey
{
	uint32 KeyHash;

	explicit FActionFilterCacheKey(const FBlueprintActionInfo& InAction);
};

bool operator==(const FActionFilterCacheKey& A, const FActionFilterCacheKey& B);
uint32 GetTypeHash(const FActionFilterCacheKey& InKey);

/*******************************************************************************
 * FActionFilterCacheNode
 ******************************************************************************/

 /**
  * This class is used by FActionFilterCache to create a tree structure that is used to store cached filter results in
  * such a way that they can be looked up using all the relevant context data that determines how items are filtered.
  * each node has a type agnostic key so that we can easily change how the cache is keyed in the future.
  * (See FActionFilterCache::GetIsFilteredByThisCache)
  */
class FActionFilterCacheNode : public TSharedFromThis<FActionFilterCacheNode>
{
public:
	template<typename TKeyType>
	friend class TActionFilterCacheKeyNode;

	// Branching nodes
	using FWeakNode = TWeakPtr<FActionFilterCacheNode>;
	template<typename TKeyType>
	using TWeakKeyNode = TWeakPtr<TActionFilterCacheKeyNode<TKeyType>>;
	template<typename TKeyType>
	using TWeakLeaf = TWeakPtr<TActionFilterCacheLeaf<TKeyType>>;

	// Leaf nodes
	using FSharedNode = TSharedPtr<FActionFilterCacheNode>;
	template<typename TKeyType>
	using TSharedKeyNode = TSharedPtr<TActionFilterCacheKeyNode<TKeyType>>;
	template<typename TKeyType>
	using TSharedLeaf = TSharedPtr<TActionFilterCacheLeaf<TKeyType>>;

	FActionFilterCacheNode() = default;
	virtual ~FActionFilterCacheNode() = default;

	/** Looks up a child of this node by key. Note: the key is not hashed. This is O(n) */
	template<typename TKeyType>
	TSharedKeyNode<TKeyType> Find(const TKeyType& Key) const
	{
		for (const TSharedPtr<FActionFilterCacheNode>& Child : Children)
		{
			if (Child->KeysMatch(&Key))
			{
				return StaticCastSharedPtr<TActionFilterCacheKeyNode<TKeyType>>(Child);
			}
		}
		return nullptr;
	}

	/** Looks up a child of this node by key and creates one if one doesnt exist.
	 *	The node that is created will be a TActionFilterCacheKeyNode. Use FindOrCacheChildLeaf if you need it to be a
	 *	TActionFilterCacheLeaf
	 *  Note: the key is not hashed. This is O(n) */
	template<typename TKeyType>
	TSharedKeyNode<TKeyType> FindOrCacheChildNode(const TKeyType& Key, const FString& InDebugName = FString(), TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr, bool* OutFoundWithoutCaching = nullptr)
	{
		TSharedKeyNode<TKeyType> Result = Find(Key);
		if (OutFoundWithoutCaching)
		{
			*OutFoundWithoutCaching = Result.IsValid();
		}

		if (Result.IsValid())
		{
			return Result;
		}

#if UE_BUILD_DEBUG
		DebugName = InDebugName;
#endif
		return Cache(new TActionFilterCacheKeyNode<TKeyType>(Key, Comparison));
	}

	/** Looks up a child of this node by key and creates one if one doesnt exist.
	 *	The node that is created will be a TActionFilterCacheLeaf. Use FindOrCacheChildNode if you need it to be a
	 *	TActionFilterCacheKeyNode
	 *  Note: the key is not hashed. This is O(n) */
	template<typename TKeyType>
	TSharedLeaf<TKeyType> FindOrCacheChildLeaf(const TKeyType& Key, const FString& InDebugName = FString(), TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr, bool* OutFoundWithoutCaching = nullptr)
	{
		TSharedKeyNode<TKeyType> Result = Find(Key);
		if (OutFoundWithoutCaching)
		{
			*OutFoundWithoutCaching = Result.IsValid();
		}

		if (Result.IsValid())
		{
			TSharedLeaf<TKeyType> Leaf = StaticCastSharedPtr<TActionFilterCacheLeaf<TKeyType>>(Result);
			++Leaf->AccessCount;
			return Leaf;
		}
#if UE_BUILD_DEBUG
		DebugName = InDebugName;
#endif
		return Cache(new TActionFilterCacheLeaf<TKeyType>(Key, Comparison));
	}

	/** get the number of times this leaf has been accessed
	 *  Warning: throws and error if this isn't a TActionFilterCacheLeaf */
	virtual uint32 GetLeafAccessCount() const
	{
		checkf(false, TEXT("GetLeafAccessCount was called on a non-leaf node"))
			return 0;
	}

	/** decrement the number of times this leaf has been accessed
	 *  Warning: throws and error if this isn't a TActionFilterCacheLeaf */
	virtual void DecrementLeafAccessCount()
	{
		checkf(false, TEXT("DecrementLeafAccessCount was called on a non-leaf node"))
	}

	/** remove this node from the the tree */
	virtual void UnCacheSelf() {}

#if UE_BUILD_DEBUG
	FString DebugName;
#endif

protected:
	virtual bool KeysMatch(const void* RawKey) const
	{
		checkf(false, TEXT("Tried to call KeysMatch on a Node with no keys"))
			return false;
	}

	template<typename TNodeType>
	TSharedPtr<TNodeType> Cache(TNodeType* Child)
	{
		Child->Parent = AsShared();
		Children.Emplace(MakeShareable<FActionFilterCacheNode>(Child));
		return StaticCastSharedPtr<TNodeType>(Children.Last());
	}

	TArray<FSharedNode> Children;
	FWeakNode Parent;
};

/*******************************************************************************
 * FActionFilterCache
 ******************************************************************************/

 /**
  * This class creates a tree structure that is used to store cached filter results in such a way that they can be
  * looked up using all the relevant context data that determines how items are filtered.
  * each node has a type agnostic key so that we can easily change how the cache is keyed.
  * (See FActionFilterCache::GetIsFilteredByThisCache)
  */
class FActionFilterCache
{
public:
	FActionFilterCache();

	static void OnReloadComplete(EReloadCompleteReason Reason);
	static void OnModuleChanged(FName ModuleName, EModuleChangeReason ChangeReason);
	static void ClearAllCache();

private:
	/** get the cached map used by FBlueprintActionFilter::IsFilteredByThis to determine whether BP actions are filtered out */
	TSharedPtr<TMap<FActionFilterCacheKey, bool>> GetCachedFilterResults(const FBlueprintActionFilter& Filter);

	template<typename TKeyType>
	void FindOrCacheChildNode(TSharedPtr<FActionFilterCacheNode>& Current, const TKeyType& Key, FString DebugName, TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr)
	{
		TSharedPtr<TActionFilterCacheKeyNode<TKeyType>> Found = Current->FindOrCacheChildNode(Key, DebugName, Comparison);
		Current = StaticCastSharedPtr<FActionFilterCacheNode>(Found);

	}

	template<typename TKeyType>
	void FindOrCacheLeaf(TSharedPtr<FActionFilterCacheNode>& Current, const TKeyType& Key, FString DebugName, TDelegate<bool(const TKeyType&, const TKeyType&)> Comparison = nullptr)
	{
		bool bFoundWithoutCaching = false;
		TSharedPtr<TActionFilterCacheLeaf<TKeyType>> Found = Current->FindOrCacheChildLeaf(Key, DebugName, Comparison, &bFoundWithoutCaching);
		CachedFilterResults = Found->CachedFilterResults;
		Current = StaticCastSharedPtr<FActionFilterCacheNode>(Found);

		// if we just expanded the cache, log it and check if the cache is getting too big
		if (!bFoundWithoutCaching)
		{
			if (CacheLeafs.Num() >= CVarCacheLeafCapacity.GetValueOnGameThread())
			{
				PopCache();
			}
			CacheLeafs.Push(Found);
		}
	}

	/** Find the least likely item in the cache to be used again, and remove it to make room for more data */
	static void PopCache();

	/** Cache result */
	void CacheFilteredResult(const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction, bool bWasFilteredOut);

	/** Filter function */
	bool IsFilteredByCachedResult(const FBlueprintActionFilter& Filter, FBlueprintActionInfo& BlueprintAction);

private:
	/** the shared tree that stores previously calculated filter results for several different contects */
	static TSharedPtr<FActionFilterCacheNode> SharedCache;

	static TAutoConsoleVariable<int32> CVarCacheLeafCapacity;

	/** array of all the leaf nodes in SharedCache so we can pop a rarely used node if we exceed CacheLeafCapacity */
	static TArray<TWeakPtr<FActionFilterCacheNode>> CacheLeafs;

	static FDelegateHandle ReloadCompleteDelegate;
	static FDelegateHandle ModuleChangedDelegate;

	/** cached results from GetIsFilteredByThisCache so we don't have to re-search the tree */
	TSharedPtr<TMap<FActionFilterCacheKey, bool>> CachedFilterResults;
};