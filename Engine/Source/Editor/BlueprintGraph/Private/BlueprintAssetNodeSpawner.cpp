// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintAssetNodeSpawner.h"

#include "EdGraph/EdGraphNode.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintAssetNodeSpawner)

class UObject;

#define LOCTEXT_NAMESPACE "BlueprintAssetNodeSpawner"

UBlueprintAssetNodeSpawner* UBlueprintAssetNodeSpawner::Create(TSubclassOf<UEdGraphNode> const InNodeClass, const FAssetData& InAssetData, UObject* InOuter, FCustomizeNodeDelegate InPostSpawnDelegate)
{
	check(InNodeClass != nullptr);
	check(InNodeClass->IsChildOf<UEdGraphNode>());

	if (InOuter == nullptr)
	{
		InOuter = GetTransientPackage();
	}

	UBlueprintAssetNodeSpawner* NodeSpawner = NewObject<UBlueprintAssetNodeSpawner>(InOuter);
	NodeSpawner->NodeClass = InNodeClass;
	NodeSpawner->CustomizeNodeDelegate = InPostSpawnDelegate;
	NodeSpawner->AssetData = InAssetData;

	return NodeSpawner;
}

const FAssetData& UBlueprintAssetNodeSpawner::GetAssetData() const
{
	return AssetData;
}

#undef LOCTEXT_NAMESPACE
