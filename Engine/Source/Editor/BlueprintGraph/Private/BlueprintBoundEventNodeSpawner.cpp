// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintBoundEventNodeSpawner.h"

#include "BlueprintNodeSpawner.h"
#include "BlueprintNodeSpawnerUtils.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Event.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintBoundEventNodeSpawner)

class UBlueprint;

#define LOCTEXT_NAMESPACE "BlueprintBoundEventNodeSpawner"

/*******************************************************************************
 * Static UBlueprintBoundEventNodeSpawner Helpers
 ******************************************************************************/

namespace BlueprintBoundEventNodeSpawnerImpl
{
	static FText GetDefaultMenuName(FMulticastDelegateProperty const* Delegate);
	static FText GetDefaultMenuCategory(FMulticastDelegateProperty const* Delegate);
}

//------------------------------------------------------------------------------
static FText BlueprintBoundEventNodeSpawnerImpl::GetDefaultMenuName(FMulticastDelegateProperty const* Delegate)
{
	bool const bShowFriendlyNames = GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
	FText const DelegateName = bShowFriendlyNames ? FText::FromString(UEditorEngine::GetFriendlyName(Delegate)) : FText::FromName(Delegate->GetFName());

	return FText::Format(LOCTEXT("ComponentEventName", "Add {0}"), DelegateName);
}

//------------------------------------------------------------------------------
static FText BlueprintBoundEventNodeSpawnerImpl::GetDefaultMenuCategory(FMulticastDelegateProperty const* Delegate)
{
	FText DelegateCategory = FText::FromString(FObjectEditorUtils::GetCategory(Delegate));
	if (DelegateCategory.IsEmpty())
	{
		DelegateCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Delegates);
	}
	return DelegateCategory;
}

/*******************************************************************************
 * UBlueprintBoundEventNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintBoundEventNodeSpawner* UBlueprintBoundEventNodeSpawner::Create(TSubclassOf<UK2Node_Event> NodeClass, FMulticastDelegateProperty* EventDelegate, UObject* Outer/* = nullptr*/)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	UBlueprintBoundEventNodeSpawner* NodeSpawner = NewObject<UBlueprintBoundEventNodeSpawner>(Outer);
	NodeSpawner->NodeClass     = NodeClass;
	NodeSpawner->EventDelegate = EventDelegate;

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	MenuSignature.MenuName = BlueprintBoundEventNodeSpawnerImpl::GetDefaultMenuName(EventDelegate);
	MenuSignature.Category = BlueprintBoundEventNodeSpawnerImpl::GetDefaultMenuCategory(EventDelegate);
	//MenuSignature.Tooltip,  will be pulled from the node template
	//MenuSignature.Keywords, will be pulled from the node template
	MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintBoundEventNodeSpawner::UBlueprintBoundEventNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, EventDelegate(nullptr)
{
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintBoundEventNodeSpawner::GetSpawnerSignature() const
{
	// explicit actions for binding (like this) cannot be reconstructed form a 
	// signature (since this spawner does not own whatever it will be binding 
	// to), therefore we return an empty (invalid) signature
	return FBlueprintNodeSignature();
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintBoundEventNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	UK2Node_Event* EventNode = nullptr;
	if (Bindings.Num() > 0)
	{
		EventNode = CastChecked<UK2Node_Event>(Super::Invoke(ParentGraph, Bindings, Location));
	}
	return EventNode;
}

//------------------------------------------------------------------------------
UK2Node_Event const* UBlueprintBoundEventNodeSpawner::FindPreExistingEvent(UBlueprint* Blueprint, FBindingSet const& Bindings) const
{
	UK2Node_Event const* PreExistingEvent = nullptr;

	FBindingObject BoundObject;
	if (Bindings.Num() > 0)
	{
		BoundObject = *Bindings.CreateConstIterator();
	}

	if (BoundObject != nullptr)
	{
		if (NodeClass->IsChildOf<UK2Node_ComponentBoundEvent>())
		{
			PreExistingEvent = FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, EventDelegate->GetFName(), BoundObject.GetFName());
		}
		else if (NodeClass->IsChildOf<UK2Node_ActorBoundEvent>())
		{
			PreExistingEvent = FKismetEditorUtilities::FindBoundEventForActor(CastChecked<AActor>(BoundObject.Get<UObject>()), EventDelegate->GetFName());
		}
	}
	return PreExistingEvent;
}

//------------------------------------------------------------------------------
bool UBlueprintBoundEventNodeSpawner::IsBindingCompatible(FBindingObject BindingCandidate) const
{
	bool bMatchesNodeType = false;
	if (NodeClass->IsChildOf<UK2Node_ComponentBoundEvent>())
	{
		FObjectProperty const* BindingProperty = BindingCandidate.Get<FObjectProperty>();
		bMatchesNodeType = (BindingProperty != nullptr);
	}
	else if (NodeClass->IsChildOf<UK2Node_ActorBoundEvent>())
	{
		bMatchesNodeType = BindingCandidate.IsA<AActor>();
	}

	const FMulticastDelegateProperty* Delegate = GetEventDelegate();

	if ( !ensureMsgf(!FBlueprintNodeSpawnerUtils::IsStaleFieldAction(this), 
			TEXT("Invalid BlueprintBoundEventNodeSpawner (for %s). Was the action database properly updated when this class was compiled?"), 
			*Delegate->GetOwnerClass()->GetName()))
	{
		return false;
	}

	UClass* DelegateOwner = Delegate->GetOwnerClass()->GetAuthoritativeClass();
	UClass* BindingClass  = FBlueprintNodeSpawnerUtils::GetBindingClass(BindingCandidate)->GetAuthoritativeClass();

	return bMatchesNodeType && BindingClass && BindingClass->IsChildOf(DelegateOwner) && !FObjectEditorUtils::IsVariableCategoryHiddenFromClass(Delegate, BindingClass);
}

//------------------------------------------------------------------------------
bool UBlueprintBoundEventNodeSpawner::BindToNode(UEdGraphNode* Node, FBindingObject Binding) const
{
	bool bWasBound = false;
	if (UK2Node_ComponentBoundEvent* ComponentEventNode = Cast<UK2Node_ComponentBoundEvent>(Node))
	{
		if (FObjectProperty const* BoundProperty = Binding.Get<FObjectProperty>())
		{
			ComponentEventNode->InitializeComponentBoundEventParams(BoundProperty, EventDelegate.Get());
			bWasBound = true;
			Node->ReconstructNode();
		}
	}
	else if (UK2Node_ActorBoundEvent* ActorEventNode = CastChecked<UK2Node_ActorBoundEvent>(Node))
	{
		if (AActor* BoundActor = Binding.Get<AActor>())
		{
			ActorEventNode->InitializeActorBoundEventParams(BoundActor, EventDelegate.Get());
			bWasBound = true;
			Node->ReconstructNode();
		}
	}
	return bWasBound;
}

//------------------------------------------------------------------------------
FMulticastDelegateProperty const* UBlueprintBoundEventNodeSpawner::GetEventDelegate() const
{
	return EventDelegate.Get();
}

#undef LOCTEXT_NAMESPACE
