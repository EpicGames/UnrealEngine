// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintBoundNodeSpawner.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintBoundNodeSpawner)

class UEdGraphNode;

#define LOCTEXT_NAMESPACE "BlueprintBoundNodeSpawner"

/*******************************************************************************
 * UBlueprintBoundNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintBoundNodeSpawner* UBlueprintBoundNodeSpawner::Create(TSubclassOf<UEdGraphNode> NodeClass, UObject* Outer/* = nullptr*/)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	UBlueprintBoundNodeSpawner* NodeSpawner = NewObject<UBlueprintBoundNodeSpawner>(Outer);
	NodeSpawner->NodeClass = NodeClass;

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintBoundNodeSpawner::UBlueprintBoundNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintBoundNodeSpawner::GetSpawnerSignature() const
{
	// explicit actions for binding (like this) cannot be reconstructed form a 
	// signature (since this spawner does not own whatever it will be binding 
	// to), therefore we return an empty (invalid) signature
	return FBlueprintNodeSignature();
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintBoundNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	UEdGraphNode* Result = nullptr;
	if( FindPreExistingNodeDelegate.IsBound() )
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
		Result = FindPreExistingNodeDelegate.Execute( Blueprint, Bindings );
	}
	if( Result == nullptr)
	{
		Result = UBlueprintNodeSpawner::Invoke( ParentGraph, Bindings, Location );
	}
	return Result;
}

//------------------------------------------------------------------------------
bool UBlueprintBoundNodeSpawner::IsBindingCompatible(FBindingObject BindingCandidate) const
{
	if(CanBindObjectDelegate.IsBound())
	{
		return CanBindObjectDelegate.Execute(BindingCandidate.Get<UObject>());
	}
	return false;
}

//------------------------------------------------------------------------------
bool UBlueprintBoundNodeSpawner::BindToNode(UEdGraphNode* Node, FBindingObject Binding) const
{
	if(OnBindObjectDelegate.IsBound())
	{
		return OnBindObjectDelegate.Execute(Node, Binding.Get<UObject>());
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
