// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintComponentNodeSpawner.h"

#include "BlueprintEditorModule.h"
#include "BlueprintNodeTemplateCache.h"
#include "ComponentAssetBroker.h"
#include "ComponentTypeRegistry.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_AddComponent.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AssertionMacros.h"
#include "Misc/PackageName.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintComponentNodeSpawner)

#define LOCTEXT_NAMESPACE "BlueprintComponenetNodeSpawner"

/*******************************************************************************
 * Static UBlueprintComponentNodeSpawner Helpers
 ******************************************************************************/

namespace BlueprintComponentNodeSpawnerImpl
{
	//------------------------------------------------------------------------------
	static FText GetMenuCategoryFormat()
	{
		return LOCTEXT("ComponentCategory", "Add Component|{0}");
	}

	//------------------------------------------------------------------------------
	static FText GetDefaultMenuCategory(const TSubclassOf<UActorComponent> ComponentClass)
	{
		TArray<FString> ClassGroupNames;
		ComponentClass->GetClassGroupNames(ClassGroupNames);

		static const FText DefaultClassGroup(LOCTEXT("DefaultClassGroup", "Common"));
		// 'Common' takes priority over other class groups
		if (ClassGroupNames.Contains(DefaultClassGroup.ToString()) || (ClassGroupNames.Num() == 0))
		{
			return DefaultClassGroup;
		}
		
		return FText::FromString(ClassGroupNames[0]);
	}
}

/*******************************************************************************
 * UBlueprintComponentNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintComponentNodeSpawner* UBlueprintComponentNodeSpawner::Create(const FComponentTypeEntry& Entry)
{
	using namespace BlueprintComponentNodeSpawnerImpl;

	UClass* ComponentClass = Entry.ComponentClass;
	if (ComponentClass == nullptr)
	{
		// unloaded class, must be blueprint created. Create an entry. We'll load the class when we spawn the node:

		UBlueprintComponentNodeSpawner* NodeSpawner = NewObject<UBlueprintComponentNodeSpawner>(GetTransientPackage());
		NodeSpawner->ComponentClass = nullptr;
		NodeSpawner->NodeClass = UK2Node_AddComponent::StaticClass();
		NodeSpawner->ComponentName = Entry.ComponentName;
		NodeSpawner->ComponentAssetName = Entry.ComponentAssetName;

		FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
		FText const ComponentTypeName = FText::FromString(Entry.ComponentName);
		MenuSignature.MenuName = FText::Format(LOCTEXT("AddComponentMenuName", "Add {0}"), ComponentTypeName);
		// Note: Non-native (i.e. BP) component types are automatically assigned to the "Custom" group name.
		// => @see FBlueprintEditorUtils::RecreateClassMetaData()
		MenuSignature.Category = FText::Format(GetMenuCategoryFormat(), LOCTEXT("BlueprintComponentCategory", "Custom"));
		MenuSignature.Tooltip = FText::Format(LOCTEXT("AddComponentTooltip", "Spawn a {0}"), ComponentTypeName);
		// add at least one character, so that PrimeDefaultUiSpec() doesn't 
		// attempt to query the template node
		if (MenuSignature.Keywords.IsEmpty())
		{
			MenuSignature.Keywords = FText::FromString(TEXT(" "));
		}
		// Note: Currently using whatever styling is in place for UActorComponent. Note that the actual
		// class when loaded could be a USceneComponent derivative, in which case it might end up having
		// an alternate styling. For now just going with this as the placeholder to match the basic type.
		MenuSignature.Icon = FSlateIconFinder::FindIconForClass(UActorComponent::StaticClass());
		MenuSignature.DocLink  = TEXT("Shared/GraphNodes/Blueprint/UK2Node_AddComponent");
		MenuSignature.DocExcerptTag = TEXT("AddComponent");

		return NodeSpawner;
	}

	if (ComponentClass->ClassWithin && ComponentClass->ClassWithin != UObject::StaticClass())
	{
		// we can't support 'Within' markup on components at this time (core needs to be aware of non-CDO archetypes
		// that have within markup, and BP system needs to property use RF_ArchetypeObject on template objects)
		return nullptr;
	}

	UClass* const AuthoritativeClass = ComponentClass->GetAuthoritativeClass();

	UBlueprintComponentNodeSpawner* NodeSpawner = NewObject<UBlueprintComponentNodeSpawner>(GetTransientPackage());
	NodeSpawner->ComponentClass = AuthoritativeClass;
	NodeSpawner->NodeClass      = UK2Node_AddComponent::StaticClass();

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	FText const ComponentTypeName = AuthoritativeClass->GetDisplayNameText();
	MenuSignature.MenuName = FText::Format(LOCTEXT("AddComponentMenuName", "Add {0}"), ComponentTypeName);
	MenuSignature.Category = FText::Format(GetMenuCategoryFormat(), GetDefaultMenuCategory(AuthoritativeClass));
	MenuSignature.Tooltip  = FText::Format(LOCTEXT("AddComponentTooltip", "Spawn a {0}"), ComponentTypeName);
	MenuSignature.Keywords = AuthoritativeClass->GetMetaDataText(*FBlueprintMetadata::MD_FunctionKeywords.ToString(), TEXT("UObjectKeywords"), AuthoritativeClass->GetFullGroupName(false));
	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	if (MenuSignature.Keywords.IsEmpty())
	{
		MenuSignature.Keywords = FText::FromString(TEXT(" "));
	}
	MenuSignature.Icon = FSlateIconFinder::FindIconForClass(AuthoritativeClass);
	MenuSignature.DocLink  = TEXT("Shared/GraphNodes/Blueprint/UK2Node_AddComponent");
	MenuSignature.DocExcerptTag = AuthoritativeClass->GetName();

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintComponentNodeSpawner::UBlueprintComponentNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintComponentNodeSpawner::GetSpawnerSignature() const
{
	FBlueprintNodeSignature SpawnerSignature(NodeClass);
	SpawnerSignature.AddSubObject(ComponentClass.Get());
	return SpawnerSignature;
}

//------------------------------------------------------------------------------
// Evolved from a combination of FK2ActionMenuBuilder::CreateAddComponentAction()
// and FEdGraphSchemaAction_K2AddComponent::PerformAction().
UEdGraphNode* UBlueprintComponentNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	UClass* ComponentType = ComponentClass;
	auto PostSpawnLambda = [ComponentType](UEdGraphNode* NewNode, bool bIsTemplateNode, FCustomizeNodeDelegate UserDelegate)
	{		
		UK2Node_AddComponent* AddCompNode = CastChecked<UK2Node_AddComponent>(NewNode);
		UBlueprint* Blueprint = AddCompNode->GetBlueprint();
		
		UFunction* AddComponentFunc = FindFieldChecked<UFunction>(AActor::StaticClass(), UK2Node_AddComponent::GetAddComponentFunctionName());
		AddCompNode->FunctionReference.SetFromField<UFunction>(AddComponentFunc, !bIsTemplateNode && FBlueprintEditorUtils::IsActorBased(Blueprint));

		AddCompNode->TemplateType = ComponentType;

		UserDelegate.ExecuteIfBound(NewNode, bIsTemplateNode);
	};

	FCustomizeNodeDelegate PostSpawnDelegate = FCustomizeNodeDelegate::CreateLambda(PostSpawnLambda, CustomizeNodeDelegate);
	// let SpawnNode() allocate default pins (so we can modify them)
	UK2Node_AddComponent* NewNode = Super::SpawnNode<UK2Node_AddComponent>(NodeClass, ParentGraph, FBindingSet(), Location, PostSpawnDelegate);
	if (NewNode->Pins.Num() == 0)
	{
		NewNode->AllocateDefaultPins();
	}

	// set the return type to be the type of the template
	UEdGraphPin* ReturnPin = NewNode->GetReturnValuePin();
	if (ReturnPin != nullptr)
	{
		if (ComponentClass != nullptr)
		{
			ReturnPin->PinType.PinSubCategoryObject = ComponentType;
		}
		else
		{
			ReturnPin->PinType.PinSubCategoryObject = UActorComponent::StaticClass();
		}
	}

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);
	if (!bIsTemplateNode)
	{
		TSubclassOf<UActorComponent> Class = ComponentClass;
		if (Class == nullptr)
		{
			const ELoadFlags LoadFlags = LOAD_None;
			UObject* LoadedObject = LoadObject<UObject>(/*Outer =*/ nullptr, *ComponentAssetName, /*Filename =*/ nullptr, LoadFlags);
			if (LoadedObject == nullptr)
			{
				return nullptr;
			}

			// Note: The asset name may refer to either the generated class or the outer BP asset.
			if (const UBlueprint* LoadedObjectAsBlueprint = Cast<UBlueprint>(LoadedObject))
			{
				Class = TSubclassOf<UActorComponent>(LoadedObjectAsBlueprint->GeneratedClass);
			}
			else
			{
				Class = TSubclassOf<UActorComponent>(Cast<UBlueprintGeneratedClass>(LoadedObject));
			}
			
			if (Class == nullptr)
			{
				return nullptr;
			}

			// Since the node has already been spawned, we need to update its template type.
			// Note that the return pin's type will be updated when we reconstruct the node below.
			NewNode->TemplateType = Class;
		}

		UActorComponent* ComponentTemplate = nullptr;
		if (FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(Class))
		{
			UBlueprint* Blueprint = NewNode->GetBlueprint();

			FName DesiredComponentName = NewNode->MakeNewComponentTemplateName(Blueprint->GeneratedClass, Class);
			ComponentTemplate = NewObject<UActorComponent>(Blueprint->GeneratedClass, Class, DesiredComponentName, RF_ArchetypeObject | RF_Public | RF_Transactional);

			Blueprint->ComponentTemplates.Add(ComponentTemplate);
		}

		// set the name of the template as the default for the TemplateName param
		UEdGraphPin* TemplateNamePin = NewNode->GetTemplateNamePinChecked();
		if (TemplateNamePin != nullptr)
		{
			TemplateNamePin->DefaultValue = ComponentTemplate ? ComponentTemplate->GetName() : TEXT("");
		}
		NewNode->ReconstructNode();
	}

	// apply bindings, after we've setup the template pin
	ApplyBindings(NewNode, Bindings);

	return NewNode;
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec UBlueprintComponentNodeSpawner::GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const
{
	UEdGraph* TargetGraph = (Context.Graphs.Num() > 0) ? Context.Graphs[0] : nullptr;
	FBlueprintActionUiSpec MenuSignature = PrimeDefaultUiSpec(TargetGraph);

	if (Bindings.Num() > 0)
	{
		FText AssetName;
		{
			FBindingObject Binding = *Bindings.CreateConstIterator();
			if (Binding.IsValid())
			{
				AssetName = FText::FromName(Binding.GetFName());
			}
		}

		FText const ComponentTypeName = FText::FromName(ComponentClass->GetFName());
		MenuSignature.MenuName = FText::Format(LOCTEXT("AddBoundComponentMenuName", "Add {0} (as {1})"), AssetName, ComponentTypeName);
		MenuSignature.Tooltip  = FText::Format(LOCTEXT("AddBoundComponentTooltip", "Spawn {0} using {1}"), ComponentTypeName, AssetName);
	}
	DynamicUiSignatureGetter.ExecuteIfBound(Context, Bindings, &MenuSignature);
	return MenuSignature;
}

//------------------------------------------------------------------------------
bool UBlueprintComponentNodeSpawner::IsBindingCompatible(FBindingObject BindingCandidate) const
{
	bool bCanBindWith = false;
	if (BindingCandidate.IsUObject() && BindingCandidate.Get<UObject>()->IsAsset())
	{
		TArray< TSubclassOf<UActorComponent> > ComponentClasses = FComponentAssetBrokerage::GetComponentsForAsset(BindingCandidate.Get<UObject>());
		bCanBindWith = ComponentClasses.Contains(ComponentClass);
	}
	return bCanBindWith;
}

//------------------------------------------------------------------------------
bool UBlueprintComponentNodeSpawner::BindToNode(UEdGraphNode* Node, FBindingObject Binding) const
{
	bool bSuccessfulBinding = false;
	UK2Node_AddComponent* AddCompNode = CastChecked<UK2Node_AddComponent>(Node);

	UActorComponent* ComponentTemplate = AddCompNode->GetTemplateFromNode();
	if (ComponentTemplate != nullptr)
	{
		check(!Binding.IsValid() || Binding.IsUObject()); // FProp
		bSuccessfulBinding = FComponentAssetBrokerage::AssignAssetToComponent(ComponentTemplate, Binding.Get<UObject>());
		AddCompNode->ReconstructNode();
	}
	return bSuccessfulBinding;
}

//------------------------------------------------------------------------------
TSubclassOf<UActorComponent> UBlueprintComponentNodeSpawner::GetComponentClass() const
{
	return ComponentClass;
}

//------------------------------------------------------------------------------
bool UBlueprintComponentNodeSpawner::IsTemplateNodeFilteredOut(const FBlueprintActionFilter& Filter) const
{
	bool bIsFilteredOut = false;

	if (Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectNonImportedFields))
	{
		TSharedPtr<IBlueprintEditor> BlueprintEditor = Filter.Context.EditorPtr.Pin();
		if (BlueprintEditor.IsValid())
		{
			if (ComponentClass)
			{
				bIsFilteredOut = BlueprintEditor->IsNonImportedObject(ComponentClass);
			}
			else if(FPackageName::IsValidObjectPath(ComponentAssetName))
			{
				bIsFilteredOut = BlueprintEditor->IsNonImportedObject(ComponentAssetName);
			}
		}
	}

	return bIsFilteredOut || UBlueprintNodeSpawner::IsTemplateNodeFilteredOut(Filter);
}

#undef LOCTEXT_NAMESPACE
