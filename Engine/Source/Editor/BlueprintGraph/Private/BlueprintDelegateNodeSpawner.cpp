// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintDelegateNodeSpawner.h"

#include "BlueprintNodeSpawner.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Text.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_Variable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintDelegateNodeSpawner)

class UBlueprint;
class UObject;
struct FLinearColor;

#define LOCTEXT_NAMESPACE "BlueprintDelegateNodeSpawner"

/*******************************************************************************
 * Static UBlueprintDelegateNodeSpawner Helpers
 ******************************************************************************/

namespace BlueprintDelegateNodeSpawnerImpl
{
	static FText GetDefaultMenuName(FMulticastDelegateProperty const* Delegate);
	static FText GetDefaultMenuCategory(FMulticastDelegateProperty const* Delegate);
	static FSlateIcon GetDefaultMenuIcon(FMulticastDelegateProperty const* Delegate, FLinearColor& ColorOut);
}

//------------------------------------------------------------------------------
static FText BlueprintDelegateNodeSpawnerImpl::GetDefaultMenuName(FMulticastDelegateProperty const* Delegate)
{
	bool const bShowFriendlyNames = GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
	return bShowFriendlyNames ? FText::FromString(UEditorEngine::GetFriendlyName(Delegate)) : FText::FromName(Delegate->GetFName());
}

//------------------------------------------------------------------------------
static FText BlueprintDelegateNodeSpawnerImpl::GetDefaultMenuCategory(FMulticastDelegateProperty const* Delegate)
{
	FText DelegateCategory = FText::FromString(FObjectEditorUtils::GetCategory(Delegate));
	if (DelegateCategory.IsEmpty())
	{
		DelegateCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Delegates);
	}
	return DelegateCategory;
}

//------------------------------------------------------------------------------
static FSlateIcon BlueprintDelegateNodeSpawnerImpl::GetDefaultMenuIcon(FMulticastDelegateProperty const* Delegate, FLinearColor& ColorOut)
{
	FName    const PropertyName = Delegate->GetFName();
	UStruct* const PropertyOwner = CastChecked<UStruct>(Delegate->GetOwner<UField>());

	return UK2Node_Variable::GetVariableIconAndColor(PropertyOwner, PropertyName, ColorOut);
}

/*******************************************************************************
 * UBlueprintDelegateNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintDelegateNodeSpawner* UBlueprintDelegateNodeSpawner::Create(TSubclassOf<UK2Node_BaseMCDelegate> NodeClass, FMulticastDelegateProperty const* const Property, UObject* Outer/* = nullptr*/)
{
	check(Property != nullptr);
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	//--------------------------------------
	// Constructing the Spawner
	//--------------------------------------

	UBlueprintDelegateNodeSpawner* NodeSpawner = NewObject<UBlueprintDelegateNodeSpawner>(Outer);
	NodeSpawner->SetField(const_cast<FMulticastDelegateProperty*>(Property));
	NodeSpawner->NodeClass = NodeClass;

	//--------------------------------------
	// Default UI Signature
	//--------------------------------------

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	//MenuSignature.MenuName, will be pulled from the node template
	MenuSignature.Category = BlueprintDelegateNodeSpawnerImpl::GetDefaultMenuCategory(Property);
	//MenuSignature.Tooltip,  will be pulled from the node template
	//MenuSignature.Keywords, will be pulled from the node template
	MenuSignature.Icon = BlueprintDelegateNodeSpawnerImpl::GetDefaultMenuIcon(Property, MenuSignature.IconTint);

	//--------------------------------------
	// Post-Spawn Setup
	//--------------------------------------

	auto SetDelegateLambda = [](UEdGraphNode* NewNode, FFieldVariant InField)
	{
		FMulticastDelegateProperty const* MCDProperty = CastField<FMulticastDelegateProperty>(InField.ToField());

		UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(NewNode);
		if ((DelegateNode != nullptr) && (MCDProperty != nullptr))
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(NewNode);
			UClass* OwnerClass = MCDProperty->GetOwnerClass();

			DelegateNode->SetFromProperty(MCDProperty, false, OwnerClass);
		}
	};
	NodeSpawner->SetNodeFieldDelegate = FSetNodeFieldDelegate::CreateStatic(SetDelegateLambda);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintDelegateNodeSpawner::UBlueprintDelegateNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
FMulticastDelegateProperty const* UBlueprintDelegateNodeSpawner::GetDelegateProperty() const
{
	return CastField<FMulticastDelegateProperty>(GetField().ToField());
}

#undef LOCTEXT_NAMESPACE
