// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEditorSettings.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintEditorModule.h"
#include "BlueprintNamespaceHelper.h"
#include "BlueprintNamespaceUtilities.h"
#include "CoreGlobals.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "initializer_list"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintEditorSettings)

UBlueprintEditorSettings::UBlueprintEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	// Style Settings
	, bDrawMidpointArrowsInBlueprints(false)
	, bShowGraphInstructionText(true)
	, bHideUnrelatedNodes(false)
	, bShowShortTooltips(true)
	, bShowFunctionParameterIcon(true)
	, bShowFunctionLocalVariableIcon(true)
	// Workflow Settings
	, bEnableInputTriggerSupportWarnings(false)
	, bSplitContextTargetSettings(true)
	, bExposeAllMemberComponentFunctions(true)
	, bShowContextualFavorites(false)
	, bExposeDeprecatedFunctions(false)
	, bCompactCallOnMemberNodes(false)
	, bFlattenFavoritesMenus(true)
	, bAutoCastObjectConnections(false)
	, bShowViewportOnSimulate(false)
	, bSpawnDefaultBlueprintNodes(true)
	, bHideConstructionScriptComponentsInDetailsView(true)
	, bNavigateToNativeFunctionsFromCallNodes(true)
	, bDoubleClickNavigatesToParent(true)
	, bEnableTypePromotion(true)
	, bShowPanelContextMenuForIncompatibleConnections(true)
	, TypePromotionPinDenyList { UEdGraphSchema_K2::PC_String, UEdGraphSchema_K2::PC_Text }
	, BreakpointReloadMethod(EBlueprintBreakpointReloadMethod::RestoreAll)
	, bEnablePinValueInspectionTooltips(true)
	, bEnableNamespaceEditorFeatures(true)
	, bEnableContextMenuTimeSlicing(true)
	, ContextMenuTimeSlicingThresholdMs(50)
	, bIncludeActionsForSelectedAssetsInContextMenu(false)
	, bLimitAssetActionBindingToSingleSelectionOnly(false)
	, bLoadSelectedAssetsForContextMenuActionBinding(true)
	, bDoNotMarkAllInstancesDirtyOnDefaultValueChange(true)
	// Experimental
	, bFavorPureCastNodes(false)
	// Compiler Settings
	, SaveOnCompile(SoC_Never)
	, bJumpToNodeErrors(false)
	, bAllowExplicitImpureNodeDisabling(true)
	// Developer Settings
	, bShowActionMenuItemSignatures(false)
	// Perf Settings
	, NodeTemplateCacheCapMB(20.f)
	// Find-in-Blueprints Settings
	, AllowIndexAllBlueprints(EFiBIndexAllPermission::LoadOnly)
	, bSearchForReferencesWhenVariableTypeChanged(false)
	// No category
	, bShowInheritedVariables(false)
	, bAlwaysShowInterfacesInOverrides(true)
	, bShowParentClassInOverrides(true)
	, bShowEmptySections(true)
	, bShowAccessSpecifier(false)
	, bIncludeCommentNodesInBookmarksTab(true)
	, bShowBookmarksForCurrentDocumentOnlyInTab(false)
	, bEnableNamespaceFilteringFeatures(false)
	, bEnableNamespaceImportingFeatures(false)
	, bInheritImportedNamespacesFromParentBP(false)
{
	// settings that were moved out of experimental...
	UEditorExperimentalSettings const* ExperimentalSettings = GetDefault<UEditorExperimentalSettings>();
	bDrawMidpointArrowsInBlueprints = ExperimentalSettings->bDrawMidpointArrowsInBlueprints;

	// settings that were moved out of editor-user settings...
	UEditorPerProjectUserSettings const* UserSettings = GetDefault<UEditorPerProjectUserSettings>();
	bShowActionMenuItemSignatures = UserSettings->bDisplayActionListItemRefIds;

	FString const ClassConfigKey = GetClass()->GetPathName();

	bool bOldSaveOnCompileVal = false;
	// backwards compatibility: handle the case where users have already switched this on
	if (GConfig->GetBool(*ClassConfigKey, TEXT("bSaveOnCompile"), bOldSaveOnCompileVal, GEditorPerProjectIni) && bOldSaveOnCompileVal)
	{
		SaveOnCompile = SoC_SuccessOnly;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRenamed().AddUObject(this, &UBlueprintEditorSettings::OnAssetRenamed);
	AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddUObject(this, &UBlueprintEditorSettings::OnAssetRemoved);
}

void UBlueprintEditorSettings::OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldName)
{
	FPerBlueprintSettings Temp;
	if(PerBlueprintSettings.RemoveAndCopyValue(InOldName, Temp))
	{
		PerBlueprintSettings.Add(AssetInfo.GetObjectPathString(), Temp);
		SaveConfig();
	}
}

void UBlueprintEditorSettings::OnAssetRemoved(UObject* Object)
{
	if(UBlueprint* Blueprint = Cast<UBlueprint>(Object))
	{
		FKismetDebugUtilities::ClearBreakpoints(Blueprint);
		FKismetDebugUtilities::ClearPinWatches(Blueprint);
	}
}

void UBlueprintEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Initialize transient flags and console variables for namespace editor features from the config.
	// @todo_namespaces - May be removed once dependent code is changed to utilize the config setting.
	bEnableNamespaceFilteringFeatures = bEnableNamespaceEditorFeatures;
	bEnableNamespaceImportingFeatures = bEnableNamespaceEditorFeatures;

	// Update console flags to match the current configuration.
	FBlueprintNamespaceHelper::RefreshEditorFeatureConsoleFlags();
}

void UBlueprintEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	bool bShouldRebuildRegistry = false;
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, bExposeDeprecatedFunctions))
	{
		bShouldRebuildRegistry = true;
	}
	
	// Type promotion changes are handled by the action database refresh
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, bEnableTypePromotion) || 
		PropertyName == GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, TypePromotionPinDenyList))
	{
		bShouldRebuildRegistry = true;
	}

	if (bShouldRebuildRegistry)
	{
		FBlueprintActionDatabase::Get().RefreshAll();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, bEnableNamespaceEditorFeatures))
	{
		// Update transient settings and console variable flags to reflect the new config setting value.
		// @todo_namespaces - May be removed once dependent code is changed to utilize the config setting.
		bEnableNamespaceFilteringFeatures = bEnableNamespaceEditorFeatures;
		bEnableNamespaceImportingFeatures = bEnableNamespaceEditorFeatures;

		// Update console flags to match the current configuration.
		FBlueprintNamespaceHelper::RefreshEditorFeatureConsoleFlags();

		// Refresh the Blueprint editor UI environment to match current settings.
		FBlueprintNamespaceUtilities::RefreshBlueprintEditorFeatures();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UBlueprintEditorSettings, NamespacesToAlwaysInclude))
	{
		// Close any open Blueprint editor windows so that we have a chance to reload them with the updated import set.
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>("Kismet");
		for (const TSharedRef<IBlueprintEditor>& BlueprintEditor : BlueprintEditorModule.GetBlueprintEditors())
		{
			BlueprintEditor->CloseWindow(EAssetEditorCloseReason::EditorRefreshRequested);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBlueprintEditorSettings::RegisterIsClassAllowedDelegate(const FName OwnerName, FOnIsClassAllowed Delegate)
{
	IsClassAllowedDelegates.Add(OwnerName, Delegate);
}

void UBlueprintEditorSettings::UnregisterIsClassAllowedDelegate(const FName OwnerName)
{
	IsClassAllowedDelegates.Remove(OwnerName);
}

bool UBlueprintEditorSettings::IsClassAllowed(const UClass* InClass) const
{
	for (const TPair<FName, FOnIsClassAllowed>& DelegatePair : IsClassAllowedDelegates)
	{
		if (!DelegatePair.Value.Execute(InClass))
		{
			return false;
		}
	}
	return true;
}

void UBlueprintEditorSettings::RegisterIsClassPathAllowedDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate)
{
	IsClassPathAllowedDelegates.Add(OwnerName, Delegate);
}

void UBlueprintEditorSettings::UnregisterIsClassPathAllowedDelegate(const FName OwnerName)
{
	IsClassPathAllowedDelegates.Remove(OwnerName);
}

bool UBlueprintEditorSettings::IsClassPathAllowed(const FTopLevelAssetPath& InClassPath) const
{
	for (const TPair<FName, FOnIsClassPathAllowed>& DelegatePair : IsClassPathAllowedDelegates)
	{
		if (!DelegatePair.Value.Execute(InClassPath))
		{
			return false;
		}
	}
	return true;
}

void UBlueprintEditorSettings::RegisterIsClassAllowedOnPinDelegate(const FName OwnerName, FOnIsClassAllowed Delegate)
{
	IsClassAllowedOnPinDelegates.Add(OwnerName, Delegate);
}

void UBlueprintEditorSettings::UnregisterIsClassAllowedOnPinDelegate(const FName OwnerName)
{
	IsClassAllowedOnPinDelegates.Remove(OwnerName);
}

bool UBlueprintEditorSettings::IsClassAllowedOnPin(const UClass* InClass) const
{
	for (const TPair<FName, FOnIsClassAllowed>& DelegatePair : IsClassAllowedOnPinDelegates)
	{
		if (!DelegatePair.Value.Execute(InClass))
		{
			return false;
		}
	}
	return true;
}

void UBlueprintEditorSettings::RegisterIsClassPathAllowedOnPinDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate)
{
	IsClassPathAllowedOnPinDelegates.Add(OwnerName, Delegate);
}

void UBlueprintEditorSettings::UnregisterIsClassPathAllowedOnPinDelegate(const FName OwnerName)
{
	IsClassPathAllowedOnPinDelegates.Remove(OwnerName);
}

bool UBlueprintEditorSettings::IsClassPathAllowedOnPin(const FTopLevelAssetPath& InClassPath) const
{
	for (const TPair<FName, FOnIsClassPathAllowed>& DelegatePair : IsClassPathAllowedOnPinDelegates)
	{
		if (!DelegatePair.Value.Execute(InClassPath))
		{
			return false;
		}
	}
	return true;
}

void UBlueprintEditorSettings::RegisterIsFunctionAllowedForAssetDelegate(const FName OwnerName, FOnIsFunctionAllowedForAsset Delegate)
{
	IsFunctionAllowedForAssetDelegates.Add(OwnerName, MoveTemp(Delegate));
}

void UBlueprintEditorSettings::UnregisterIsFunctionAllowedForAssetDelegate(const FName OwnerName)
{
	IsFunctionAllowedForAssetDelegates.Remove(OwnerName);
}

bool UBlueprintEditorSettings::IsFunctionAllowed(const UBlueprint* InBlueprint, const FName InFunctionName) const
{
	return IsFunctionAllowedForPermissions(InBlueprint, InFunctionName, FunctionPermissions);
}

bool UBlueprintEditorSettings::IsFunctionAllowedForAsset(const UBlueprint* InBlueprint, const FName InFunctionName) const
{
	for (const TPair<FName, FOnIsFunctionAllowedForAsset>& DelegatePair : IsFunctionAllowedForAssetDelegates)
	{
		if (!DelegatePair.Value.Execute(InBlueprint, InFunctionName))
		{
			return false;
		}
	}
	return IsFunctionAllowed(InBlueprint, InFunctionName);
}

bool UBlueprintEditorSettings::IsFunctionAllowedForPermissions(const UBlueprint* InBlueprint, const FName InFunctionName, const FPathPermissionList& InFunctionPermissions)
{
	if (!InFunctionPermissions.HasFiltering())
	{
		return true;
	}

	if (InFunctionPermissions.PassesFilter(InFunctionName))
	{
		return true;
	}

	if (InBlueprint)
	{
		if (const UClass* NativeParentClass = FBlueprintEditorUtils::FindFirstNativeClass(InBlueprint->ParentClass))
		{
			if (const UFunction* NativeParentFunction = NativeParentClass->FindFunctionByName(InFunctionName))
			{
				if (InFunctionPermissions.PassesFilter(NativeParentFunction->GetPathName()))
				{
					return true;
				}
			}
		}
	}

	return false;
}
