// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintEventNodeSpawner.h"

#include "BlueprintNodeTemplateCache.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Event.h"
#include "K2Node_FunctionEntry.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintEventNodeSpawner)

class UObject;

#define LOCTEXT_NAMESPACE "BlueprintEventNodeSpawner"

/*******************************************************************************
 * Static UBlueprintEventNodeSpawner Helpers
 ******************************************************************************/

namespace UBlueprintEventNodeSpawnerImpl
{
	/**
	 * Helper function for removing a ghost node (and connected ghost nodes, e.g., an autogenerated super call)
	 *
	 * @param InNode			The node to start with
	 * @param InParentGraph		The graph to remove the nodes from
	 */
	static void RemoveGhostNodes(UEdGraphNode* InNode, UEdGraph* InParentGraph);
}

//------------------------------------------------------------------------------
static void UBlueprintEventNodeSpawnerImpl::RemoveGhostNodes(UEdGraphNode* InNode, UEdGraph* InParentGraph)
{
	if(InNode && InNode->IsAutomaticallyPlacedGhostNode())
	{
		// Go through all pin connections and consume any disabled nodes so we do not leave garbage.
		for (UEdGraphPin* Pin : InNode->Pins)
		{
			TArray<UEdGraphPin*> LinkedToCopy = Pin->LinkedTo;
			for (UEdGraphPin* OtherPin : LinkedToCopy)
			{
				// Break the pin link back
				OtherPin->BreakLinkTo(Pin);
				RemoveGhostNodes(OtherPin->GetOwningNode(), InParentGraph);
			}
		}

		InNode->BreakAllNodeLinks();
		InParentGraph->RemoveNode(InNode);
	}
};

/*******************************************************************************
 * UBlueprintEventNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintEventNodeSpawner* UBlueprintEventNodeSpawner::Create(UFunction const* const EventFunc, UObject* Outer/* = nullptr*/)
{
	check(EventFunc != nullptr);

	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	UBlueprintEventNodeSpawner* NodeSpawner = NewObject<UBlueprintEventNodeSpawner>(Outer);
	NodeSpawner->EventFunc = EventFunc;
	NodeSpawner->NodeClass = UK2Node_Event::StaticClass();

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	FText const FuncName = UEdGraphSchema_K2::GetFriendlySignatureName(EventFunc);
	MenuSignature.MenuName = FText::Format(LOCTEXT("EventWithSignatureName", "Event {0}"), FuncName);
	MenuSignature.Category = UK2Node_CallFunction::GetDefaultCategoryForFunction(EventFunc, LOCTEXT("AddEventCategory", "Add Event"));
	//MenuSignature.Tooltip, will be pulled from the node template
	MenuSignature.Keywords = UK2Node_CallFunction::GetKeywordsForFunction(EventFunc);
	if (MenuSignature.Keywords.IsEmpty())
	{
		MenuSignature.Keywords = FText::FromString(TEXT(" "));
	}
	MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintEventNodeSpawner* UBlueprintEventNodeSpawner::Create(TSubclassOf<UK2Node_Event> NodeClass, FName CustomEventName, UObject* Outer/* = nullptr*/)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	UBlueprintEventNodeSpawner* NodeSpawner = NewObject<UBlueprintEventNodeSpawner>(Outer);
	NodeSpawner->NodeClass       = NodeClass;
	NodeSpawner->CustomEventName = CustomEventName;

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	if (CustomEventName.IsNone())
	{
		MenuSignature.MenuName = LOCTEXT("AddCustomEvent", "Add Custom Event...");
		MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.CustomEvent_16x");
	}
	else
	{
		FText const EventName = FText::FromName(CustomEventName);
		MenuSignature.MenuName = FText::Format(LOCTEXT("EventWithSignatureName", "Event {0}"), EventName);
		MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	}
	//MenuSignature.Category, will be pulled from the node template
	//MenuSignature.Tooltip,  will be pulled from the node template 
	//MenuSignature.Keywords, will be pulled from the node template

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintEventNodeSpawner::UBlueprintEventNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
	, EventFunc(nullptr)
{
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintEventNodeSpawner::GetSpawnerSignature() const
{
	FBlueprintNodeSignature SpawnerSignature(NodeClass);
	if (IsForCustomEvent() && !CustomEventName.IsNone())
	{
		static const FName CustomSignatureKey(TEXT("CustomEvent"));
		SpawnerSignature.AddNamedValue(CustomSignatureKey, CustomEventName.ToString());
	}
	else
	{
		SpawnerSignature.AddSubObject(const_cast<UFunction*>(ToRawPtr(EventFunc)));
	}
	return SpawnerSignature;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintEventNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	check(ParentGraph != nullptr);
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);

	UK2Node_Event* EventNode = nullptr;
	if (!FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph))
	{
		// look to see if a node for this event already exists (only one node is
		// allowed per event, per blueprint)
		UK2Node_Event const* PreExistingNode = FindPreExistingEvent(Blueprint, Bindings);
		// @TODO: casting away the const is bad form!
		EventNode = const_cast<UK2Node_Event*>(PreExistingNode);
	}

	bool const bIsCustomEvent = IsForCustomEvent();
	check(bIsCustomEvent || (EventFunc != nullptr));

	FName EventName = CustomEventName;
	if (!bIsCustomEvent)
	{
		EventName  = EventFunc->GetFName();	
	}

	// If there is a function with this name on the blueprint, focus on that instead
	for (UEdGraph* FuncGraph : Blueprint->FunctionGraphs)
	{
		if (FuncGraph && FuncGraph->GetFName() == EventName)
		{
			TArray<UK2Node_FunctionEntry*> FunctionEntry;
			FuncGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntry);
			if (FunctionEntry.Num())
			{
				return FunctionEntry[0];
			}
			else
			{
				return nullptr;
			}
		}
	}

	// This Event node might already be present in the Blueprint in a disabled state, 
	// remove it and allow the user to successfully place the node where they want it.
	if(EventNode && EventNode->IsAutomaticallyPlacedGhostNode())
	{
		UBlueprintEventNodeSpawnerImpl::RemoveGhostNodes(EventNode, ParentGraph);
		EventNode = nullptr;
	}

	// if there is no existing node, then we can happily spawn one into the graph
	if (EventNode == nullptr)
	{
		auto PostSpawnLambda = [](UEdGraphNode* NewNode, bool bInIsTemplateNode, UFunction const* InEventFunc, FName InEventName, FCustomizeNodeDelegate UserDelegate)
		{
			UK2Node_Event* K2EventNode = CastChecked<UK2Node_Event>(NewNode);
			if (InEventFunc != nullptr)
			{
				K2EventNode->EventReference.SetFromField<UFunction>(InEventFunc, false);
				K2EventNode->bOverrideFunction   = true;
			}
			else if (!bInIsTemplateNode)
			{
				K2EventNode->CustomFunctionName = InEventName;
			}

			UserDelegate.ExecuteIfBound(NewNode, bInIsTemplateNode);
		};

		FCustomizeNodeDelegate PostSpawnDelegate = FCustomizeNodeDelegate::CreateStatic(PostSpawnLambda, ToRawPtr(EventFunc), EventName, CustomizeNodeDelegate);
		EventNode = Super::SpawnNode<UK2Node_Event>(NodeClass, ParentGraph, Bindings, Location, PostSpawnDelegate);
	}
	// else, a node for this event already exists, and we should return that 
	// (the FBlueprintActionMenuItem should detect this and focus in on it).

	return EventNode;
}

//------------------------------------------------------------------------------
UFunction const* UBlueprintEventNodeSpawner::GetEventFunction() const
{
	return EventFunc;
}

//------------------------------------------------------------------------------
UK2Node_Event const* UBlueprintEventNodeSpawner::FindPreExistingEvent(UBlueprint* Blueprint, FBindingSet const& /*Bindings*/) const
{
	UK2Node_Event* PreExistingNode = nullptr;

	check(Blueprint != nullptr);
	if (IsForCustomEvent())
	{
		PreExistingNode = FBlueprintEditorUtils::FindCustomEventNode(Blueprint, CustomEventName);
	}
	else
	{
		check(EventFunc != nullptr);
		UClass* ClassOwner = EventFunc->GetOwnerClass()->GetAuthoritativeClass();

		PreExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(Blueprint, ClassOwner, EventFunc->GetFName());
	}

	return PreExistingNode;
}

//------------------------------------------------------------------------------
bool UBlueprintEventNodeSpawner::IsForCustomEvent() const
{
	return (EventFunc == nullptr);
}

#undef LOCTEXT_NAMESPACE
