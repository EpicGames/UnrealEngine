// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintFieldNodeSpawner.h"

#include "EdGraph/EdGraphNode.h"
#include "K2Node.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintFieldNodeSpawner)

class UObject;

#define LOCTEXT_NAMESPACE "BlueprintFieldNodeSpawner"

//------------------------------------------------------------------------------
UBlueprintFieldNodeSpawner* UBlueprintFieldNodeSpawner::Create(TSubclassOf<UK2Node> NodeClass, FFieldVariant Field, UObject* Outer/* = nullptr*/, UClass const* OwnerClass)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}
	UBlueprintFieldNodeSpawner* NodeSpawner = NewObject<UBlueprintFieldNodeSpawner>(Outer);
	NodeSpawner->SetField(Field);
	NodeSpawner->NodeClass = NodeClass;
	NodeSpawner->OwnerClass = OwnerClass;
	
	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintFieldNodeSpawner::UBlueprintFieldNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)	
	, OwnerClass(nullptr)
	, Field(nullptr)
{
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintFieldNodeSpawner::GetSpawnerSignature() const
{
	FBlueprintNodeSignature SpawnerSignature(NodeClass);
	if (Field)
	{
		SpawnerSignature.AddSubObject(Field);
	}
	else
	{
		check(Property.Get());
		SpawnerSignature.AddKeyValue(Property->GetPathName());
	}

	return SpawnerSignature;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintFieldNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FFieldVariant InField, FSetNodeFieldDelegate SetFieldDelegate, FCustomizeNodeDelegate UserDelegate)
	{
		SetFieldDelegate.ExecuteIfBound(NewNode, InField);
		UserDelegate.ExecuteIfBound(NewNode, bIsTemplateNode);
	};

	FCustomizeNodeDelegate PostSpawnSetupDelegate = FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, GetField(), SetNodeFieldDelegate, CustomizeNodeDelegate);
	return Super::SpawnNode<UEdGraphNode>(NodeClass, ParentGraph, Bindings, Location, PostSpawnSetupDelegate);
}

//------------------------------------------------------------------------------
FFieldVariant UBlueprintFieldNodeSpawner::GetField() const
{
	return Field ? FFieldVariant(Field) : FFieldVariant(Property.Get());
}

void UBlueprintFieldNodeSpawner::SetField(FFieldVariant InField)
{
	if (InField.IsUObject())
	{
		Field = InField.Get<UField>();
	}
	else
	{
		Property = InField.Get<FProperty>();
	}
}

#undef LOCTEXT_NAMESPACE
