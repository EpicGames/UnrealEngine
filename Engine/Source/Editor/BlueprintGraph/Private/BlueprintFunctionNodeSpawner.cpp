// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintFunctionNodeSpawner.h"

#include "BlueprintActionFilter.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeSpawnerUtils.h"
#include "BlueprintNodeTemplateCache.h"
#include "BlueprintTypePromotion.h"
#include "BlueprintVariableNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CallDataTableFunction.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CallFunctionOnMember.h"
#include "K2Node_CallMaterialParameterCollectionFunction.h"
#include "K2Node_CommutativeAssociativeBinaryOperator.h"
#include "K2Node_Literal.h"
#include "K2Node_PromotableOperator.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "ObjectTools.h"
#include "SNodePanel.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/Script.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintFunctionNodeSpawner)

#define LOCTEXT_NAMESPACE "BlueprintFunctionNodeSpawner"

/*******************************************************************************
 * Static UBlueprintFunctionNodeSpawner Helpers
 ******************************************************************************/

//------------------------------------------------------------------------------
namespace BlueprintFunctionNodeSpawnerImpl
{
	FVector2D BindingOffset = FVector2D::ZeroVector;
	static const FText FallbackCategory = LOCTEXT("UncategorizedFallbackCategory", "Call Function");

	/**
	 * 
	 * 
	 * @param  NewNode	
	 * @param  BoundObject	
	 * @return 
	 */
	static bool BindFunctionNode(UK2Node_CallFunction* NewNode, FBindingObject BoundObject);

	/**
	 * 
	 * 
	 * @param  NewNode	
	 * @param  BindingSpawner	
	 * @return 
	 */
	template <class NodeType>
	static bool BindFunctionNode(UK2Node_CallFunction* NewNode, UBlueprintNodeSpawner* BindingSpawner);

	/**
	 * 
	 * 
	 * @param  InputNode
	 * @return 
	 */
	static FVector2D CalculateBindingPosition(UEdGraphNode* const InputNode);

	/**
	 * 
	 * 
	 * @param  Struct	
	 * @param  Function	
	 * @param  OperatorMetaTag	
	 * @return 
	 */
	static bool IsStructOperatorFunc(const UScriptStruct* Struct, const UFunction* Function, FName const OperatorMetaTag);
}

//------------------------------------------------------------------------------
static bool BlueprintFunctionNodeSpawnerImpl::BindFunctionNode(UK2Node_CallFunction* NewNode, FBindingObject BoundObject)
{
	bool bSuccessfulBinding = false;

	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(NewNode->GetGraph());
	if (!bIsTemplateNode)
	{
		if (FProperty const* BoundProperty = BoundObject.Get<FProperty>())
		{
			if (UK2Node_CallFunctionOnMember* CallOnMemberNode = Cast<UK2Node_CallFunctionOnMember>(NewNode))
			{
				// force bIsConsideredSelfContext to false, else the target 
				// could end up being the skeleton class (and functionally, 
				// there is no difference)
				CallOnMemberNode->MemberVariableToCallOn.SetFromField<FProperty>(BoundProperty, /*bIsConsideredSelfContext =*/false);
				bSuccessfulBinding = true;
				CallOnMemberNode->ReconstructNode();
			}
			else
			{
				UBlueprintNodeSpawner* TempNodeSpawner = UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(UK2Node_VariableGet::StaticClass(), BoundProperty);
				bSuccessfulBinding = BindFunctionNode<UK2Node_VariableGet>(NewNode, TempNodeSpawner);
			}
		}
		else if (AActor* BoundActor = BoundObject.Get<AActor>())
		{
			auto PostSpawnSetupLambda = [](UEdGraphNode* InNewNode, bool /*bIsTemplateNode*/, AActor* ActorInst)
			{
				UK2Node_Literal* ActorRefNode = CastChecked<UK2Node_Literal>(InNewNode);
				ActorRefNode->SetObjectRef(ActorInst);
			};
			UBlueprintNodeSpawner::FCustomizeNodeDelegate PostSpawnDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, BoundActor);
			
			UBlueprintNodeSpawner* TempNodeSpawner = UBlueprintNodeSpawner::Create<UK2Node_Literal>(/*Outer =*/GetTransientPackage(), PostSpawnDelegate);
			bSuccessfulBinding = BindFunctionNode<UK2Node_Literal>(NewNode, TempNodeSpawner);
		}		
	}
	return bSuccessfulBinding;
}

//------------------------------------------------------------------------------
template <class NodeType>
static bool BlueprintFunctionNodeSpawnerImpl::BindFunctionNode(UK2Node_CallFunction* NewNode, UBlueprintNodeSpawner* BindingSpawner)
{
	bool bSuccessfulBinding = false;

	FVector2D BindingPos = CalculateBindingPosition(NewNode);
	UEdGraph* ParentGraph = NewNode->GetGraph();
	NodeType* BindingNode = CastChecked<NodeType>(BindingSpawner->Invoke(ParentGraph, IBlueprintNodeBinder::FBindingSet(), BindingPos));

	BindingOffset.Y += UEdGraphSchema_K2::EstimateNodeHeight(BindingNode);

	UEdGraphPin* LiteralOutput = BindingNode->GetValuePin();
	UEdGraphPin* CallSelfInput = NewNode->FindPin(UEdGraphSchema_K2::PN_Self);
	// connect the new "get-var" node with the spawned function node
	if ((LiteralOutput != nullptr) && (CallSelfInput != nullptr))
	{
		LiteralOutput->MakeLinkTo(CallSelfInput);
		bSuccessfulBinding = true;
	}

	return bSuccessfulBinding;
}

//------------------------------------------------------------------------------
static FVector2D BlueprintFunctionNodeSpawnerImpl::CalculateBindingPosition(UEdGraphNode* const InputNode)
{
	float const EstimatedVarNodeWidth = 224.0f;
	FVector2D AttachingNodePos;
	AttachingNodePos.X = InputNode->NodePosX - EstimatedVarNodeWidth;

	float const EstimatedVarNodeHeight = 48.0f;
	float const EstimatedFuncNodeHeight = UEdGraphSchema_K2::EstimateNodeHeight(InputNode);
	float const FuncNodeMidYCoordinate = InputNode->NodePosY + (EstimatedFuncNodeHeight / 2.0f);
	AttachingNodePos.Y = FuncNodeMidYCoordinate - (EstimatedVarNodeWidth / 2.0f);

	AttachingNodePos += BindingOffset;
	return AttachingNodePos;
}

//------------------------------------------------------------------------------
static bool BlueprintFunctionNodeSpawnerImpl::IsStructOperatorFunc(const UScriptStruct* Struct, const UFunction* Function, FName const OperatorMetaTag)
{
	bool bIsOperatorFunc = false;

	FString NamedOperatorFunction = Struct->GetMetaData(OperatorMetaTag);
	if (!NamedOperatorFunction.IsEmpty())
	{
		UObject* OperatorOuter = nullptr;
		if (ResolveName(OperatorOuter, NamedOperatorFunction, /*Create =*/false, /*Throw =*/false))
		{
			if ((Function->GetOuter() == OperatorOuter) &&
				(Function->GetName()  == NamedOperatorFunction))
			{
				bIsOperatorFunc = true;
			}
		}
	}
	return bIsOperatorFunc;
}

/*******************************************************************************
 * UBlueprintFunctionNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
// Evolved from FK2ActionMenuBuilder::AddSpawnInfoForFunction()
UBlueprintFunctionNodeSpawner* UBlueprintFunctionNodeSpawner::Create(UFunction const* const Function, UObject* Outer/* = nullptr*/)
{
	check(Function != nullptr);

	bool const bIsPure = Function->HasAllFunctionFlags(FUNC_BlueprintPure);
	bool const bHasArrayPointerParms = Function->HasMetaData(FBlueprintMetadata::MD_ArrayParam);
	bool const bIsCommutativeAssociativeBinaryOp = Function->HasMetaData(FBlueprintMetadata::MD_CommutativeAssociativeBinaryOperator);
	bool const bIsMaterialParamCollectionFunc = Function->HasMetaData(FBlueprintMetadata::MD_MaterialParameterCollectionFunction);
	bool const bIsDataTableFunc = Function->HasMetaData(FBlueprintMetadata::MD_DataTablePin);

	bool const bIsPromotableFunction = TypePromoDebug::IsTypePromoEnabled() && FTypePromotion::IsFunctionPromotionReady(Function);

	TSubclassOf<UK2Node_CallFunction> NodeClass;
	if (bIsPromotableFunction)
	{
		NodeClass = UK2Node_PromotableOperator::StaticClass();
	}
	else if(bIsCommutativeAssociativeBinaryOp && bIsPure)
	{
		NodeClass = UK2Node_CommutativeAssociativeBinaryOperator::StaticClass();
	}
	else if (bIsMaterialParamCollectionFunc)
	{
		NodeClass = UK2Node_CallMaterialParameterCollectionFunction::StaticClass();
	}
	else if (bIsDataTableFunc)
	{
		NodeClass = UK2Node_CallDataTableFunction::StaticClass();
	}
	// @TODO: else if bIsParentContext => UK2Node_CallParentFunction
	else if (bHasArrayPointerParms)
	{
		NodeClass = UK2Node_CallArrayFunction::StaticClass();
	}
	else
	{
		NodeClass = UK2Node_CallFunction::StaticClass();
	}

	return Create(NodeClass, Function, Outer);
}

//------------------------------------------------------------------------------
UBlueprintFunctionNodeSpawner* UBlueprintFunctionNodeSpawner::Create(TSubclassOf<UK2Node_CallFunction> NodeClass, UFunction const* const Function, UObject* Outer/* = nullptr*/)
{
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	//--------------------------------------
	// Constructing the Spawner
	//--------------------------------------

	bool const bIsPromotableFunction =
		TypePromoDebug::IsTypePromoEnabled() &&
		FTypePromotion::IsFunctionPromotionReady(Function);

	FName OpName = FTypePromotion::GetOpNameFromFunction(Function);

	// If a spawner for this operator has been created already, than just return that
	if (bIsPromotableFunction && FTypePromotion::IsOperatorSpawnerRegistered(Function))
	{
		if (UBlueprintFunctionNodeSpawner* OpSpawner = FTypePromotion::GetOperatorSpawner(OpName))
		{
			return OpSpawner;
		}
	}

	UBlueprintFunctionNodeSpawner* NodeSpawner = NewObject<UBlueprintFunctionNodeSpawner>(Outer);
	NodeSpawner->SetField(const_cast<UFunction*>(Function));

	if (NodeClass == nullptr)
	{
		NodeSpawner->NodeClass = UK2Node_CallFunction::StaticClass();
	}
	else
	{
		NodeSpawner->NodeClass = NodeClass;
	}

	//--------------------------------------
	// Default UI Signature
	//--------------------------------------

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;

	if(bIsPromotableFunction)
	{
		MenuSignature.MenuName = FTypePromotion::GetUserFacingOperatorName(OpName);
		MenuSignature.Category = LOCTEXT("UtilityOperatorCategory", "Utilities|Operators");
		// Possibly generate some special tooltips for promotable operators?
		MenuSignature.Tooltip = FTypePromotion::GetUserFacingOperatorName(OpName);
		MenuSignature.Keywords = FTypePromotion::GetKeywordsForOperator(OpName);
		FTypePromotion::RegisterOperatorSpawner(OpName, NodeSpawner);
	}
	else
	{
		constexpr bool bAllowFriendlyNames = true;
		MenuSignature.MenuName = ObjectTools::GetUserFacingFunctionName(Function, bAllowFriendlyNames);
		MenuSignature.Category = UK2Node_CallFunction::GetDefaultCategoryForFunction(Function, FText::GetEmpty());
		MenuSignature.Tooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function));
		// add at least one character, so that PrimeDefaultUiSpec() doesn't attempt to query the template node
		MenuSignature.Keywords = UK2Node_CallFunction::GetKeywordsForFunction(Function);
	}
	
	
	if (MenuSignature.Keywords.IsEmpty())
	{
		MenuSignature.Keywords = FText::FromString(TEXT(" "));
	}

	MenuSignature.Icon = UK2Node_CallFunction::GetPaletteIconForFunction(Function, MenuSignature.IconTint);

	if (MenuSignature.Category.IsEmpty())
	{
		MenuSignature.Category = BlueprintFunctionNodeSpawnerImpl::FallbackCategory;
	}

	if (MenuSignature.Tooltip.IsEmpty() && !bIsPromotableFunction)
	{
		MenuSignature.Tooltip = MenuSignature.MenuName;
	}

	if (Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction))
	{
		MenuSignature.MenuName = FBlueprintEditorUtils::GetDeprecatedMemberMenuItemName(MenuSignature.MenuName);
	}

	//--------------------------------------
	// Post-Spawn Setup
	//--------------------------------------

	auto SetNodeFunctionLambda = [](UEdGraphNode* NewNode, FFieldVariant InField)
	{
		// user could have changed the node class (to something like
		// UK2Node_BaseAsyncTask, which also wraps a function)
		if (UK2Node_CallFunction* FuncNode = Cast<UK2Node_CallFunction>(NewNode))
		{
			FuncNode->SetFromFunction(Cast<UFunction>(InField.ToUObject()));
		}
	};
	NodeSpawner->SetNodeFieldDelegate = FSetNodeFieldDelegate::CreateStatic(SetNodeFunctionLambda);


	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintFunctionNodeSpawner::UBlueprintFunctionNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void UBlueprintFunctionNodeSpawner::Prime()
{
	// we expect that you don't need a node template to construct menu entries
	// from this, so we choose not to pre-cache one here
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec UBlueprintFunctionNodeSpawner::GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const
{
	// for FallbackCategory, IsStructOperatorFunc(), etc.
	using namespace BlueprintFunctionNodeSpawnerImpl;

	UEdGraph* TargetGraph = (Context.Graphs.Num() > 0) ? Context.Graphs[0] : nullptr;
	FBlueprintActionUiSpec MenuSignature = PrimeDefaultUiSpec(TargetGraph);

	//
	// stick uncategorized functions in either "Call Function" (for self 
	// members), or "<ClassName>|..." for external members

	FString const CategoryString = MenuSignature.Category.ToString();
	// FText compares are slow, so let's use FString compares
	bool const bIsUncategorized = (CategoryString == FallbackCategory.ToString());
	if (bIsUncategorized)
	{
		checkSlow(Context.Blueprints.Num() > 0);
		UBlueprint* TargetBlueprint = Context.Blueprints[0];

		// @TODO: this is duplicated in a couple places, move it to some shared resource
		UClass const* TargetClass = (TargetBlueprint->GeneratedClass != nullptr) ? TargetBlueprint->GeneratedClass : TargetBlueprint->ParentClass;
		for (UEdGraphPin* Pin : Context.Pins)
		{
			if ((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) &&
				Pin->PinType.PinSubCategoryObject.IsValid())
			{
				TargetClass = CastChecked<UClass>(Pin->PinType.PinSubCategoryObject.Get());
			}
		}
		
		UFunction const* WrappedFunction = GetFunction();
		checkSlow(WrappedFunction != nullptr);
		UClass* FunctionClass = WrappedFunction->GetOwnerClass()->GetAuthoritativeClass();

		if (!TargetClass || !TargetClass->IsChildOf(FunctionClass))
		{
			// When there are no bindings set, functions need to be categorized into a category "Class" to help reduce clutter in the tree root
			if(Bindings.Num() == 0)
			{
				MenuSignature.Category = FEditorCategoryUtils::BuildCategoryString( FCommonEditorCategory::Class, 
					FText::FromString(FunctionClass->GetDisplayNameText().ToString()) );
			}
			else
			{
				MenuSignature.Category = FText::FromString(FunctionClass->GetDisplayNameText().ToString());
			}
		}
	}

	//
	// bubble up important make/break functions (when dragging from a struct pin)

	for (UEdGraphPin* Pin : Context.Pins)
	{
		const UScriptStruct* PinStruct = Cast<const UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
		if (PinStruct != nullptr)
		{
			UFunction const* WrappedFunction = GetFunction();
			checkSlow(WrappedFunction != nullptr);

			bool const bIsStructOperator = IsStructOperatorFunc(PinStruct, WrappedFunction, FBlueprintMetadata::MD_NativeBreakFunction) ||
				IsStructOperatorFunc(PinStruct, WrappedFunction, FBlueprintMetadata::MD_NativeMakeFunction);

			if (bIsStructOperator)
			{
				MenuSignature.Category = LOCTEXT("EmptyFunctionCategory", "|");
				break;
			}
		}
	}

	//
	// call out functions bound to sub-component (members); give them a unique name

	if (Bindings.Num() == 1)
	{
		FObjectProperty const* ObjectProperty = Bindings.CreateConstIterator()->Get<FObjectProperty>();
		if (ObjectProperty != nullptr)
		{
			FString BoundFunctionName = FString::Printf(TEXT("%s (%s)"), *MenuSignature.MenuName.ToString(), *ObjectProperty->GetName());
			// @TODO: this should probably be an FText::Format()
			MenuSignature.MenuName = FText::FromString(BoundFunctionName);
		}
	}
	DynamicUiSignatureGetter.ExecuteIfBound(Context, Bindings, &MenuSignature);
	return MenuSignature;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintFunctionNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	auto PostSpawnSetupLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, UFunction const* Function, FSetNodeFieldDelegate SetFieldDelegate, FCustomizeNodeDelegate UserDelegate)
	{
		SetFieldDelegate.ExecuteIfBound(NewNode, const_cast<UFunction*>(Function));
		UserDelegate.ExecuteIfBound(NewNode, bIsTemplateNode);
	};
	FCustomizeNodeDelegate PostSpawnSetupDelegate = FCustomizeNodeDelegate::CreateStatic(PostSpawnSetupLambda, GetFunction(), SetNodeFieldDelegate, CustomizeNodeDelegate);

	UClass* SpawnClass = NodeClass;

	const UBlueprintEditorSettings* BPSettings = GetDefault<UBlueprintEditorSettings>();
	bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);

	bool const bSpawnCallOnMember = (Bindings.Num() == 1) && Bindings.CreateConstIterator()->Get<FObjectProperty>();
	if (bSpawnCallOnMember && (bIsTemplateNode || BPSettings->bCompactCallOnMemberNodes))
	{
		SpawnClass = UK2Node_CallFunctionOnMember::StaticClass();
	}

	// if this spawner was set up to spawn a bound node, reset this so the 
	// bound nodes get positioned properly
	BlueprintFunctionNodeSpawnerImpl::BindingOffset = FVector2D::ZeroVector;

	UEdGraphNode* SpawnedNode = Super::SpawnNode<UEdGraphNode>(SpawnClass, ParentGraph, Bindings, Location, PostSpawnSetupDelegate);
	SpawnedNode->SnapToGrid(SNodePanel::GetSnapGridSize());
	return SpawnedNode;
}

//------------------------------------------------------------------------------
bool UBlueprintFunctionNodeSpawner::CanBindMultipleObjects() const
{
	UFunction const* Function = GetFunction();
	check(Function != nullptr);
	return UK2Node_CallFunction::CanFunctionSupportMultipleTargets(Function);
}

//------------------------------------------------------------------------------
bool UBlueprintFunctionNodeSpawner::IsBindingCompatible(FBindingObject BindingCandidate) const
{
	UFunction const* Function = GetFunction();
	checkSlow(Function != nullptr);

	if ( !ensureMsgf(!FBlueprintNodeSpawnerUtils::IsStaleFieldAction(this), 
			TEXT("Invalid BlueprintFunctionNodeSpawner (for %s). Was the action database properly updated when this class was compiled?"), 
			*Function->GetOwnerClass()->GetName()) )
	{
		return false;
	}

	bool const bNodeTypeMatches = (NodeClass == UK2Node_CallFunction::StaticClass());
	bool bClassOwnerMatches = false;

	UClass* BindingClass = FBlueprintNodeSpawnerUtils::GetBindingClass(BindingCandidate)->GetAuthoritativeClass();
	if (UClass const* FuncOwner = Function->GetOwnerClass()->GetAuthoritativeClass())
	{
		bClassOwnerMatches = BindingClass && BindingClass->IsChildOf(FuncOwner);
	}

	return bNodeTypeMatches && bClassOwnerMatches && !FObjectEditorUtils::IsFunctionHiddenFromClass(Function, BindingClass);
}

//------------------------------------------------------------------------------
bool UBlueprintFunctionNodeSpawner::BindToNode(UEdGraphNode* Node, FBindingObject Binding) const
{
	return BlueprintFunctionNodeSpawnerImpl::BindFunctionNode(CastChecked<UK2Node_CallFunction>(Node), Binding);
}

//------------------------------------------------------------------------------
UFunction const* UBlueprintFunctionNodeSpawner::GetFunction() const
{
	return Cast<UFunction>(GetField().ToUObject());
}

#undef LOCTEXT_NAMESPACE
