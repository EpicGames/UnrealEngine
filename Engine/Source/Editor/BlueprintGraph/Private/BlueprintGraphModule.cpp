// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGraphModule.h"

#include "AssetBlueprintGraphActions.h"
#include "EdGraphSchema_K2.h"
#include "BlueprintTypePromotion.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE( FBlueprintGraphModule, BlueprintGraph );

void FBlueprintGraphModule::ShutdownModule()
{
	UEdGraphSchema_K2::Shutdown();
	FTypePromotion::Shutdown();
	AssetBlueprintGraphActions.Reset();
}

void FBlueprintGraphModule::RegisterGraphAction(const UClass* AssetType, TUniquePtr<FAssetBlueprintGraphActions> Action)
{	
	ensure(!AssetBlueprintGraphActions.Contains(AssetType));
	AssetBlueprintGraphActions.Add(AssetType, MoveTemp(Action));
}

void FBlueprintGraphModule::UnregisterGraphAction(const UClass* AssetType)
{
	AssetBlueprintGraphActions.Remove(AssetType);
}

const FAssetBlueprintGraphActions* FBlueprintGraphModule::GetAssetBlueprintGraphActions(const UClass* AssetType) const
{
	if (const TUniquePtr<FAssetBlueprintGraphActions>* Actions = AssetBlueprintGraphActions.Find(AssetType))
	{
		return Actions->Get();
	}
	return nullptr;
}
