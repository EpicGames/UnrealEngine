// Copyright Epic Games, Inc. All Rights Reserved.
#include "BlueprintGraphPanelPinFactory.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/CurveTable.h"
#include "Engine/DataTable.h"
#include "HAL/PlatformCrt.h"
#include "K2Node_CallFunction.h"
#include "K2Node_GetDataTableRow.h"
#include "SGraphPinDataTableRowName.h"
#include "SGraphPinNameList.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FString;
struct FRealCurve;
	
TSharedPtr<class SGraphPin> FBlueprintGraphPanelPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		UObject* Outer = InPin->GetOuter();

		// Create drop down combo boxes for DataTable and CurveTable RowName pins
		const UEdGraphPin* DataTablePin = nullptr;
		if (Outer->IsA(UK2Node_CallFunction::StaticClass()))
		{
			const UK2Node_CallFunction* CallFunctionNode = CastChecked<UK2Node_CallFunction>(Outer);
			if (CallFunctionNode)
			{
				const UFunction* FunctionToCall = CallFunctionNode->GetTargetFunction();
				if (FunctionToCall)
				{
					const FString& DataTablePinName = FunctionToCall->GetMetaData(FBlueprintMetadata::MD_DataTablePin);
					DataTablePin = CallFunctionNode->FindPin(DataTablePinName);
				}
			}
		}
		else if (Outer->IsA(UK2Node_GetDataTableRow::StaticClass()))
		{
			const UK2Node_GetDataTableRow* GetDataTableRowNode = CastChecked<UK2Node_GetDataTableRow>(Outer);
			DataTablePin = GetDataTableRowNode->GetDataTablePin();
		}

		if (DataTablePin)
		{
			if (DataTablePin->DefaultObject != nullptr && DataTablePin->LinkedTo.Num() == 0)
			{
				if (auto DataTable = Cast<UDataTable>(DataTablePin->DefaultObject))
				{
					return SNew(SGraphPinDataTableRowName, InPin, DataTable);
				}
				if (DataTablePin->DefaultObject->IsA(UCurveTable::StaticClass()))
				{
					UCurveTable* CurveTable = (UCurveTable*)DataTablePin->DefaultObject;
					if (CurveTable)
					{
						TArray<TSharedPtr<FName>> RowNames;
						/** Extract all the row names from the RowMap */
						for (TMap<FName, FRealCurve*>::TConstIterator Iterator(CurveTable->GetRowMap()); Iterator; ++Iterator)
						{
							/** Create a simple array of the row names */
							TSharedPtr<FName> RowNameItem = MakeShareable(new FName(Iterator.Key()));
							RowNames.Add(RowNameItem);
						}
						return SNew(SGraphPinNameList, InPin, RowNames);
					}
				}
			}
		}
	}

	return nullptr;
}
