// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeBinder.h"

void FBindingObject::AddStructReferencedObjects(FReferenceCollector& Collector)
{
	if (bIsUObject)
	{
		Collector.AddReferencedObject(Object);
	}
	else if (FField* CurrentField = Get<FField>())
	{
		CurrentField->AddReferencedObjects(Collector);
	}
}
