// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeSignature.h"

#include "Containers/Array.h"
#include "Containers/StringConv.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/CString.h"
#include "Misc/SecureHash.h"
#include "Templates/Tuple.h"

/*******************************************************************************
 * Static FBlueprintNodeSignature Helpers
 ******************************************************************************/

namespace BlueprintNodeSignatureImpl
{
	static FString const SignatureOpeningStr("(");
	static FString const SignatureElementDelim(",");
	static FString const SignatureClosingStr(")");
	static FString const SignatureKeyDelim("=");

	static FName const LegacySubObjSignatureKey(TEXT("FieldName"));

	/**
	 * Iterates through the given KeyMap and looks for name collisions. Will 
	 * append an incrementing value until a collision is not detected.
	 * 
	 * @param  BaseName	The base name you want some permutation of.
	 * @param  KeyMap	The map you want the name key for.
	 * @return A unique name key for the supplied map.
	 */
	static FName FindUniqueKeyName(FName BaseName, TMap<FName, FString>& KeyMap);
}

//------------------------------------------------------------------------------
static FName BlueprintNodeSignatureImpl::FindUniqueKeyName(FName BaseName, TMap<FName, FString>& KeyMap)
{
	FName SingatureKey = BaseName;

	int32 FNameIndex = 0;
	while (KeyMap.Find(SingatureKey) != nullptr)
	{
		SingatureKey = FName(BaseName, ++FNameIndex);
	}
	return SingatureKey;
}

/*******************************************************************************
 * FBlueprintNodeSignature
 ******************************************************************************/

//------------------------------------------------------------------------------
FBlueprintNodeSignature::FBlueprintNodeSignature(FString const& UserString)
{
	using namespace BlueprintNodeSignatureImpl;

	FString SanitizedSignature = UserString;
	SanitizedSignature.RemoveFromStart(SignatureOpeningStr);
	SanitizedSignature.RemoveFromEnd(SignatureClosingStr);

	TArray<FString> SignatureElements;
	SanitizedSignature.ParseIntoArray(SignatureElements, *SignatureElementDelim, 1);

	for (FString& SignatureElement : SignatureElements)
	{
		SignatureElement.TrimStartInline();

		FString SignatureKey, SignatureValue;
		SignatureElement.Split(SignatureKeyDelim, &SignatureKey, &SignatureValue);
		// @TODO: look for UObject redirects with SignatureValue

		SignatureValue.RemoveFromStart(TEXT("\""), ESearchCase::CaseSensitive);
		SignatureValue.RemoveFromEnd(TEXT("\""), ESearchCase::CaseSensitive);
		AddNamedValue(FName(*SignatureKey), SignatureValue);
	}
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature::FBlueprintNodeSignature(TSubclassOf<UEdGraphNode> NodeClass)
{
	SetNodeClass(NodeClass);
}

//------------------------------------------------------------------------------
void FBlueprintNodeSignature::SetNodeClass(TSubclassOf<UEdGraphNode> NodeClass)
{
	static const FName NodeClassSignatureKey(TEXT("NodeName"));

	if (NodeClass != nullptr)
	{
		AddNamedValue(NodeClassSignatureKey, NodeClass->GetPathName());
	}
	else
	{
		SignatureSet.Remove(NodeClassSignatureKey);
		MarkDirty();
	}
}

//------------------------------------------------------------------------------
void FBlueprintNodeSignature::AddSubObject(FFieldVariant SignatureObj)
{
	// not ideal for generic "objects", but we have to keep in line with the 
	// old favorites system (for backwards compatibility)
	using namespace BlueprintNodeSignatureImpl;
	FName SubObjectSignatureKey = FindUniqueKeyName(LegacySubObjSignatureKey, SignatureSet);

	AddNamedValue(SubObjectSignatureKey, SignatureObj.GetPathName());
}

//------------------------------------------------------------------------------
void FBlueprintNodeSignature::AddKeyValue(FString const& KeyValue)
{
	// not ideal for some arbitrary value, but we have to keep in line with the 
	// old favorites system (for backwards compatibility)
	using namespace BlueprintNodeSignatureImpl;
	FName SignatureKey = FindUniqueKeyName(LegacySubObjSignatureKey, SignatureSet);

	AddNamedValue(SignatureKey, KeyValue);
}

//------------------------------------------------------------------------------
void FBlueprintNodeSignature::AddNamedValue(FName SignatureKey, FString const& Value)
{
	SignatureSet.Add(SignatureKey, Value);
	MarkDirty();
}

//------------------------------------------------------------------------------
bool FBlueprintNodeSignature::IsValid() const
{
	return (SignatureSet.Num() > 0);
}

//------------------------------------------------------------------------------
FString const& FBlueprintNodeSignature::ToString() const
{
	using namespace BlueprintNodeSignatureImpl;

	if (CachedSignatureString.IsEmpty() && IsValid())
	{
		TArray<FString> SignatureElements;
		for (auto& SignatureElement : SignatureSet)
		{
			FString ObjSignature = SignatureElement.Key.ToString() + SignatureKeyDelim + TEXT("\"") + SignatureElement.Value + TEXT("\"");
			SignatureElements.Add(ObjSignature);
		}
		SignatureElements.Sort();

		CachedSignatureString = SignatureOpeningStr;
		for (FString& SignatureElement : SignatureElements)
		{
			CachedSignatureString += SignatureElement + SignatureElementDelim;
		}
		CachedSignatureString.RemoveFromEnd(SignatureElementDelim);
		CachedSignatureString += SignatureClosingStr;
	}
	
	return CachedSignatureString;
}

//------------------------------------------------------------------------------
FGuid const& FBlueprintNodeSignature::AsGuid() const
{
	static const int32 BytesPerMd5Hash = 16;
	static const int32 BytesPerGuidVal =  4;
	static const int32 MembersPerGuid  =  4;
	static const int32 BitsPerByte     =  8;

	if (!CachedSignatureGuid.IsValid() && IsValid())
	{
		FString const& SignatureString = ToString();

		FMD5  Md5Gen;
		uint8 HashedBytes[BytesPerMd5Hash];
		Md5Gen.Update((unsigned char*)TCHAR_TO_ANSI(*SignatureString), SignatureString.Len());
		Md5Gen.Final(HashedBytes);
		//FString HashedStr = FMD5::HashAnsiString(*SignatureString); // to check against

		for (int32 GuidIndex = 0; GuidIndex < MembersPerGuid; ++GuidIndex)
		{
			int32 const Md5UpperIndex = (GuidIndex + 1) * BytesPerGuidVal - 1;
			int32 const Md5LowerIndex = (Md5UpperIndex - BytesPerGuidVal) + 1;

			int32 GuidValue = 0;

			int32 BitOffset = 1;
			for (int32 Md5ByteIndex = Md5UpperIndex; Md5ByteIndex >= Md5LowerIndex; --Md5ByteIndex)
			{
				GuidValue += HashedBytes[Md5ByteIndex] * BitOffset;
				BitOffset <<= BitsPerByte;
			}
			CachedSignatureGuid[GuidIndex] = GuidValue;
		}
	}
	return CachedSignatureGuid;
}

//------------------------------------------------------------------------------
void FBlueprintNodeSignature::MarkDirty()
{
	CachedSignatureGuid.Invalidate();
	CachedSignatureString.Empty();
}
