// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeSpawner.h"

#include "BlueprintNodeSpawnerUtils.h"
#include "BlueprintNodeTemplateCache.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Styling/AppStyle.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintNodeSpawner)

/*******************************************************************************
 * Static UBlueprintNodeSpawner Helpers
 ******************************************************************************/

namespace BlueprintNodeSpawnerImpl
{
	/**
	 * Retrieves a singleton like instance of FBlueprintNodeTemplateCache, will
	 * spawn one if 
	 * 
	 * @return 
	 */
	FBlueprintNodeTemplateCache* GetSharedTemplateCache(bool const bNoInit = false);
}

//------------------------------------------------------------------------------
FBlueprintNodeTemplateCache* BlueprintNodeSpawnerImpl::GetSharedTemplateCache(bool const bNoInit)
{
	static FBlueprintNodeTemplateCache* NodeTemplateManager = nullptr;
	if (!bNoInit && (NodeTemplateManager == nullptr))
	{
		NodeTemplateManager = new FBlueprintNodeTemplateCache();
	}
	return NodeTemplateManager;
}

/*******************************************************************************
 * UBlueprintNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintNodeSpawner* UBlueprintNodeSpawner::Create(TSubclassOf<UEdGraphNode> const NodeClass, UObject* Outer/* = nullptr*/, FCustomizeNodeDelegate CustomizeNodeDelegate/* = FCustomizeNodeDelegate()*/)
{
	check(NodeClass != nullptr);
	check(NodeClass->IsChildOf<UEdGraphNode>());

	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}
	
	UBlueprintNodeSpawner* NodeSpawner = NewObject<UBlueprintNodeSpawner>(Outer);
	NodeSpawner->NodeClass = NodeClass;
	NodeSpawner->CustomizeNodeDelegate = CustomizeNodeDelegate;
	
	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintNodeSpawner::UBlueprintNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void UBlueprintNodeSpawner::BeginDestroy()
{
	ClearCachedTemplateNode();
	Super::BeginDestroy();
}

//------------------------------------------------------------------------------
void UBlueprintNodeSpawner::Prime()
{
	if (FBlueprintNodeSpawnerUtils::IsStaleFieldAction(this))
	{
		const UFunction* AssociatedFunction = FBlueprintNodeSpawnerUtils::GetAssociatedFunction(this);
		const FProperty* AssociatedProperty = FBlueprintNodeSpawnerUtils::GetAssociatedProperty(this);
		ensureMsgf(false, TEXT("Priming invalid BlueprintActionDatabase entry (for %s). Was the database properly updated when this class was compiled?"), 
			AssociatedFunction ? *AssociatedFunction->GetPathName() : *AssociatedProperty->GetPathName());
		return;
	}

	if (UEdGraphNode* CachedTemplateNode = GetTemplateNode())
	{
		// since we're priming incrementally, someone could have already
		// requested this template, and allocated its pins (don't need to do 
		// redundant work)
		if (CachedTemplateNode->Pins.Num() == 0)
		{
			// in certain scenarios we need pin information from the 
			// spawner (to help filter by pin context)
			CachedTemplateNode->AllocateDefaultPins();
		}
	}
	PrimeDefaultUiSpec();
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec const& UBlueprintNodeSpawner::PrimeDefaultUiSpec(UEdGraph* TargetGraph) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintNodeSpawner::PrimeDefaultUiSpec);

	bool bTemplateNodeFetched = false;
	UEdGraphNode* NodeTemplate = nullptr;
	// @TODO: boo! const cast... all to make this callable from GetUiSpec()
	FBlueprintActionUiSpec& MenuSignature = const_cast<FBlueprintActionUiSpec&>(DefaultMenuSignature);

	//--------------------------------------
	// Verify MenuName
	//--------------------------------------

	if (MenuSignature.MenuName.IsEmpty())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		if (NodeTemplate != nullptr)
		{
			MenuSignature.MenuName = NodeTemplate->GetNodeTitle(ENodeTitleType::MenuTitle);
		}
		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (MenuSignature.MenuName.IsEmpty() && (TargetGraph != nullptr))
		{
			MenuSignature.MenuName = FText::FromName(GetFName());
		}
		bTemplateNodeFetched = true;
	}

	//--------------------------------------
	// Verify Category
	//--------------------------------------

	if (MenuSignature.Category.IsEmpty())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		// @TODO: consider moving GetMenuCategory() up into UEdGraphNode
		if (UK2Node* K2ReferenceNode = Cast<UK2Node>(NodeTemplate))
		{
			MenuSignature.Category = K2ReferenceNode->GetMenuCategory();
		}
		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (MenuSignature.Category.IsEmpty() && (TargetGraph != nullptr))
		{
			// want to set it to something so we won't end up back in this condition
			MenuSignature.Category = NSLOCTEXT("BlueprintNodeSpawner", "EmptyMenuCategory", "|");
		}
		bTemplateNodeFetched = true;
	}

	//--------------------------------------
	// Verify Tooltip
	//--------------------------------------

	if (MenuSignature.Tooltip.IsEmpty())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		if (NodeTemplate != nullptr)
		{
			MenuSignature.Tooltip = NodeTemplate->GetTooltipText();
		}
		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (MenuSignature.Tooltip.IsEmpty() && (TargetGraph != nullptr))
		{
			MenuSignature.Tooltip = MenuSignature.MenuName;
		}
		bTemplateNodeFetched = true;
	}

	//--------------------------------------
	// Verify Search Keywords
	//--------------------------------------

	if (MenuSignature.Keywords.IsEmpty())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		if (NodeTemplate != nullptr)
		{
			MenuSignature.Keywords = NodeTemplate->GetKeywords();
		}
		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (MenuSignature.Keywords.IsEmpty() && (TargetGraph != nullptr))
		{
			// want to set it to something so we won't end up back in this condition
			MenuSignature.Keywords = FText::FromString(TEXT(" "));
		}
		bTemplateNodeFetched = true;
	}

	//--------------------------------------
	// Verify Icon Brush Name
	//--------------------------------------

	if (!MenuSignature.Icon.IsSet())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		if (NodeTemplate != nullptr)
		{
			MenuSignature.Icon = NodeTemplate->GetIconAndTint(MenuSignature.IconTint);
		}
		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (!MenuSignature.Icon.IsSet() && (TargetGraph != nullptr))
		{
			// want to set it to something so we won't end up back in this condition
			MenuSignature.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Default_16x");
		}
		bTemplateNodeFetched = true;
	}

	//--------------------------------------
	// Verify Documentation Link
	//--------------------------------------

	if (MenuSignature.DocExcerptTag.IsEmpty())
	{
		NodeTemplate = bTemplateNodeFetched ? NodeTemplate : GetTemplateNode(TargetGraph);
		if (NodeTemplate != nullptr)
		{
			MenuSignature.DocLink = NodeTemplate->GetDocumentationLink();
			MenuSignature.DocExcerptTag = NodeTemplate->GetDocumentationExcerptName();
		}

		// if a target graph was provided, then we've done all we can to spawn a
		// template node... we have to default to something
		if (MenuSignature.DocExcerptTag.IsEmpty() && (TargetGraph != nullptr))
		{
			// want to set it to something so we won't end up back in this condition
			MenuSignature.DocExcerptTag = TEXT(" ");
		}
		bTemplateNodeFetched = true;
	}

	return MenuSignature;
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintNodeSpawner::GetSpawnerSignature() const
{
	FBlueprintNodeSignature SpawnerSignature;
	if (UK2Node const* NodeTemplate = Cast<UK2Node>(GetTemplateNode()))
	{
		SpawnerSignature = NodeTemplate->GetSignature();
	}

	if (!SpawnerSignature.IsValid())
	{
		SpawnerSignature.SetNodeClass(NodeClass);
	}
	return SpawnerSignature;
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec UBlueprintNodeSpawner::GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintNodeSpawner::GetUiSpec);

	FBlueprintActionUiSpec MenuSignature = PrimeDefaultUiSpec();
	DynamicUiSignatureGetter.ExecuteIfBound(Context, Bindings, &MenuSignature);
	return MenuSignature;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	return SpawnNode<UEdGraphNode>(NodeClass, ParentGraph, Bindings, Location, CustomizeNodeDelegate);
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintNodeSpawner::GetCachedTemplateNode() const
{
	return BlueprintNodeSpawnerImpl::GetSharedTemplateCache()->GetNodeTemplate(this, NoInit);
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintNodeSpawner::GetTemplateNode(UEdGraph* TargetGraph, FBindingSet const& Bindings) const
{       
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintNodeSpawner::GetTemplateNode);
	
	UEdGraphNode* TemplateNode = BlueprintNodeSpawnerImpl::GetSharedTemplateCache()->GetNodeTemplate(this, TargetGraph);

	if (TemplateNode && Bindings.Num() > 0) 
	{ 
		UEdGraphNode* BoundTemplateNode = DuplicateObject(TemplateNode, TemplateNode->GetOuter()); 
		ApplyBindings(BoundTemplateNode, Bindings); 
		return BoundTemplateNode; 
	} 
	return TemplateNode; 
}

//------------------------------------------------------------------------------
void UBlueprintNodeSpawner::ClearCachedTemplateNode() const
{
	FBlueprintNodeTemplateCache* TemplateCache = BlueprintNodeSpawnerImpl::GetSharedTemplateCache(/*bNoInit =*/true);
	if (TemplateCache != nullptr)
	{
		TemplateCache->ClearCachedTemplate(this);
	}
}

bool UBlueprintNodeSpawner::IsTemplateNodeFilteredOut(FBlueprintActionFilter const& Filter) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBlueprintNodeSpawner::IsTemplateNodeFilteredOut);

	bool bIsFilteredOut = false;
	if(UK2Node* NodeTemplate = Cast<UK2Node>(GetTemplateNode()))
	{
		bIsFilteredOut = NodeTemplate->IsActionFilteredOut(Filter);
	}
	return bIsFilteredOut;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintNodeSpawner::SpawnEdGraphNode(TSubclassOf<UEdGraphNode> InNodeClass, UEdGraph* ParentGraph, const FBindingSet& Bindings, FVector2D Location, FCustomizeNodeDelegate PostSpawnDelegate) const
{
	UEdGraphNode* NewNode = nullptr;
	if (InNodeClass != nullptr)
	{
		NewNode = NewObject<UEdGraphNode>(ParentGraph, InNodeClass);
		check(NewNode != nullptr);
		NewNode->CreateNewGuid();

		// position the node before invoking PostSpawnDelegate (in case it 
		// wishes to modify this positioning)
		NewNode->NodePosX = static_cast<int32>(Location.X);
		NewNode->NodePosY = static_cast<int32>(Location.Y);

		bool const bIsTemplateNode = FBlueprintNodeTemplateCache::IsTemplateOuter(ParentGraph);
		PostSpawnDelegate.ExecuteIfBound(NewNode, bIsTemplateNode);

		if (!bIsTemplateNode)
		{
			NewNode->SetFlags(RF_Transactional);
			NewNode->AllocateDefaultPins();
			NewNode->PostPlacedNewNode();

			ParentGraph->Modify();
			// the FBlueprintMenuActionItem should do the selecting
			ParentGraph->AddNode(NewNode, /*bFromUI =*/true, /*bSelectNewNode =*/false);
		}

		ApplyBindings(NewNode, Bindings);
	}

	return NewNode;
}
