// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeSpawnerUtils.h"

#include "BlueprintBoundEventNodeSpawner.h"
#include "BlueprintDelegateNodeSpawner.h"
#include "BlueprintEventNodeSpawner.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintVariableNodeSpawner.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

//------------------------------------------------------------------------------
FFieldVariant FBlueprintNodeSpawnerUtils::GetAssociatedField(UBlueprintNodeSpawner const* BlueprintAction)
{
	FFieldVariant ClassField;

	if (UFunction const* Function = GetAssociatedFunction(BlueprintAction))
	{
		ClassField = Function;
	}
	else if (FProperty const* Property = GetAssociatedProperty(BlueprintAction))
	{
		ClassField = Property;
	}
	// @TODO: have to fix up some of the filter cases to ignore structs/enums
// 	else if (UBlueprintFieldNodeSpawner const* FieldNodeSpawner = Cast<UBlueprintFieldNodeSpawner>(BlueprintAction))
// 	{
// 		ClassField = FieldNodeSpawner->GetField();
// 	}
	return ClassField;
}

//------------------------------------------------------------------------------
UFunction const* FBlueprintNodeSpawnerUtils::GetAssociatedFunction(UBlueprintNodeSpawner const* BlueprintAction)
{
	UFunction const* Function = nullptr;

	if (UBlueprintFunctionNodeSpawner const* FuncNodeSpawner = Cast<UBlueprintFunctionNodeSpawner>(BlueprintAction))
	{
		Function = FuncNodeSpawner->GetFunction();
	}
	else if (UBlueprintEventNodeSpawner const* EventSpawner = Cast<UBlueprintEventNodeSpawner>(BlueprintAction))
	{
		Function = EventSpawner->GetEventFunction();
	}

	return Function;
}

//------------------------------------------------------------------------------
FProperty const* FBlueprintNodeSpawnerUtils::GetAssociatedProperty(UBlueprintNodeSpawner const* BlueprintAction)
{
	FProperty const* Property = nullptr;

	if (UBlueprintDelegateNodeSpawner const* PropertySpawner = Cast<UBlueprintDelegateNodeSpawner>(BlueprintAction))
	{
		Property = PropertySpawner->GetDelegateProperty();
	}
	else if (UBlueprintVariableNodeSpawner const* VarSpawner = Cast<UBlueprintVariableNodeSpawner>(BlueprintAction))
	{
		Property = VarSpawner->GetVarProperty();
	}
	else if (UBlueprintBoundEventNodeSpawner const* BoundSpawner = Cast<UBlueprintBoundEventNodeSpawner>(BlueprintAction))
	{
		Property = BoundSpawner->GetEventDelegate();
	}
	return Property;
}

//------------------------------------------------------------------------------
UClass* FBlueprintNodeSpawnerUtils::GetBindingClass(FBindingObject Binding)
{
	UClass* BindingClass = nullptr;
	if (Binding.IsUObject())
	{
		BindingClass = Binding.Get<UObject>()->GetClass();
	}
	else if (const FObjectProperty* ObjProperty = Binding.Get<FObjectProperty>())
	{
		BindingClass = ObjProperty->PropertyClass;
	}
	else
	{
		check(false); // Can the binding object be an FProperty?
	}
	return BindingClass;
}

//------------------------------------------------------------------------------
bool FBlueprintNodeSpawnerUtils::IsStaleFieldAction(UBlueprintNodeSpawner const* BlueprintAction)
{
	bool bHasStaleAssociatedField= false;
	UClass* ClassOwner = nullptr;
	
	if (const UFunction* AssociatedFunction = GetAssociatedFunction(BlueprintAction))
	{
		ClassOwner = AssociatedFunction->GetOwnerClass();
	}
	else if (const FProperty* AssociatedProperty = GetAssociatedProperty(BlueprintAction))
	{
		ClassOwner = AssociatedProperty->GetOwnerClass();
	}
	
	if (ClassOwner != nullptr)
	{
		// check to see if this field belongs to a TRASH or REINST class,
		// maybe to a class that was thrown out because of a hot-reload?
		bHasStaleAssociatedField = ClassOwner->HasAnyClassFlags(CLASS_NewerVersionExists) || (ClassOwner->GetOutermost() == GetTransientPackage());
	}

	return bHasStaleAssociatedField;
}
