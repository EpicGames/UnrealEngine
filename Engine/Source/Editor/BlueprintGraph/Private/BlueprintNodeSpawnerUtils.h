// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeBinder.h"
#include "UObject/Field.h"

class FProperty;
class UBlueprintNodeSpawner;
class UClass;
class UFunction;

struct FBlueprintNodeSpawnerUtils
{
	/**
	 * Certain node-spawners are associated with specific UFields (functions, 
	 * properties, etc.). This attempts to retrieve the field from the spawner.
	 * 
	 * @param  BlueprintAction	The node-spawner you want an associated field for.
	 * @return The action's associated field (null if it doesn't have one).
	 */
	static FFieldVariant GetAssociatedField(UBlueprintNodeSpawner const* BlueprintAction);

	/**
	 * Certain node-spawners are associated with specific UFunctions (call-
	 * function, and event spawners). This attempts to retrieve the function 
	 * from the spawner.
	 * 
	 * @param  BlueprintAction	The node-spawner you want an associated function for.
	 * @return The action's associated function (null if it doesn't have one).
	 */
	static UFunction const* GetAssociatedFunction(UBlueprintNodeSpawner const* BlueprintAction);

	/**
	 * Certain node-spawners are associated with specific UProperties (get/set 
	 * nodes, delegates, etc.). This attempts to retrieve a property from the
	 * the spawner.
	 * 
	 * @param  BlueprintAction	The node-spawner you want an associated property for.
	 * @return The action's associated property (null if it doesn't have one).
	 */
	static FProperty const* GetAssociatedProperty(UBlueprintNodeSpawner const* BlueprintAction);

	/**
	 * Utility function to pull UClass info from a tentative binding object 
	 * (defaults to the object's class, uses the PropertyClass if the object is
	 * an object property).
	 * 
	 * @param  Binding	The binding you want a class for.
	 * @return A UClass that corresponds to the supplied binding.
	 */
	static UClass* GetBindingClass(FBindingObject Binding);

	/**
	 * Checks if the node-spawner's associated action is stale (meaning it 
	 * belongs to a TRASH or REINST class).
	 * 
	 * @param  BlueprintAction    The node-spawner you want to check.
	 * @return True if the action is stale (associated with a TRASH or REINST class).
	 */
	static bool IsStaleFieldAction(UBlueprintNodeSpawner const* BlueprintAction);
};

