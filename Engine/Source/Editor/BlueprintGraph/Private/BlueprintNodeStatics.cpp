// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeStatics.h"

#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "FBlueprintNodeStatics"

UEdGraphPin* FBlueprintNodeStatics::CreateSelfPin(UK2Node* Node, const UFunction* Function)
{
	// Chase up the function's Super chain, the function can be called on any object that is at least that specific
	const UFunction* FirstDeclaredFunction = Function;
	while (FirstDeclaredFunction->GetSuperFunction() != nullptr)
	{
		FirstDeclaredFunction = FirstDeclaredFunction->GetSuperFunction();
	}

	// Create the self pin
	UClass* FunctionClass = CastChecked<UClass>(FirstDeclaredFunction->GetOuter());
	// we don't want blueprint-function target pins to be formed from the
	// skeleton class (otherwise, they could be incompatible with other pins
	// that represent the same type)... this here could lead to a compiler 
	// warning (the GeneratedClass could not have the function yet), but in
	// that, the user would be reminded to compile the other blueprint
	if (FunctionClass->ClassGeneratedBy)
	{
		FunctionClass = FunctionClass->GetAuthoritativeClass();
	}

	UEdGraphPin* SelfPin = NULL;
	if (FunctionClass == Node->GetBlueprint()->GeneratedClass)
	{
		// This means the function is defined within the blueprint, so the pin should be a true "self" pin
		SelfPin = Node->CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UEdGraphSchema_K2::PSC_Self, nullptr, UEdGraphSchema_K2::PN_Self);
	}
	else if (FunctionClass->IsChildOf(UInterface::StaticClass()))
	{
		SelfPin = Node->CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Interface, FunctionClass, UEdGraphSchema_K2::PN_Self);
	}
	else
	{
		// This means that the function is declared in an external class, and should reference that class
		SelfPin = Node->CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, FunctionClass, UEdGraphSchema_K2::PN_Self);
	}
	check(SelfPin != nullptr);

	return SelfPin;
}

bool FBlueprintNodeStatics::CreateParameterPinsForFunction(UK2Node* Node, const UFunction* Function, TFunctionRef<void(UEdGraphPin* /*Pin*/)> PostParameterPinCreatedCallback)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Create the inputs and outputs
	bool bAllPinsGood = true;
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Param = *PropIt;
		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_ReturnParm) && (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm));
		const bool bIsRefParam = Param->HasAnyPropertyFlags(CPF_ReferenceParm) && bIsFunctionInput;

		const EEdGraphPinDirection Direction = bIsFunctionInput ? EGPD_Input : EGPD_Output;

		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.bIsReference = bIsRefParam;
		UEdGraphPin* Pin = Node->CreatePin(Direction, NAME_None, Param->GetFName(), PinParams);
		const bool bPinGood = (Pin && K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType));

		if (bPinGood)
		{
			// Check for a display name override
			const FString& PinDisplayName = Param->GetMetaData(FBlueprintMetadata::MD_DisplayName);
			if (!PinDisplayName.IsEmpty())
			{
				Pin->PinFriendlyName = FText::FromString(PinDisplayName);
			}
			else if (Function->GetReturnProperty() == Param && Function->HasMetaData(FBlueprintMetadata::MD_ReturnDisplayName))
			{
				Pin->PinFriendlyName = Function->GetMetaDataText(FBlueprintMetadata::MD_ReturnDisplayName);
			}

			//Flag pin as read only for const reference property
			Pin->bDefaultValueIsIgnored = Param->HasAllPropertyFlags(CPF_ConstParm | CPF_ReferenceParm) && (!Function->HasMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm) || Pin->PinType.IsContainer());

			Pin->bAdvancedView = Param->HasAllPropertyFlags(CPF_AdvancedDisplay);

			FString ParamValue;
			if (K2Schema->FindFunctionParameterDefaultValue(Function, Param, ParamValue))
			{
				K2Schema->SetPinAutogeneratedDefaultValue(Pin, ParamValue);
			}
			else
			{
				K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
			}

			PostParameterPinCreatedCallback(Pin);
		}

		bAllPinsGood = bAllPinsGood && bPinGood;
	}

	return bAllPinsGood;
}

#undef LOCTEXT_NAMESPACE