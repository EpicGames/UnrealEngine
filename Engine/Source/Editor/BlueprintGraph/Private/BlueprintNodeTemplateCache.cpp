// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintNodeTemplateCache.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/Tuple.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/MetaData.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"

class UEdGraphSchema;

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintNodeCache, Log, All);

/*******************************************************************************
 * Static FBlueprintNodeTemplateCache Helpers
 ******************************************************************************/

namespace BlueprintNodeTemplateCacheImpl
{
	/** */
	static int64 ActiveMemFootprint   = 0;
	/** Used to track the average blueprint size (so that we can try to predict when a blueprint would fail to be cached) */
	static int32 MadeBlueprintCount   = 0;
	static int32 AverageBlueprintSize = 0;
	
	/** Metadata tag used to identify graphs created by this system. */
	static const FName TemplateGraphMetaTag(TEXT("NodeTemplateCache_Graph"));

	/**
	 * Checks to see if this node is compatible with the given graph (to know if
	 * a node template can be spawned within it).
	 * 
	 * @param  NodeObj	The CDO of the node you want to spawn.
	 * @param  Graph	The graph you want to check compatibility for.
	 * @return True if the node and graph are compatible (for templating purposes).
	 */
	static bool IsCompatible(UEdGraphNode* NodeObj, UEdGraph* Graph);

	/**
	 * Looks through a blueprint for compatible graphs (ones that the specified
	 * node can spawn into).
	 * 
	 * @param  BlueprintOuter	The blueprint to search through.
	 * @param  NodeObj			The CDO of the node you want to spawn.
	 * @param  IsCompatibleFunc	An option callback function to further filter out incompatible nodes.
	 * @return The first compatible graph found (null if no graph was found).
	 */
	static UEdGraph* FindCompatibleGraph(UBlueprint* BlueprintOuter, UEdGraphNode* NodeObj, bool(*IsCompatibleFunc)(UEdGraph*) = nullptr);

	/**
	 * Creates a transient, temporary blueprint. Intended to be used as a 
	 * template-node's outer (grandparent).
	 * 
	 * @param  BlueprintClass		The class of blueprint to make.
	 * @param  ParentClass			The class type of blueprint to make (actor, object, etc.).
	 * @param  GeneratedClassType	The type of class that the blueprint should generate.
	 * @return A newly spawned (transient) blueprint.
	 */
	static UBlueprint* MakeCompatibleBlueprint(TSubclassOf<UBlueprint> BlueprintClass, UClass* ParentClass, TSubclassOf<UBlueprintGeneratedClass> GeneratedClassType);

	/**
	 * Creates a new transient graph, for template node use (meant to be used as
	 * a template node's outer).
	 * 
	 * @param  BlueprintOuter	The blueprint to nest the new graph under.
	 * @param  SchemaClass		The schema to assign the new graph.
	 * @return A newly created graph.
	 */
	static UEdGraph* AddGraph(UBlueprint* BlueprintOuter, TSubclassOf<UEdGraphSchema> SchemaClass);

	/**
	 * Adds metadata to the supplied graph, flagging it as a graph belonging to 
	 * the BlueprintNodeTemplateCache (so we can easily identify it later on).
	 * 
	 * @param  NewGraph	The graph to flag.
	 */
	static void MarkGraphForTemplateUse(UEdGraph* NewGraph);

	/**
	 * Determines if the specified graph is one that was allocated by 
	 * BlueprintNodeTemplateCache (to house template nodes).
	 * 
	 * @param  ParentGraph	The graph you want checked.
	 * @return True if this graph belongs to a BlueprintNodeTemplateCache, false if not.
	 */
	static bool IsTemplateOuter(UEdGraph* ParentGraph);

	/**
	 * Converts the cache memory cap into bytes (form user settings).
	 * 
	 * @return The user defined cache cap (in bytes).
	 */
	static int32 GetCacheCapSize();

	/**
	 * Totals the size of the specified object, along with any other objects 
	 * that have it in their outer chain. Does not account for any allocated 
	 * memory that belongs to the object(s).
	 * 
	 * @param  Object	The object you want an estimated byte size for.
	 * @return An estimated size (in bytes)... currently does not account for any allocated memory that the object may be responsible for.
	 */
	static int32 ApproximateMemFootprint(UObject const* Object);	
}

//------------------------------------------------------------------------------
static bool BlueprintNodeTemplateCacheImpl::IsCompatible(UEdGraphNode* NodeObj, UEdGraph* Graph)
{
	const UEdGraphSchema* Schema = Graph->GetSchema();
	return ensureMsgf(Schema != nullptr, TEXT("PROTO_BP graph with invalid schema: %s "), *Graph->GetName()) && 
		NodeObj->CanCreateUnderSpecifiedSchema(Schema);
}

//------------------------------------------------------------------------------
static UEdGraph* BlueprintNodeTemplateCacheImpl::FindCompatibleGraph(UBlueprint* BlueprintOuter, UEdGraphNode* NodeObj, bool(*IsCompatibleFunc)(UEdGraph*))
{
	UEdGraph* FoundGraph = nullptr;

	TArray<UObject*> BlueprintChildObjs;
	GetObjectsWithOuter(BlueprintOuter, BlueprintChildObjs, EGetObjectsFlags::None, /*ExclusionFlags =*/ RF_NoFlags, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage);

	for (UObject* Child : BlueprintChildObjs)
	{
		UEdGraph* ChildGraph = Cast<UEdGraph>(Child);
		bool bIsCompatible = (ChildGraph != nullptr) && IsCompatible(NodeObj, ChildGraph);

		if (bIsCompatible && (IsCompatibleFunc != nullptr))
		{
			bIsCompatible = IsCompatibleFunc(ChildGraph);
		}

		if (bIsCompatible)
		{
			FoundGraph = ChildGraph;
			break;
		}
	}

	return FoundGraph;
}

//------------------------------------------------------------------------------
static UBlueprint* BlueprintNodeTemplateCacheImpl::MakeCompatibleBlueprint(TSubclassOf<UBlueprint> BlueprintClass, UClass* ParentClass, TSubclassOf<UBlueprintGeneratedClass> GeneratedClassType)
{
	EBlueprintType BlueprintType = BPTYPE_Normal;

	if (GeneratedClassType == nullptr)
	{
		GeneratedClassType = UBlueprintGeneratedClass::StaticClass();
	}

	UPackage* BlueprintOuter = GetTransientPackage();
	const FString DesiredName = FString::Printf(TEXT("PROTO_BP_%s"), *BlueprintClass->GetName());
	const FName BlueprintName = MakeUniqueObjectName(BlueprintOuter, BlueprintClass, FName(*DesiredName));

	BlueprintClass = FBlueprintEditorUtils::FindFirstNativeClass(BlueprintClass);
	UBlueprint* NewBlueprint = FKismetEditorUtilities::CreateBlueprint(ParentClass, BlueprintOuter, BlueprintName, BlueprintType, BlueprintClass, GeneratedClassType);
	NewBlueprint->SetFlags(RF_Transient);

	++MadeBlueprintCount;

	const float AproxBlueprintSize = static_cast<float>(ApproximateMemFootprint(NewBlueprint));
	const float MadeBlueprintCountFloat = static_cast<float>(MadeBlueprintCount);

	// track the average blueprint size, so that we can attempt to predict 
	// whether a  blueprint will fail to be cached (when the cache is near full)
	AverageBlueprintSize = 
		static_cast<int32>(
			(AverageBlueprintSize * ((MadeBlueprintCountFloat - 1.0f) / MadeBlueprintCountFloat)) +
			(AproxBlueprintSize / MadeBlueprintCountFloat) +
			0.5f
		);

	return NewBlueprint;
}

//------------------------------------------------------------------------------
static UEdGraph* BlueprintNodeTemplateCacheImpl::AddGraph(UBlueprint* BlueprintOuter, TSubclassOf<UEdGraphSchema> SchemaClass)
{
	UClass* GraphClass = UEdGraph::StaticClass();
	FName const GraphName = MakeUniqueObjectName(BlueprintOuter, GraphClass, FName(TEXT("TEMPLATE_NODE_OUTER")));

	UEdGraph* NewGraph = NewObject<UEdGraph>(BlueprintOuter, GraphClass, GraphName, RF_Transient);
	NewGraph->Schema = SchemaClass;

	MarkGraphForTemplateUse(NewGraph);
	return NewGraph;
}

//------------------------------------------------------------------------------
static void BlueprintNodeTemplateCacheImpl::MarkGraphForTemplateUse(UEdGraph* NewGraph)
{
	UPackage* TemplatePackage = NewGraph->GetPackage();
	FMetaData& PackageMetadata = TemplatePackage->GetMetaData();
	PackageMetadata.SetValue(NewGraph, TemplateGraphMetaTag, TEXT("true"));
}

//------------------------------------------------------------------------------
bool BlueprintNodeTemplateCacheImpl::IsTemplateOuter(UEdGraph* ParentGraph)
{
	if (ParentGraph && ParentGraph->HasAnyFlags(RF_Transactional))
	{
		UPackage* GraphPackage = ParentGraph->GetPackage();
		FMetaData& PackageMetadata = GraphPackage->GetMetaData();
		return PackageMetadata.HasValue(ParentGraph, TemplateGraphMetaTag);
	}
	return false;
}

//------------------------------------------------------------------------------
static int32 BlueprintNodeTemplateCacheImpl::GetCacheCapSize()
{
	const UBlueprintEditorSettings* BpSettings = GetDefault<UBlueprintEditorSettings>();
	// have to convert from MB to bytes
	return (static_cast<int32>(BpSettings->NodeTemplateCacheCapMB) << 20);
}

//------------------------------------------------------------------------------
static int32 BlueprintNodeTemplateCacheImpl::ApproximateMemFootprint(UObject const* Object)
{
	TArray<UObject*> ChildObjs;
	GetObjectsWithOuter(Object, ChildObjs, EGetObjectsFlags::IncludeNestedObjects);

	int32 ApproimateDataSize = sizeof(*Object);
	for (UObject* ChildObj : ChildObjs)
	{
		// @TODO: doesn't account for any internal allocated memory (for member TArrays, etc.)
		ApproimateDataSize += sizeof(*ChildObj);
	}
	return ApproimateDataSize;
}

/*******************************************************************************
 * FBlueprintNodeTemplateCache
 ******************************************************************************/

//------------------------------------------------------------------------------
FBlueprintNodeTemplateCache::FBlueprintNodeTemplateCache()
	: ApproximateObjectMem(0)
{
	using namespace BlueprintNodeTemplateCacheImpl; // for MakeCompatibleBlueprint()

	UBlueprint* StandardBlueprint = MakeCompatibleBlueprint(UBlueprint::StaticClass(), AActor::StaticClass(), UBlueprintGeneratedClass::StaticClass());
	CacheBlueprintOuter(StandardBlueprint);

	UBlueprint* AnimBlueprint = MakeCompatibleBlueprint(UAnimBlueprint::StaticClass(), UAnimInstance::StaticClass(), UAnimBlueprintGeneratedClass::StaticClass());
	CacheBlueprintOuter(AnimBlueprint);
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintNodeTemplateCache::GetNodeTemplate(UBlueprintNodeSpawner const* NodeSpawner, UEdGraph* TargetGraph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBlueprintNodeTemplateCache::GetNodeTemplate);

	using namespace BlueprintNodeTemplateCacheImpl;

	bool bIsOverMemCap = false;
	auto LogCacheFullMsg = [&bIsOverMemCap]()
	{
		if (!bIsOverMemCap)
		{
			static int32 LoggedCapSize = -1;
			int32 const CurrentCacheSize = GetCacheCapSize();
			// log only once for each cap size change
			if (LoggedCapSize != CurrentCacheSize)
			{
				UE_LOGF(LogBlueprintNodeCache, Display, "The blueprint template-node cache is full. As a result, you may experience interactions which are slower than normal. To avoid this, increase the cache's cap in the blueprint editor prefences.");
				LoggedCapSize = CurrentCacheSize;
			}			
			bIsOverMemCap = true;
		}
	};
	

	UEdGraphNode* TemplateNode = nullptr;
	if (auto* FoundNode = NodeTemplateCache.Find(NodeSpawner))
	{
		TemplateNode = *FoundNode;
	}
	else if (NodeSpawner->NodeClass != nullptr)
	{
		UEdGraphNode* NodeCDO = NodeSpawner->NodeClass->GetDefaultObject<UEdGraphNode>();
		check(NodeCDO != nullptr);

		UBlueprint* TargetBlueprint = nullptr;
		TSubclassOf<UBlueprint> BlueprintClass;

		bool const bHasTargetGraph = (TargetGraph != nullptr);
		if (bHasTargetGraph)
		{
			// by the time we're asking for a prototype for this spawner, we should
			// be sure that it is compatible with the TargetGraph
			//check(IsCompatible(NodeCDO, TargetGraph));

			TargetBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
			check(TargetBlueprint != nullptr);
			BlueprintClass = TargetBlueprint->GetClass();

			// check used to help identify user interactable graphs (as opposed to 
			// intermediate/transient graphs).
			auto IsCompatibleUserGraph = [](UEdGraph* Graph)->bool
			{
				return !Graph->HasAnyFlags(RF_Transient);
			};

			TargetGraph = FindCompatibleGraph(TargetBlueprint, NodeCDO, IsCompatibleUserGraph);
			check(TargetGraph != nullptr);
		}		

		UBlueprint* CompatibleBlueprint = nullptr;
		UEdGraph*   CompatibleOuter     = nullptr;
		// find a compatible outer (don't want to have to create a new one if we don't have to)
		for (UBlueprint* Blueprint : TemplateOuters)
		{
			CompatibleOuter = FindCompatibleGraph(Blueprint, NodeCDO);
			if (CompatibleOuter != nullptr)
			{
				MarkGraphForTemplateUse(CompatibleOuter);
			}

			if (CompatibleOuter != nullptr)
			{
				CompatibleBlueprint = Blueprint;
				break;
			}
			else if ((BlueprintClass != nullptr) && Blueprint->GetClass()->IsChildOf(BlueprintClass))
			{
				CompatibleBlueprint = Blueprint;
			}
		}

		// reset ActiveMemFootprint, so calls to CacheBlueprintOuter()/CacheTemplateNode()
		// use the most up-to-date value (users could have since modified the
		// nodes, so they could have grown in size... like with AllocateDefaultPins)
		//
		// @TODO: GetEstimateCacheSize() is (most likely) inaccurate, seeing as
		//        external systems mutate template-nodes (such as calling
		//        AllocateDefaultPins), and this returns a size estimate from
		//        when the node was first spawned (it is too slow to recalculate
		//        the size of the object hierarchy here)
		ActiveMemFootprint = GetEstimateCacheSize();

		int32 const CacheCapSize = GetCacheCapSize();
		if (ActiveMemFootprint > CacheCapSize)
		{
			LogCacheFullMsg();
			// @TODO: evict nodes until we're back under the cap (in case the cap
			//        was changed at runtime, or external user modified node sizes)
		}

		// if a TargetGraph was supplied, and we couldn't find a suitable outer
		// for this template-node, then attempt to emulate that graph
		if (bHasTargetGraph)
		{
			if (CompatibleBlueprint == nullptr)
			{
				// if the cache is near full, attempt to predict if this  
				// impending cache will fail (if so, we don't want to waste the 
				// cycles on allocating a temp blueprint)
				if (!bIsOverMemCap && ((AverageBlueprintSize == 0) ||
					(ActiveMemFootprint + AverageBlueprintSize <= CacheCapSize)))
				{
					TSubclassOf<UBlueprintGeneratedClass> GeneratedClassType = UBlueprintGeneratedClass::StaticClass();
					if (TargetBlueprint->GeneratedClass != nullptr)
					{
						GeneratedClassType = TargetBlueprint->GeneratedClass->GetClass();
					}

					UClass* ParentClass = TargetBlueprint->ParentClass != nullptr ? TargetBlueprint->ParentClass.Get() : UObject::StaticClass();
					CompatibleBlueprint = MakeCompatibleBlueprint(BlueprintClass, ParentClass, GeneratedClassType);
					if (!CacheBlueprintOuter(CompatibleBlueprint))
					{
						LogCacheFullMsg();
					}

					// this graph may come default with a compatible graph
					CompatibleOuter = FindCompatibleGraph(CompatibleBlueprint, NodeCDO);
					if (CompatibleOuter != nullptr)
					{
						MarkGraphForTemplateUse(CompatibleOuter);
					}
				}
				else
				{
 					CompatibleBlueprint = TargetBlueprint;
 					CompatibleOuter = FindCompatibleGraph(TargetBlueprint, NodeCDO, &BlueprintNodeTemplateCacheImpl::IsTemplateOuter);

					LogCacheFullMsg();
				}
			}

			if (CompatibleOuter == nullptr)
			{
				CompatibleOuter = AddGraph(CompatibleBlueprint, TargetGraph->Schema);
				ensureMsgf( CompatibleOuter->Schema != nullptr, TEXT("Invalid schema for template graph (from '%s :: %s')."),
						*TargetBlueprint->GetName(), *TargetGraph->GetName() );

				if (CompatibleBlueprint != TargetBlueprint)
				{
					int32 const ApproxGraphSize = ApproximateMemFootprint(CompatibleOuter);
					ActiveMemFootprint   += ApproxGraphSize;
					ApproximateObjectMem += ApproxGraphSize;
				}				
			}
		}

		if (CompatibleOuter != nullptr)
		{
			TemplateNode = NodeSpawner->Invoke(CompatibleOuter, IBlueprintNodeBinder::FBindingSet(), FVector2D::ZeroVector);
			if (!bIsOverMemCap && !CacheTemplateNode(NodeSpawner, TemplateNode))
			{
				LogCacheFullMsg();
			}
		}
	}

	return TemplateNode;
}

//------------------------------------------------------------------------------
UEdGraphNode* FBlueprintNodeTemplateCache::GetNodeTemplate(UBlueprintNodeSpawner const* NodeSpawner, ENoInit) const
{
	UEdGraphNode* TemplateNode = nullptr;
	if (auto* FoundNode = NodeTemplateCache.Find(NodeSpawner))
	{
		return *FoundNode;
	}
	return nullptr;
}

//------------------------------------------------------------------------------
void FBlueprintNodeTemplateCache::ClearCachedTemplate(UBlueprintNodeSpawner const* NodeSpawner)
{
	NodeTemplateCache.Remove(NodeSpawner);
	// GC should take care of the rest
}

//------------------------------------------------------------------------------
int64 FBlueprintNodeTemplateCache::GetEstimateCacheSize() const
{
	int64 TotalEstimatedSize = ApproximateObjectMem;
	TotalEstimatedSize += TemplateOuters.GetAllocatedSize();
	TotalEstimatedSize += NodeTemplateCache.GetAllocatedSize();
	TotalEstimatedSize += sizeof(*this);

	return TotalEstimatedSize;
}

//------------------------------------------------------------------------------
int64 FBlueprintNodeTemplateCache::RecalculateCacheSize()
{
	ApproximateObjectMem = 0;
	for (UBlueprint* Blueprint : TemplateOuters)
	{
		// if we didn't run garbage collection at the top, then this could also
		// account for nodes that were never stored (because the cache was too full)
		ApproximateObjectMem += BlueprintNodeTemplateCacheImpl::ApproximateMemFootprint(Blueprint);
	}
	return ApproximateObjectMem;
}

//------------------------------------------------------------------------------
bool FBlueprintNodeTemplateCache::IsTemplateOuter(UEdGraph* ParentGraph)
{
	return BlueprintNodeTemplateCacheImpl::IsTemplateOuter(ParentGraph);
}

//------------------------------------------------------------------------------
void FBlueprintNodeTemplateCache::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (auto& TemplateEntry : NodeTemplateCache)
	{
		Collector.AddReferencedObject(TemplateEntry.Value);
	}
	Collector.AddReferencedObjects(TemplateOuters);
}

FString FBlueprintNodeTemplateCache::GetReferencerName() const
{
	return TEXT("FBlueprintNodeTemplateCache");
}

//------------------------------------------------------------------------------
bool FBlueprintNodeTemplateCache::CacheBlueprintOuter(UBlueprint* Blueprint)
{
	using namespace BlueprintNodeTemplateCacheImpl;
	int32 const ApproxBlueprintSize = ApproximateMemFootprint(Blueprint);

	if (ActiveMemFootprint + ApproxBlueprintSize > GetCacheCapSize())
	{
		return false;
	}
	else
	{
		ApproximateObjectMem += ApproxBlueprintSize;
		TemplateOuters.Add(Blueprint);
		return true;
	}
}

//------------------------------------------------------------------------------
bool FBlueprintNodeTemplateCache::CacheTemplateNode(UBlueprintNodeSpawner const* NodeSpawner, UEdGraphNode* NewNode)
{
	using namespace BlueprintNodeTemplateCacheImpl;

	if (NewNode == nullptr)
	{
		return true;
	}

	int32 const ApproxNodeSize = BlueprintNodeTemplateCacheImpl::ApproximateMemFootprint(NewNode);
	if (ActiveMemFootprint + ApproxNodeSize > GetCacheCapSize())
	{
		return false;
	}
	else
	{
		ApproximateObjectMem += ApproxNodeSize;
		NodeTemplateCache.Add(NodeSpawner, NewNode);
		return true;
	}
}
