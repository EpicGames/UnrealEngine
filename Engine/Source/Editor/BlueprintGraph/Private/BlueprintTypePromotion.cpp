// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintTypePromotion.h"

#include "Async/ParallelFor.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MTAccessDetector.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Script.h"
#include "UObject/UObjectHash.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "initializer_list"

class UBlueprintFunctionNodeSpawner;
class UK2Node;

namespace TypePromotionImpl
{
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(InstanceAccessProtector);
}

FTypePromotion* FTypePromotion::Instance = nullptr;

#define LOCTEXT_NAMESPACE "TypePromotion"

namespace OperatorNames
{
	static const FName NoOp			= TEXT("NO_OP");

	static const FName Add			= TEXT("Add");
	static const FName Multiply		= TEXT("Multiply");
	static const FName Subtract		= TEXT("Subtract");
	static const FName Divide		= TEXT("Divide");
	
	static const FName Greater		= TEXT("Greater");
	static const FName GreaterEq	= TEXT("GreaterEqual");
	static const FName Less			= TEXT("Less");
	static const FName LessEq		= TEXT("LessEqual");
	static const FName NotEq		= TEXT("NotEqual");
	static const FName Equal		= TEXT("EqualEqual");
}

FTypePromotion& FTypePromotion::Get()
{
	if (Instance == nullptr)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TypePromotionImpl::InstanceAccessProtector);
		Instance = new FTypePromotion();
	}
	return *Instance;
}

void FTypePromotion::Shutdown()
{
	if (Instance)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TypePromotionImpl::InstanceAccessProtector);
		delete Instance;
		Instance = nullptr;
	}
}

FTypePromotion::FTypePromotion()
	: PromotionTable(CreatePromotionTable())
{
	CreateOpTable();
}

FTypePromotion::~FTypePromotion()
{
}

TMap<FName, TArray<FName>> FTypePromotion::CreatePromotionTable()
{
	return
	{
		// Type_X...						Can be promoted to...
		{ UEdGraphSchema_K2::PC_Int,		{ UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Int64 } },
		{ UEdGraphSchema_K2::PC_Byte,		{ UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Int, UEdGraphSchema_K2::PC_Int64 } },
		{ UEdGraphSchema_K2::PC_Real,		{ UEdGraphSchema_K2::PC_Int64 } },
		{ UEdGraphSchema_K2::PC_Wildcard,	{ UEdGraphSchema_K2::PC_Int, UEdGraphSchema_K2::PC_Int64, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Byte, UEdGraphSchema_K2::PC_Boolean } },
	};
}

const TMap<FName, TArray<FName>>* const FTypePromotion::GetPrimitivePromotionTable()
{
	if (Instance)
	{
		return &Instance->PromotionTable;
	}

	return nullptr;
}

const TArray<FName>* FTypePromotion::GetAvailablePrimitivePromotions(const FEdGraphPinType& Type)
{
	const TMap<FName, TArray<FName>>* PromoTable = GetPrimitivePromotionTable();

	return PromoTable->Find(Type.PinCategory);
}

bool FTypePromotion::IsValidPromotion(const FEdGraphPinType& A, const FEdGraphPinType& B)
{
	// If either of these pin types is a struct, than we have to have some kind of valid
	// conversion function, otherwise we can't possibly connect them
	if (A.PinCategory == UEdGraphSchema_K2::PC_Struct || B.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		const TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> Res = K2Schema->SearchForAutocastFunction(A, B);
		
		const UFunction* AutoFunc =
			Res.IsSet() ?
			Res->FunctionOwner->FindFunctionByName(Res->TargetFunction) :
			nullptr;

		// We can only use this conversion function for promotion if it is not marked to ignore it
		return AutoFunc != nullptr && !AutoFunc->HasMetaData(FBlueprintMetadata::MD_IgnoreTypePromotion);
	}
	else
	{
		return FTypePromotion::GetHigherType(A, B) == ETypeComparisonResult::TypeBHigher;
	}
}

bool FTypePromotion::HasStructConversion(const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin)
{
	check(InputPin);
	check(OutputPin);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Check if there is a "simple" auto cast function first
	const TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> AutoCastRes = K2Schema->SearchForAutocastFunction(OutputPin->PinType, InputPin->PinType);
	const UFunction* AutoFunc =
		AutoCastRes.IsSet() ?
		AutoCastRes->FunctionOwner->FindFunctionByName(AutoCastRes->TargetFunction) :
		nullptr;

	// We can auto-cast the two types so long as there is a valid function and it is not marked to ignore type promo
	const bool bCanAutocast = AutoFunc && !AutoFunc->HasMetaData(FBlueprintMetadata::MD_IgnoreTypePromotion);
	
	if (bCanAutocast)
	{
		return true;
	}

	// Otherwise we have to search for a "special" conversion node. This is
	// mostly for enum types but can also include some other types like containers and specific assets
	const bool bCanAutoConvert = K2Schema->FindSpecializedConversionNode(OutputPin->PinType, *InputPin, false).IsSet();
	
	// If we have a valid result, then we can convert.
	return bCanAutoConvert;
}

FTypePromotion::ETypeComparisonResult FTypePromotion::GetHigherType(const FEdGraphPinType& A, const FEdGraphPinType& B)
{
	return FTypePromotion::Get().GetHigherType_Internal(A, B);
}

FTypePromotion::ETypeComparisonResult FTypePromotion::GetHigherType_Internal(const FEdGraphPinType& A, const FEdGraphPinType& B) const
{
	if(A == B)
	{
		return ETypeComparisonResult::TypesEqual;
	}
	// Is this A promotable type?					  Can type A be promoted to type B?
	else if(PromotionTable.Contains(A.PinCategory) && PromotionTable[A.PinCategory].Contains(B.PinCategory))
	{
		return ETypeComparisonResult::TypeBHigher;
	}
	// Can B get promoted to A?
	else if(PromotionTable.Contains(B.PinCategory) && PromotionTable[B.PinCategory].Contains(A.PinCategory))
	{
		return ETypeComparisonResult::TypeAHigher;
	}
	// Handle "No" Pin type, the default value of FEdGraphPinType
	else if(A.PinCategory == NAME_None && B.PinCategory != NAME_None)
	{
		return ETypeComparisonResult::TypeBHigher;
	}
	else if(B.PinCategory == NAME_None && A.PinCategory != NAME_None)
	{
		return ETypeComparisonResult::TypeAHigher;
	}
	// A is a struct and B is not a struct
	else if(A.PinCategory == UEdGraphSchema_K2::PC_Struct && B.PinCategory != UEdGraphSchema_K2::PC_Struct)
	{
		return ETypeComparisonResult::TypeAHigher;
	}
	// A is not a struct and B is a struct
	else if (A.PinCategory != UEdGraphSchema_K2::PC_Struct && B.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		return ETypeComparisonResult::TypeBHigher;
	}

	// We couldn't find any possible promotions, so this is an invalid comparison
	return ETypeComparisonResult::InvalidComparison;
}

bool FTypePromotion::IsFunctionPromotionReady(const UFunction* const FuncToConsider)
{
	return FTypePromotion::Get().IsFunctionPromotionReady_Internal(FuncToConsider);
}

bool FTypePromotion::IsFunctionPromotionReady_Internal(const UFunction* const FuncToConsider) const
{
	const FName FuncOpName = GetOpNameFromFunction(FuncToConsider);

	FScopeLock ScopeLock(&Lock);
	if(const FFunctionsList* FuncList = OperatorTable.Find(FuncOpName))
	{
		return FuncList->Contains(FuncToConsider);
	}

	return false;
}

FEdGraphPinType FTypePromotion::GetPromotedType(const TArray<UEdGraphPin*>& WildcardPins)
{
	return FTypePromotion::Get().GetPromotedType_Internal(WildcardPins);
}

FEdGraphPinType FTypePromotion::GetPromotedType_Internal(const TArray<UEdGraphPin*>& WildcardPins) const
{
	// There must be some wildcard pins in order to get the promoted type
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotionTable::GetPromotedType);
	
	FEdGraphPinType HighestPinType = FEdGraphPinType();

	for (const UEdGraphPin* CurPin : WildcardPins)
	{
		if(CurPin)
		{
			ETypeComparisonResult Res = GetHigherType(/* A */ HighestPinType, /* B */ CurPin->PinType);

			// If this pin is a different type and "higher" than set our out pin type to that
			switch(Res)
			{
				case ETypeComparisonResult::TypeBHigher:
					HighestPinType = CurPin->PinType;
				break;
			}
		}
	}
	return HighestPinType;
}

UFunction* FTypePromotion::FindBestMatchingFunc(FName Operation, const TArray<UEdGraphPin*>& PinsToConsider)
{
	return FTypePromotion::Get().FindBestMatchingFunc_Internal(Operation, PinsToConsider);
}

static bool PropertyCompatibleWithPin(const FProperty* Param, FEdGraphPinType const& TypeToMatch)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	FEdGraphPinType ParamType;
	if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
	{
		if (Schema->ArePinTypesCompatible(TypeToMatch, ParamType) &&
			FTypePromotion::GetHigherType(TypeToMatch, ParamType) != FTypePromotion::ETypeComparisonResult::InvalidComparison)
		{
			return true;
		}
	}
	return false;
}

UFunction* FTypePromotion::FindBestMatchingFunc_Internal(FName Operation, const TArray<UEdGraphPin*>& PinsToConsider)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotionTable::FindBestMatchingFunc_Internal);

	FScopeLock ScopeLock(&Lock);

	const FFunctionsList* FuncList = OperatorTable.Find(Operation);
	if (!FuncList)
	{
		return nullptr;
	}

	// Track the function with the best score, input, and output types
	UFunction* BestFunc = nullptr;
	FEdGraphPinType BestFuncLowestInputType;
	FEdGraphPinType BestFuncOutputType;
	int32 BestScore = -1;

	const bool bIsSinglePin = PinsToConsider.Num() == 1;
	const bool bIsComparisonOp = GetComparisonOpNames().Contains(Operation);
	const bool bHasStruct = PinsToConsider.ContainsByPredicate([](const UEdGraphPin* Pin) { return Pin && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct; });

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		
	// We have to keep track of what pins we have already given points for, otherwise 
	// we will end up giving the same pin multiple points.
	TSet<const UEdGraphPin*> CheckedPins;
		
	for (UFunction* Func : *FuncList)
	{
		int32 FuncScore = -1;
		CheckedPins.Reset();
		bool bIsInvalidMatch = false;

		// Track this functions highest input and output types so that if there is a function with
		// the same score as it we can prefer the correct one. 
		FEdGraphPinType CurFuncHighestInputType;
		FEdGraphPinType CurFuncOutputType;

		// For each property in the func, see if it matches any of the given pins
		for (TFieldIterator<FProperty> PropIt(Func); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;
			FEdGraphPinType ParamType;
			if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
			{
				// Don't bother with this function if there is a struct param, if no pins have any structs
				if (ParamType.PinCategory == UEdGraphSchema_K2::PC_Struct && !bHasStruct)
				{
					bIsInvalidMatch = true;
					break;
				}

				for (const UEdGraphPin* Pin : PinsToConsider)
				{
					// Object types will not match ever the function input params and neither will classes, because their
					// sub-category will be different. 
					const bool bObjectTypesMatch = ParamType.PinCategory == UEdGraphSchema_K2::PC_Object && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object;
					const bool bClassTypesMatch = ParamType.PinCategory == UEdGraphSchema_K2::PC_Class && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class;

					// Give a point for each function parameter that matches up with a pin to consider
					if ((!CheckedPins.Contains(Pin) || bIsSinglePin) && (bObjectTypesMatch || bClassTypesMatch || Schema->ArePinTypesCompatible(Pin->PinType, ParamType)))
					{
						// Are the directions compatible? 
						// If we are a comparison or only a single pin then we don't care about the direction
						if (bIsSinglePin || bIsComparisonOp ||
						   (Param->HasAnyPropertyFlags(CPF_ReturnParm) && Pin->Direction == EGPD_Output) ||
						   (!Param->HasAnyPropertyFlags(CPF_ReturnParm) && Pin->Direction == EGPD_Input))
						{
							++FuncScore;
							CheckedPins.Add(Pin);
						}

						break;
					}
				}

				// Keep track of the highest input pin type on this function
				if (Param->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					CurFuncOutputType = ParamType;
				}
				else if (CurFuncHighestInputType.PinCategory == NAME_None || FTypePromotion::GetHigherType(ParamType, CurFuncHighestInputType) == ETypeComparisonResult::TypeBHigher)
				{
					CurFuncHighestInputType = ParamType;
				}
			}
		}

		// If the pin type has no name, then this is an invalid comparison
		ETypeComparisonResult InputCompareRes = BestFuncLowestInputType.PinCategory != NAME_None ? FTypePromotion::GetHigherType(CurFuncHighestInputType, BestFuncLowestInputType) : ETypeComparisonResult::InvalidComparison;
		ETypeComparisonResult OutputCompareRes = BestFuncOutputType.PinCategory != NAME_None ? FTypePromotion::GetHigherType(CurFuncOutputType, BestFuncOutputType) : ETypeComparisonResult::InvalidComparison;
		
		// We want to prefer a HIGHER input, and a LOWER output. 
		const bool bHasInputOutputPreference =
			InputCompareRes != ETypeComparisonResult::TypeBHigher &&
			OutputCompareRes != ETypeComparisonResult::TypeAHigher;

		// If the scores are equal, then prefer the LARGER input and output type because we can promote up, but we can never go back down
		const bool bScoresEqualAndPreferred = 
			FuncScore == BestScore && FuncScore != -1 &&
			(bIsComparisonOp || bIsSinglePin || 
			(InputCompareRes == ETypeComparisonResult::TypeAHigher ||
			OutputCompareRes == ETypeComparisonResult::TypeAHigher));

		// Keep track of the best function!
		if (!bIsInvalidMatch && (bScoresEqualAndPreferred || (FuncScore > BestScore && (bHasInputOutputPreference || bIsComparisonOp))))
		{
			BestScore = FuncScore;
			BestFuncLowestInputType = CurFuncHighestInputType;
			BestFuncOutputType = CurFuncOutputType;
			BestFunc = Func;
		}
	}
	return BestFunc;
}

void FTypePromotion::GetAllFuncsForOp(FName Operation, TArray<UFunction*>& OutFuncs)
{
	return FTypePromotion::Get().GetAllFuncsForOp_Internal(Operation, OutFuncs);
}

const TSet<FName>& FTypePromotion::GetAllOpNames()
{
	static const TSet<FName> OpsArray =
	{
		OperatorNames::Add,
		OperatorNames::Multiply,
		OperatorNames::Subtract,
		OperatorNames::Divide,
		OperatorNames::Greater,
		OperatorNames::GreaterEq,
		OperatorNames::Less,
		OperatorNames::LessEq,
		OperatorNames::NotEq,
		OperatorNames::Equal
	};

	return OpsArray;
}

const TSet<FName>& FTypePromotion::GetComparisonOpNames()
{
	static const TSet<FName> ComparisonOps =
	{
		OperatorNames::Greater,
		OperatorNames::GreaterEq,
		OperatorNames::Less,
		OperatorNames::LessEq,
		OperatorNames::NotEq,
		OperatorNames::Equal
	};
	return ComparisonOps;
}

const FText& FTypePromotion::GetKeywordsForOperator(const FName Operator)
{
	static const TMap<FName, FText> OpKeywords =
	{
		{ OperatorNames::Add,		LOCTEXT("AddKeywords",			"+ add plus") },
		{ OperatorNames::Multiply,  LOCTEXT("MultiplyKeywords",		"* multiply") },
		{ OperatorNames::Subtract,  LOCTEXT("SubtractKeywords",		"- subtract minus") },
		{ OperatorNames::Divide,	LOCTEXT("DivideKeywords",		"/ divide division") },
		{ OperatorNames::Greater,	LOCTEXT("GreaterKeywords",		"> greater") },
		{ OperatorNames::GreaterEq, LOCTEXT("GreaterEqKeywords",	">= greater") },
		{ OperatorNames::Less,		LOCTEXT("LessKeywords",			"< less") },
		{ OperatorNames::LessEq,	LOCTEXT("LessEqKeywords",		"<= less") },
		{ OperatorNames::NotEq,		LOCTEXT("NotEqKeywords",		"!= not equal") },
		{ OperatorNames::Equal,		LOCTEXT("EqualKeywords",		"== equal") },
		{ OperatorNames::NoOp,		FText() },
	};
	const FText* Keywords = OpKeywords.Find(Operator);
	return Keywords ? *Keywords : OpKeywords[OperatorNames::NoOp];
}

const FText& FTypePromotion::GetUserFacingOperatorName(const FName Operator)
{
	static const TMap<FName, FText> OperatorDisplayNames = 
	{
		{ OperatorNames::Add,		LOCTEXT("AddDisplayName",			"Add") },
		{ OperatorNames::Multiply,  LOCTEXT("MultiplyDisplayName",		"Multiply") },
		{ OperatorNames::Subtract,  LOCTEXT("SubtractDisplayName",		"Subtract") },
		{ OperatorNames::Divide,	LOCTEXT("DivideDisplayName",		"Divide") },
		{ OperatorNames::Greater,	LOCTEXT("GreaterDisplayName",		"Greater ( > )") },
		{ OperatorNames::GreaterEq, LOCTEXT("GreaterEqDisplayName",		"Greater Equal ( >= )") },
		{ OperatorNames::Less,		LOCTEXT("LessDisplayName",			"Less ( < )") },
		{ OperatorNames::LessEq,	LOCTEXT("LessEqDisplayName",		"Less Equal ( <= )") },
		{ OperatorNames::NotEq,		LOCTEXT("NotEqDisplayName",			"Not Equal ( != )") },
		{ OperatorNames::Equal,		LOCTEXT("EqualDisplayName",			"Equal ( == )") },
		{ OperatorNames::NoOp,		FText() },
	};


	const FText* Keywords = OperatorDisplayNames.Find(Operator);
	return Keywords ? *Keywords : OperatorDisplayNames[OperatorNames::NoOp];
}

bool FTypePromotion::IsComparisonFunc(UFunction const* const Func)
{
	return Func && GetComparisonOpNames().Contains(GetOpNameFromFunction(Func));
}

bool FTypePromotion::IsComparisonOpName(const FName OpName)
{
	return GetComparisonOpNames().Contains(OpName);
}

void FTypePromotion::GetAllFuncsForOp_Internal(FName Operation, TArray<UFunction*>& OutFuncs)
{
	OutFuncs.Empty();

	FScopeLock ScopeLock(&Lock);
	OutFuncs.Append(OperatorTable[Operation]);
}

FName FTypePromotion::GetOpNameFromFunction(UFunction const* const Func)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotion::GetOpNameFromFunction);
	if(!Func)
	{
		return OperatorNames::NoOp;
	}

	TStringBuilder<256> FuncName;
	Func->GetFName().ToString(FuncName);
	TStringView FuncNameView = FuncName.ToView();
	// Get everything before the "_"
	int32 Index = FuncNameView.Find(TEXT("_"));
	
	FName FuncNameChopped(FuncNameView.Mid(0, Index));
	if (GetAllOpNames().Contains(FuncNameChopped))
	{
		return FuncNameChopped;
	}

	return OperatorNames::NoOp;
}

void FTypePromotion::CreateOpTable()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotion::CreateOpTable);
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	FCriticalSection NewOperatorTableLock;
	TMap<FName, FFunctionsList> NewOperatorTable;

	TArray<UClass*> Libraries;
	GetDerivedClasses(UBlueprintFunctionLibrary::StaticClass(), Libraries);
	ParallelFor(Libraries.Num(), 
		[this, &Schema, &NewOperatorTable, &NewOperatorTableLock, &Libraries](int32 Index)
		{
			UClass* Library = Libraries[Index];
			// Ignore abstract libraries/classes
			if (!Library || Library->HasAnyClassFlags(CLASS_Abstract))
			{
				return;
			}

			for (UFunction* Function : TFieldRange<UFunction>(Library, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated))
			{
				if (!IsPromotableFunction(Function))
				{
					continue;
				}

				FEdGraphPinType FuncPinType;
				FName OpName = GetOpNameFromFunction(Function);

				if (OpName != OperatorNames::NoOp && Schema->ConvertPropertyToPinType(Function->GetReturnProperty(), /* out */ FuncPinType))
				{
					FScopeLock ScopeLock(&NewOperatorTableLock);
					NewOperatorTable.FindOrAdd(OpName).Add(Function);
				}
			}
		});

	FScopeLock ScopeLock(&Lock);
	OperatorTable = MoveTemp(NewOperatorTable);
}

void FTypePromotion::AddOpFunction(FName OpName, UFunction* Function)
{
	FScopeLock ScopeLock(&Lock);
	OperatorTable.FindOrAdd(OpName).Add(Function);
}

static bool IsPinTypeDeniedForTypePromotion(const UFunction* Function)
{
	const TSet<FName>& DenyList = GetDefault<UBlueprintEditorSettings>()->TypePromotionPinDenyList;

	check(Function);
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// For each property in the func, see if the pin type is on the deny list 
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		const FProperty* Param = *PropIt;
		FEdGraphPinType ParamType;
		if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
		{
			if (DenyList.Contains(ParamType.PinCategory))
			{
				return true;
			}
		}
	}

	return false;
}

bool FTypePromotion::IsPromotableFunction(const UFunction* Function)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotion::IsPromotableFunction);

	// Ensure that we don't have an invalid OpName as well for extra safety when this function 
	// is called outside of this class, not during the OpTable creation process
	FName OpName = GetOpNameFromFunction(Function);
	return Function &&
		Function->HasAnyFunctionFlags(FUNC_BlueprintPure) &&
		Function->GetReturnProperty() &&
		OpName != OperatorNames::NoOp && 
		!IsPinTypeDeniedForTypePromotion(Function) &&
		// Users can deny specific functions from being considered for type promotion
		!Function->HasMetaData(FBlueprintMetadata::MD_IgnoreTypePromotion);
}

bool FTypePromotion::IsOperatorSpawnerRegistered(UFunction const* const Func)
{
	return FTypePromotion::GetOperatorSpawner(FTypePromotion::GetOpNameFromFunction(Func)) != nullptr;
}

void FTypePromotion::RegisterOperatorSpawner(FName OpName, UBlueprintFunctionNodeSpawner* Spawner)
{
	if(Instance)
	{
		FScopeLock ScopeLock(&Instance->Lock);
		if (OpName != OperatorNames::NoOp && !Instance->OperatorNodeSpawnerMap.Contains(OpName))
		{
			Instance->OperatorNodeSpawnerMap.Add(OpName, Spawner);
		}
	}
}

UBlueprintFunctionNodeSpawner* FTypePromotion::GetOperatorSpawner(FName OpName)
{
	if(Instance)
	{
		FScopeLock ScopeLock(&Instance->Lock);
		if (Instance->OperatorNodeSpawnerMap.Contains(OpName))
		{
			return Instance->OperatorNodeSpawnerMap[OpName];
		}
	}

	return nullptr;
}

void FTypePromotion::ClearNodeSpawners()
{
	if (Instance)
	{
		FScopeLock ScopeLock(&Instance->Lock);
		Instance->OperatorNodeSpawnerMap.Empty();
	}
}

void FTypePromotion::RefreshPromotionTables(EReloadCompleteReason Reason /* = EReloadCompleteReason::None */)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTypePromotionTable::RefreshPromotionTables);

	if(Instance)
	{
		// Hold the lock during the whole operation since we don't want readers to see
		// any intermediate state.
		FScopeLock ScopeLock(&Instance->Lock);

		FTypePromotion::ClearNodeSpawners();

		Instance->OperatorTable.Empty();
		Instance->CreateOpTable();
	}
}

#undef LOCTEXT_NAMESPACE
