// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintVariableNodeSpawner.h"

#include "BlueprintActionFilter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "ObjectEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintVariableNodeSpawner)

#define LOCTEXT_NAMESPACE "BlueprintVariableNodeSpawner"

/*******************************************************************************
 * UBlueprintVariableNodeSpawner
 ******************************************************************************/

//------------------------------------------------------------------------------
UBlueprintVariableNodeSpawner* UBlueprintVariableNodeSpawner::CreateFromMemberOrParam(TSubclassOf<UK2Node_Variable> NodeClass, FProperty const* VarProperty, UEdGraph* VarContext, UClass* OwnerClass)
{
	check(VarProperty != nullptr);

	//--------------------------------------
	// Constructing the Spawner
	//--------------------------------------

	UBlueprintVariableNodeSpawner* NodeSpawner = NewObject<UBlueprintVariableNodeSpawner>(GetTransientPackage());
	NodeSpawner->NodeClass = NodeClass;
	NodeSpawner->SetField(const_cast<FProperty*>(VarProperty));
	NodeSpawner->LocalVarOuter = VarContext;

	//--------------------------------------
	// Default UI Signature
	//--------------------------------------

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	FString const VarSubCategory = FObjectEditorUtils::GetCategory(VarProperty);
	MenuSignature.Category = FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Variables, FText::FromString(VarSubCategory));

	FText const VarName = NodeSpawner->GetVariableName();
	// @TODO: NodeClass could be modified post Create()
	if (NodeClass != nullptr)
	{
		if (NodeClass->IsChildOf(UK2Node_VariableGet::StaticClass()))
		{
			MenuSignature.MenuName = FText::Format(LOCTEXT("GetterMenuName", "Get {0}"), VarName);
			MenuSignature.Tooltip  = UK2Node_VariableGet::GetPropertyTooltip(VarProperty);
		}
		else if (NodeClass->IsChildOf(UK2Node_VariableSet::StaticClass()))
		{
			MenuSignature.MenuName = FText::Format(LOCTEXT("SetterMenuName", "Set {0}"), VarName);
			MenuSignature.Tooltip  = UK2Node_VariableSet::GetPropertyTooltip(VarProperty);
		}
	}
	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	//
	// @TODO: maybe UPROPERTY() fields should have keyword metadata like functions
	if (MenuSignature.Keywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuSignature.Keywords = FText::FromString(TEXT(" "));
	}
	MenuSignature.Icon = UK2Node_Variable::GetVarIconFromPinType(NodeSpawner->GetVarType(), MenuSignature.IconTint);

	//--------------------------------------
	// Post-Spawn Setup
	//--------------------------------------

	auto MemberVarSetupLambda = [](UEdGraphNode* NewNode, FFieldVariant InField, UClass* OwnerClass)
	{
		if (FProperty const* Property = CastField<FProperty>(InField.ToField()))
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(NewNode);
			OwnerClass = OwnerClass ? OwnerClass : Property->GetOwnerClass();

			// We need to use a generated class instead of a skeleton class for IsChildOf, so if the OwnerClass has a Blueprint, grab the GeneratedClass
			const bool bOwnerClassIsSelfContext = (Blueprint->SkeletonGeneratedClass->GetAuthoritativeClass() == OwnerClass) || Blueprint->SkeletonGeneratedClass->IsChildOf(OwnerClass);
			const bool bIsFunctionVariable = Property->GetOwner<UFunction>() != nullptr;

			UK2Node_Variable* VarNode = CastChecked<UK2Node_Variable>(NewNode);
			VarNode->SetFromProperty(Property, bOwnerClassIsSelfContext && !bIsFunctionVariable, OwnerClass);
		}
	};
	NodeSpawner->SetNodeFieldDelegate = FSetNodeFieldDelegate::CreateStatic(MemberVarSetupLambda, OwnerClass);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintVariableNodeSpawner* UBlueprintVariableNodeSpawner::CreateFromLocal(TSubclassOf<UK2Node_Variable> NodeClass, UEdGraph* VarContext, FBPVariableDescription const& VarDesc, FProperty* VarProperty, UObject* Outer/*= nullptr*/)
{
	check(VarContext != nullptr);
	if (Outer == nullptr)
	{
		Outer = GetTransientPackage();
	}

	//--------------------------------------
	// Constructing the Spawner
	//--------------------------------------

	// @TODO: consider splitting out local variable spawners (since they don't 
	//        conform to UBlueprintFieldNodeSpawner
	UBlueprintVariableNodeSpawner* NodeSpawner = NewObject<UBlueprintVariableNodeSpawner>(Outer);
	NodeSpawner->NodeClass     = NodeClass;
	NodeSpawner->LocalVarOuter = VarContext;
	NodeSpawner->LocalVarDesc  = VarDesc;
	NodeSpawner->SetField(VarProperty);

	//--------------------------------------
	// Default UI Signature
	//--------------------------------------

	FBlueprintActionUiSpec& MenuSignature = NodeSpawner->DefaultMenuSignature;
	MenuSignature.Category = FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Variables, VarDesc.Category);

	FText const VarName = NodeSpawner->GetVariableName();
	// @TODO: NodeClass could be modified post Create()
	if (NodeClass != nullptr)
	{
		if (NodeClass->IsChildOf(UK2Node_VariableGet::StaticClass()))
		{
			MenuSignature.MenuName = FText::Format(LOCTEXT("LocalGetterMenuName", "Get {0}"), VarName);
			MenuSignature.Tooltip  = UK2Node_VariableGet::GetBlueprintVarTooltip(VarDesc);
		}
		else if (NodeClass->IsChildOf(UK2Node_VariableSet::StaticClass()))
		{
			MenuSignature.MenuName = FText::Format(LOCTEXT("LocalSetterMenuName", "Set {0}"), VarName);
			MenuSignature.Tooltip  = UK2Node_VariableSet::GetBlueprintVarTooltip(VarDesc);
		}
	}
	// add at least one character, so that PrimeDefaultUiSpec() doesn't 
	// attempt to query the template node
	if (MenuSignature.Keywords.IsEmpty())
	{
		// want to set it to something so we won't end up back in this condition
		MenuSignature.Keywords = FText::FromString(TEXT(" "));
	}
	MenuSignature.Icon = UK2Node_Variable::GetVarIconFromPinType(NodeSpawner->GetVarType(), MenuSignature.IconTint);

	return NodeSpawner;
}

//------------------------------------------------------------------------------
UBlueprintVariableNodeSpawner::UBlueprintVariableNodeSpawner(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void UBlueprintVariableNodeSpawner::Prime()
{
	// we expect that you don't need a node template to construct menu entries
	// from this, so we choose not to pre-cache one here
}

//------------------------------------------------------------------------------
FBlueprintNodeSignature UBlueprintVariableNodeSpawner::GetSpawnerSignature() const
{
	FBlueprintNodeSignature SpawnerSignature(NodeClass);
	if (IsUserLocalVariable())
	{
		SpawnerSignature.AddSubObject(LocalVarOuter);
		static const FName LocalVarSignatureKey(TEXT("LocalVarName"));
		SpawnerSignature.AddNamedValue(LocalVarSignatureKey, LocalVarDesc.VarName.ToString());
	}
	
	return SpawnerSignature;
}

//------------------------------------------------------------------------------
FBlueprintActionUiSpec UBlueprintVariableNodeSpawner::GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const
{
	UEdGraph* TargetGraph = (Context.Graphs.Num() > 0) ? Context.Graphs[0] : nullptr;
	FBlueprintActionUiSpec MenuSignature = PrimeDefaultUiSpec(TargetGraph);

	if (FProperty const* WrappedVariable = GetVarProperty())
	{
		checkSlow(Context.Blueprints.Num() > 0);
		UBlueprint const* TargetBlueprint = Context.Blueprints[0];

		// @TODO: this is duplicated in a couple places, move it to some shared resource
		UClass const* TargetClass = (TargetBlueprint->GeneratedClass != nullptr) ? TargetBlueprint->GeneratedClass : TargetBlueprint->ParentClass;
		for (UEdGraphPin* Pin : Context.Pins)
		{
			if ((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) && 
				 Pin->PinType.PinSubCategoryObject.IsValid())
			{
				TargetClass = CastChecked<UClass>(Pin->PinType.PinSubCategoryObject.Get());
			}
		}

		UClass const* EffectiveOwnerClass = WrappedVariable->GetOwnerClass();
		EffectiveOwnerClass = EffectiveOwnerClass ? EffectiveOwnerClass : ToRawPtr(OwnerClass);
		const bool bIsOwningClassValid = EffectiveOwnerClass && (!Cast<const UBlueprintGeneratedClass>(EffectiveOwnerClass) || EffectiveOwnerClass->ClassGeneratedBy); //todo: more general validation
		UClass const* VariableClass = bIsOwningClassValid ? EffectiveOwnerClass->GetAuthoritativeClass() : nullptr;
		if (VariableClass && (!TargetClass || !TargetClass->IsChildOf(VariableClass)))
		{
			MenuSignature.Category = FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Class,
				FText::FromString(VariableClass->GetDisplayNameText().ToString()));
		}
	}
	DynamicUiSignatureGetter.ExecuteIfBound(Context, Bindings, &MenuSignature);
	return MenuSignature;
}

//------------------------------------------------------------------------------
UEdGraphNode* UBlueprintVariableNodeSpawner::Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const
{
	UEdGraphNode* NewNode = nullptr;
	// @TODO: consider splitting out local variable spawners (since they don't 
	//        conform to UBlueprintFieldNodeSpawner
	if (IsLocalVariable())
	{
		auto LocalVarSetupLambda = [](UEdGraphNode* InNewNode, bool bIsTemplateNode, FName VarName, FFieldVariant VarOuter, FGuid VarGuid, FCustomizeNodeDelegate UserDelegate)
		{
			UK2Node_Variable* VarNode = CastChecked<UK2Node_Variable>(InNewNode);
			VarNode->VariableReference.SetLocalMember(VarName, VarOuter.GetName(), VarGuid);
			UserDelegate.ExecuteIfBound(InNewNode, bIsTemplateNode);
		};

		FCustomizeNodeDelegate PostSpawnDelegate = CustomizeNodeDelegate;
		if (FFieldVariant LocalVariableOuter = GetVarOuter())
		{
			PostSpawnDelegate = FCustomizeNodeDelegate::CreateStatic(LocalVarSetupLambda, IsUserLocalVariable() ? LocalVarDesc.VarName : GetField().GetFName(), LocalVariableOuter, LocalVarDesc.VarGuid, CustomizeNodeDelegate);
		}

		NewNode = UBlueprintNodeSpawner::SpawnNode<UEdGraphNode>(NodeClass, ParentGraph, Bindings, Location, PostSpawnDelegate);
	}
	else
	{
		NewNode = Super::Invoke(ParentGraph, Bindings, Location);
	}

	return NewNode;
}

//------------------------------------------------------------------------------
bool UBlueprintVariableNodeSpawner::IsUserLocalVariable() const
{
	return (LocalVarDesc.VarName != NAME_None);
}

//------------------------------------------------------------------------------
bool UBlueprintVariableNodeSpawner::IsLocalVariable() const
{
	return (LocalVarDesc.VarName != NAME_None) || (LocalVarOuter != nullptr);
}

//------------------------------------------------------------------------------
FFieldVariant UBlueprintVariableNodeSpawner::GetVarOuter() const
{
	FFieldVariant VarOuter;
	if (IsLocalVariable())
	{
		VarOuter = LocalVarOuter;
	}
	else if (FProperty const* MemberVariable = GetVarProperty())
	{
		VarOuter = MemberVariable->GetOwnerVariant();
	}	
	return VarOuter;
}

//------------------------------------------------------------------------------
FProperty const* UBlueprintVariableNodeSpawner::GetVarProperty() const
{
	// Get() does IsValid() checks for us
	return CastField<const FProperty>(GetField().ToField());
}

//------------------------------------------------------------------------------
FEdGraphPinType UBlueprintVariableNodeSpawner::GetVarType() const
{
	FEdGraphPinType VarType;
	if (IsUserLocalVariable())
	{
		VarType = LocalVarDesc.VarType;
	}
	else if (FProperty const* VarProperty = GetVarProperty())
	{
		UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->ConvertPropertyToPinType(VarProperty, VarType);
	}
	return VarType;
}

//------------------------------------------------------------------------------
FText UBlueprintVariableNodeSpawner::GetVariableName() const
{
	FText VarName;

	bool bShowFriendlyNames = GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
	if (IsUserLocalVariable())
	{
		VarName = bShowFriendlyNames ? FText::FromString(LocalVarDesc.FriendlyName) : FText::FromName(LocalVarDesc.VarName);
	}
	else if (FProperty const* MemberVariable = GetVarProperty())
	{
		VarName = bShowFriendlyNames ? FText::FromString(UEditorEngine::GetFriendlyName(MemberVariable)) : FText::FromName(MemberVariable->GetFName());
	}
	return VarName;
}

#undef LOCTEXT_NAMESPACE
