// Copyright Epic Games, Inc. All Rights Reserved.

#include "CallFunctionHandler.h"

#include "BlueprintCompilationManager.h"
#include "UObject/MetaData.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_Event.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_Self.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_CallFunction.h"

#include "EdGraphUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "KismetCastingUtils.h"
#include "KismetCompiler.h"
#include "Net/Core/PushModel/PushModelMacros.h"
#include "PushModelHelpers.h"

#if WITH_PUSH_MODEL
#include "Net/NetPushModelHelpers.h"
#endif

#define LOCTEXT_NAMESPACE "CallFunctionHandler"

namespace UE::K2NodeCallFunction::Private
{
	static bool IsCalledFunctionPure(const UEdGraphNode* Node)
	{
		if (const UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(Node))
		{
			return CallFunctionNode->IsNodePure();
		}
		
		return false;
	}

	// REMOVE IN 5.9
	// 
	// IsMutableContainerFunction and IsMutableContainerInput are used to permit connections between mutable array functions and read-only inputs.
	// This will eventually become an error, but users might still have content that accidentally modifies read-only arrays.
	// For now, we'll issue a warning and give them a chance to fix their content.

	static bool IsMutableContainerFunction(UFunction* Function)
	{
		check(Function);

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		check(K2Schema);

		return K2Schema->AllowedMutableContainerFunctions.Contains(FAllowedMutableContainerFunction{Function, TEXT("")});
	}

	static bool IsMutableContainerInput(FName InputName, UFunction* Function)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		check(K2Schema);

		if (const FAllowedMutableContainerFunction* Entry = K2Schema->AllowedMutableContainerFunctions.Find(FAllowedMutableContainerFunction{Function, TEXT("")}))
		{
			return (InputName == Entry->ContainerParameterName);
		}

		return false;
	}

	static bool FindMatchingReferencedNetOrFieldNotifyPropertyAndPin(TArray<UEdGraphPin*>& RemainingPins, FProperty* FunctionProperty, FProperty*& NetProperty, FProperty*& FieldNotifyProperty, UEdGraphPin*& PropertyObjectPin)
	{
		NetProperty = nullptr;
		FieldNotifyProperty = nullptr;
		PropertyObjectPin = nullptr;

		const bool bIsValidProperty =
			(FunctionProperty->HasAllPropertyFlags(CPF_OutParm | CPF_ReferenceParm) && !FunctionProperty->HasAnyPropertyFlags(CPF_ReturnParm | CPF_ConstParm)) ||
			IsMutableContainerInput(FunctionProperty->GetFName(), FunctionProperty->GetTypedOwner<UFunction>())
		;

		if (bIsValidProperty)
		{
			for (int32 i = 0; i < RemainingPins.Num(); ++i)
			{
				if (FunctionProperty->GetFName() == RemainingPins[i]->PinName)
				{
					bool bResult = false;
					UEdGraphPin* ParamPin = RemainingPins[i];
					RemainingPins.RemoveAtSwap(i);
					if (UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(ParamPin))
					{
						// TODO: Should we traverse pin links to find references somehow?
						//			E.G., How are select statements that pass through references
						//					to net properties handled?

						if (UK2Node_VariableGet* GetPropertyNode = Cast<UK2Node_VariableGet>(PinToTry->GetOwningNode()))
						{
							if (FProperty* ToCheck = GetPropertyNode->GetPropertyForVariable())
							{
								if (UNLIKELY(FKismetCompilerUtilities::IsPropertyUsesFieldNotificationSetValueAndBroadcast(ToCheck)))
								{
									FieldNotifyProperty = ToCheck;
									PropertyObjectPin = GetPropertyNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
									bResult = true;
								}
								if (UNLIKELY(ToCheck->HasAnyPropertyFlags(CPF_Net)))
								{
									NetProperty = ToCheck;
									PropertyObjectPin = GetPropertyNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
									bResult = true;
								}
							}
						}
					}

					return bResult;
				}
			}
		}

		return false;
	}
}

//////////////////////////////////////////////////////////////////////////
// FImportTextErrorContext

// Support class to pipe logs from FProperty->ImportText (for struct literals) to the message log as warnings
class FImportTextErrorContext : public FOutputDevice
{
protected:
	FCompilerResultsLog& MessageLog;
	UObject* TargetObject;
public:
	int32 NumErrors;

	FImportTextErrorContext(FCompilerResultsLog& InMessageLog, UObject* InTargetObject)
		: FOutputDevice()
		, MessageLog(InMessageLog)
		, TargetObject(InTargetObject)
		, NumErrors(0)
	{
	}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		if (TargetObject == NULL)
		{
			MessageLog.Error(V);
		}
		else
		{
			const FString ErrorString = FString::Printf(TEXT("Invalid default on node @@: %s"), V);
			MessageLog.Error(*ErrorString, TargetObject);		
		}
		NumErrors++;
	}
};

//////////////////////////////////////////////////////////////////////////
// FKCHandler_CallFunction

#ifdef _MSC_VER
#pragma warning(push)
#pragma warning(disable : 4750)
#endif

/**
 * Searches for the function referenced by a graph node in the CallingContext class's list of functions,
 * validates that the wiring matches up correctly, and creates an execution statement.
 */
void FKCHandler_CallFunction::CreateFunctionCallStatement(FKismetFunctionContext& Context, UEdGraphNode* Node, UEdGraphPin* SelfPin)
{
	using namespace UE::KismetCompiler;

	int32 NumErrorsAtStart = CompilerContext.MessageLog.NumErrors;

	// Find the function, starting at the parent class
	if (UFunction* Function = FindFunction(Context, Node))
	{
		CheckIfFunctionIsCallable(Function, Context, Node);
		// Make sure the pin mapping is sound (all pins wire up to a matching function parameter, and all function parameters match a pin)

		// Remaining unmatched pins
		// Note: Should maintain a stable order for variadic arguments
		TArray<UEdGraphPin*> RemainingPins;
		RemainingPins.Append(Node->Pins);

		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

		// Remove expected exec and self pins
		RemainingPins.RemoveAll([Schema](UEdGraphPin* Pin) { return (Pin->bOrphanedPin || Schema->IsMetaPin(*Pin)); });

		// Check for magic pins
		const bool bIsLatent = Function->HasMetaData(FBlueprintMetadata::MD_Latent);
		if (bIsLatent && (CompilerContext.UbergraphContext != &Context))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ContainsLatentCall_Error", "@@ contains a latent call, which cannot exist outside of the event graph").ToString(), Node);
		}

		UEdGraphPin* LatentInfoPin = nullptr;

		if (TMap<FName, FString>* MetaData = FMetaData::GetMapForObject(Function))
		{
			for (TMap<FName, FString>::TConstIterator It(*MetaData); It; ++It)
			{
				const FName& Key = It.Key();

				if (Key == TEXT("LatentInfo"))
				{
					UEdGraphPin* Pin = Node->FindPin(It.Value());
					if (Pin && (Pin->Direction == EGPD_Input) && (Pin->LinkedTo.Num() == 0))
					{
						LatentInfoPin = Pin;

						UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(Pin);
						if (FBPTerminal** Term = Context.NetMap.Find(PinToTry))
						{
							check((*Term)->bIsLiteral);
						
							int32 LatentUUID = CompilerContext.MessageLog.CalculateStableIdentifierForLatentActionManager(LatentInfoPin->GetOwningNode());

							const FString ExecutionFunctionName = UEdGraphSchema_K2::FN_ExecuteUbergraphBase.ToString() + TEXT("_") + Context.Blueprint->GetName();
							(*Term)->Name = FString::Printf(TEXT("(Linkage=%s,UUID=%s,ExecutionFunction=%s,CallbackTarget=None)"), *FString::FromInt(INDEX_NONE), *FString::FromInt(LatentUUID), *ExecutionFunctionName);

							// Record the UUID in the debugging information
							UEdGraphNode* TrueSourceNode = Cast<UEdGraphNode>(Context.MessageLog.FindSourceObject(Node));
							Context.NewClass->GetDebugData().RegisterUUIDAssociation(TrueSourceNode, LatentUUID);
						}
					}
					else
					{
						CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FindPinFromLinkage_ErrorFmt", "Function {0} (called from @@) was specified with LatentInfo metadata but does not have a pin named {1}"),
							FText::FromString(Function->GetName()), FText::FromString(It.Value())).ToString(), Node);
					}
				}
			}
		}

		// Parameter info to be stored, and assigned to all function call statements generated below
		FBPTerminal* LHSTerm = nullptr;
		TArray<FBPTerminal*> RHSTerms;
		UEdGraphPin* ThenExecPin = nullptr;
		UEdGraphNode* LatentTargetNode = nullptr;
		int32 LatentTargetParamIndex = INDEX_NONE;

		// Grab the special case structs that use their own literal path
		UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		UScriptStruct* Vector3fStruct = TVariantStructure<FVector3f>::Get();
		UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
		UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();

		// If a function parameter needs an implicit double<->float cast *and* it's a non-const reference,
		// then we need to copy the value of the casted temporary *back* to its source.
		// 
		// Just to illustrate the scenario, take the following example in pseudocode:
		// 
		//     double Input = 2.0
		//     float CastedInput = (float)Input					; Narrowing conversion needed for function input
		//     NativeFunctionWithReferenceParam(CastedInput)	; CastedInput has possibly changed since this function takes a float&
		//     Input = (double)CastedInput						; Now we need to propagate that change back to Input
		//
		using CastEntryT = TPair<FBPTerminal*, CastingUtils::FImplicitCastParams>;
		TArray<CastEntryT> ModifiedCastInputs;

		// Check each property
		bool bMatchedAllParams = true;
		for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & CPF_Parm); ++It)
		{
			FProperty* Property = *It;

			bool bFoundParam = false;
			for (int32 i = 0; !bFoundParam && (i < RemainingPins.Num()); ++i)
			{
				UEdGraphPin* PinMatch = RemainingPins[i];
				if (Property->GetFName() == PinMatch->PinName)
				{
					// Found a corresponding pin, does it match in type and direction?
					if (UK2Node_CallFunction::IsStructureWildcardProperty(Function, Property->GetFName()) ||
						FKismetCompilerUtilities::IsTypeCompatibleWithProperty(PinMatch, Property, CompilerContext.MessageLog, CompilerContext.GetSchema(), Context.NewClass))
					{
						UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(PinMatch);

						if (FBPTerminal** Term = Context.NetMap.Find(PinToTry))
						{
							// For literal structs, we have to verify the default here to make sure that it has valid formatting
							if( (*Term)->bIsLiteral && (PinMatch != LatentInfoPin))
							{
								FStructProperty* StructProperty = CastField<FStructProperty>(Property);
								if( StructProperty )
								{
									UScriptStruct* Struct = StructProperty->Struct;
									if( Struct != VectorStruct
										&& Struct != Vector3fStruct
										&& Struct != RotatorStruct
										&& Struct != TransformStruct )
									{
										// Ensure all literal struct terms can be imported if its empty
										if ( (*Term)->Name.IsEmpty() )
										{
											(*Term)->Name = TEXT("()");
										}

										int32 StructSize = Struct->GetStructureSize();
										[this, StructSize, StructProperty, Node, Term, &bMatchedAllParams]()
										{
											uint8* StructData = (uint8*)FMemory_Alloca(StructSize);
											StructProperty->InitializeValue(StructData);

											// Import the literal text to a dummy struct to verify it's well-formed
											FImportTextErrorContext ErrorPipe(CompilerContext.MessageLog, Node);
											StructProperty->ImportText_Direct(*((*Term)->Name), StructData, nullptr, 0, &ErrorPipe);
											if(ErrorPipe.NumErrors > 0)
											{
												bMatchedAllParams = false;
											}
										}();
									}
									
								}
							}

							if (Property->HasAnyPropertyFlags(CPF_ReturnParm))
							{
								LHSTerm = *Term;
							}
							else
							{
								FBPTerminal* RHSTerm = *Term;

								// if this term is an object that needs to be cast to an interface
								if (FBPTerminal** InterfaceTerm = InterfaceTermMap.Find(PinMatch))
								{
									UClass* InterfaceClass = CastChecked<UClass>(PinMatch->PinType.PinSubCategoryObject.Get());

									FBPTerminal* ClassTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
									ClassTerm->Name       = InterfaceClass->GetName();
									ClassTerm->bIsLiteral = true;
									ClassTerm->Source     = Node;
									ClassTerm->ObjectLiteral = InterfaceClass;
									ClassTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Class;

									// insert a cast op before a call to the function (and replace
									// the param with the result from the cast)
									FBlueprintCompiledStatement& CastStatement = Context.AppendStatementForNode(Node);
									CastStatement.Type = InterfaceClass->HasAnyClassFlags(CLASS_Interface) ? KCST_CastObjToInterface : KCST_CastInterfaceToObj;
									CastStatement.LHS = *InterfaceTerm;
									CastStatement.RHS.Add(ClassTerm);
									CastStatement.RHS.Add(*Term);

									RHSTerm = *InterfaceTerm;
								}

								if (const CastingUtils::FImplicitCastParams* CastParams = Context.ImplicitCastMap.Find(PinMatch))
								{
									CastingUtils::InsertImplicitCastStatement(Context, *CastParams, RHSTerm);

									FBPTerminal* ImplicitCastTerm = CastParams->TargetTerminal;
									check(ImplicitCastTerm);

									const bool bIsNonConstReference =
										Property->HasAllPropertyFlags(CPF_OutParm | CPF_ReferenceParm) &&
										!Property->HasAllPropertyFlags(CPF_ConstParm);

									if (bIsNonConstReference)
									{
										CastingUtils::FImplicitCastParams& InverseCastParams = ModifiedCastInputs.Emplace_GetRef(ImplicitCastTerm, *CastParams).Value;

										const CastingUtils::FloatingPointCastType CastType = CastParams->Conversion.Type;
										if (CastType == CastingUtils::FloatingPointCastType::FloatToDouble)
										{
											InverseCastParams.Conversion.Type = CastingUtils::FloatingPointCastType::DoubleToFloat;
										}
										else if (CastType == CastingUtils::FloatingPointCastType::DoubleToFloat)
										{
											InverseCastParams.Conversion.Type = CastingUtils::FloatingPointCastType::FloatToDouble;
										}
										InverseCastParams.TargetTerminal = RHSTerm;
									}

									Context.ImplicitCastMap.Remove(PinMatch);

									RHSTerm = ImplicitCastTerm;
								}

								int32 ParameterIndex = RHSTerms.Add(RHSTerm);

								if (PinMatch == LatentInfoPin)
								{
									// Record the (latent) output impulse from this node
									ThenExecPin = CompilerContext.GetSchema()->FindExecutionPin(*Node, EGPD_Output);

									if( ThenExecPin && (ThenExecPin->LinkedTo.Num() > 0) )
									{
										LatentTargetNode = ThenExecPin->LinkedTo[0]->GetOwningNode();
									}

									if (LatentTargetNode)
									{
										LatentTargetParamIndex = ParameterIndex;
									}
								}
							}

							// Make sure it isn't trying to modify a const term
							if (Property->HasAnyPropertyFlags(CPF_OutParm) && !((*Term)->IsTermWritable()))
							{
								if (Property->HasAnyPropertyFlags(CPF_ReferenceParm))
								{
									if (!Property->HasAnyPropertyFlags(CPF_ConstParm))
									{
										if (UE::K2NodeCallFunction::Private::IsMutableContainerFunction(Function))
										{
											if (Schema->bEnableMutableContainerWarnings)
											{
												CompilerContext.MessageLog.Warning(*LOCTEXT("MutatingReadOnlyData_Warning", "Array is changing read-only data @@. This will become an error in a future engine release.").ToString(), (*Term)->SourcePin);
											}
										}
										else
										{
											CompilerContext.MessageLog.Error(*LOCTEXT("PassReadOnlyReferenceParam_Error", "Cannot pass a read-only variable to a reference parameter @@").ToString(), PinMatch);
										}
									}
								}
								else
								{
									CompilerContext.MessageLog.Error(*LOCTEXT("PassReadOnlyOutputParam_Error", "Cannot pass a read-only variable to a output parameter @@").ToString(), PinMatch);
								}
							}
						}
						else
						{
							CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermPassed_Error", "Failed to resolve term passed into @@").ToString(), PinMatch);
							bMatchedAllParams = false;
						}
					}
					else
					{
						bMatchedAllParams = false;
					}

					bFoundParam = true;
					RemainingPins.RemoveAt(i);
				}
			}

			if (!bFoundParam)
			{
				CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FindPinParameter_ErrorFmt", "Could not find a pin for the parameter {0} of {1} on @@"), FText::FromString(Property->GetName()), FText::FromString(Function->GetName())).ToString(), Node);
				bMatchedAllParams = false;
			}
		}

		// If we have pins remaining then it's either an error, or extra variadic terms that need to be emitted
		if (RemainingPins.Num() > 0)
		{
			const bool bIsVariadic = Function->HasMetaData(FBlueprintMetadata::MD_Variadic);
			if (bIsVariadic)
			{
				// Add a RHS term for every remaining pin
				for (UEdGraphPin* RemainingPin : RemainingPins)
				{
					// Variadic pins are assumed to be wildcard pins that have been connected to something else
					if (RemainingPin->LinkedTo.Num() == 0)
					{
						CompilerContext.MessageLog.Error(*LOCTEXT("UnlinkedVariadicPin_Error", "The variadic pin @@ must be connected. Connect something to @@.").ToString(), RemainingPin, RemainingPin->GetOwningNodeUnchecked());
						continue;
					}

					UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(RemainingPin);
					if (FBPTerminal** Term = Context.NetMap.Find(PinToTry))
					{
						FBPTerminal* RHSTerm = *Term;
						RHSTerms.Add(RHSTerm);
					}
					else
					{
						CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermVariadic_Error", "Failed to resolve variadic term passed into @@").ToString(), RemainingPin);
						bMatchedAllParams = false;
					}
				}
			}
			else
			{
				// At this point, we should have consumed all pins.  If not, there are extras that need to be removed.
				for (const UEdGraphPin* RemainingPin : RemainingPins)
				{
					CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("PinMismatchParameter_ErrorFmt", "Pin @@ named {0} doesn't match any parameters of function {1}"), FText::FromName(RemainingPin->PinName), FText::FromString(Function->GetName())).ToString(), RemainingPin);
				}
			}
		}

		if (NumErrorsAtStart == CompilerContext.MessageLog.NumErrors)
		{
			// Build up a list of contexts that this function will be called on
			TArray<FBPTerminal*> ContextTerms;
			if (SelfPin)
			{
				const bool bIsConstSelfContext = Context.IsConstFunction();
				const bool bIsNonConstFunction = !Function->HasAnyFunctionFlags(FUNC_Const|FUNC_Static);
				const bool bEnforceConstCorrectness = Context.EnforceConstCorrectness();
				auto CheckAndAddSelfTermLambda = [this, &Node, &ContextTerms, bIsConstSelfContext, bIsNonConstFunction, bEnforceConstCorrectness](FBPTerminal* Target)
				{
					bool bIsSelfTerm = true;
					if(Target != nullptr)
					{
						const UEdGraphPin* SourcePin = Target->SourcePin;
						bIsSelfTerm = (SourcePin == nullptr || CompilerContext.GetSchema()->IsSelfPin(*SourcePin));
					}

					// Ensure const correctness within the context of the function call:
					//	a) Attempting to call a non-const, non-static function within a const function graph (i.e. 'const self' as context)
					//	b) Attempting to call a non-const, non-static function with a 'const' term linked to the target pin as the function context
					if(bIsSelfTerm && bIsConstSelfContext && bIsNonConstFunction)
					{
						// If we're not enforcing const correctness in this context, emit a warning here rather than an error, and allow compilation of this statement to proceed
						if(Target != nullptr)
						{
							if(bEnforceConstCorrectness)
							{
								CompilerContext.MessageLog.Error(*LOCTEXT("NonConstFunctionCallOnReadOnlyTarget_Error", "Function @@ can modify state and cannot be called on @@ because it is a read-only Target in this context").ToString(), Node, Target->Source);
							}
							else
							{
								CompilerContext.MessageLog.Warning(*LOCTEXT("NonConstFunctionCallOnReadOnlyTarget_Warning", "Function @@ can modify state and should not be called on @@ because it is considered to be a read-only Target in this context").ToString(), Node, Target->Source);
							}
						}
						else
						{
							if(bEnforceConstCorrectness)
							{
								CompilerContext.MessageLog.Error(*LOCTEXT("NonConstFunctionCallOnReadOnlySelfScope_Error", "Function @@ can modify state and cannot be called on 'self' because it is a read-only Target in this context").ToString(), Node);
							}
							else
							{
								CompilerContext.MessageLog.Warning(*LOCTEXT("NonConstFunctionCallOnReadOnlySelfScope_Warning", "Function @@ can modify state and should not be called on 'self' because it is considered to be a read-only Target in this context").ToString(), Node);
							}
						}
					}

					ContextTerms.Add(Target);
				};

				if( SelfPin->LinkedTo.Num() > 0 )
				{
					for(int32 i = 0; i < SelfPin->LinkedTo.Num(); i++)
					{
						FBPTerminal** pContextTerm = Context.NetMap.Find(SelfPin->LinkedTo[i]);
						if(ensureMsgf(pContextTerm != nullptr, TEXT("'%s' is missing a target input - if this is a server build, the input may be a cosmetic only property which was discarded (if this is the case, and this is expecting component variable try resaving.)"), *Node->GetPathName()))
						{
							CheckAndAddSelfTermLambda(*pContextTerm);
						}
					}
				}
				else
				{
					FBPTerminal** pContextTerm = Context.NetMap.Find(SelfPin);
					CheckAndAddSelfTermLambda((pContextTerm != nullptr) ? *pContextTerm : nullptr);
				}
			}

			// Check for a call into the ubergraph, which will require a patchup later on for the exact state entry point
			UEdGraphNode** pSrcEventNode = NULL;
			if (!bIsLatent)
			{
				pSrcEventNode = CompilerContext.CallsIntoUbergraph.Find(Node);
			}

			// Iterate over all the contexts this functions needs to be called on, and emit a call function statement for each
			FBlueprintCompiledStatement* LatentStatement = nullptr;
			for (FBPTerminal* Target : ContextTerms)
			{
				// Currently, call site nodes will (incorrectly) expose the target pin as an interface type for calls to
				// interface functions that are implemented by the owning class, so in that case we need to flag that the
				// calling context is an interface if the target pin is also linked to an interface pin type (e.g. result
				// of a cast node). Otherwise, we'll infer the wrong context type at runtime and corrupt the stack by
				// reading an interface ptr (16 bytes) into an object ptr (8 bytes) when we process the context opcode.
				const bool bIsInterfaceContextTerm = Target && Target->AssociatedVarProperty && Target->AssociatedVarProperty->IsA<FInterfaceProperty>();
				
				UClass* FunctionOwnerClass = Function->GetOwnerClass();
				const bool bIsInterfaceFunc = FunctionOwnerClass ? FunctionOwnerClass->HasAnyClassFlags(CLASS_Interface) : false;

				FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
				Statement.FunctionToCall = Function;
				Statement.FunctionContext = Target;
				Statement.Type = KCST_CallFunction;
				Statement.bIsInterfaceContext = bIsInterfaceFunc || bIsInterfaceContextTerm;
				Statement.bIsParentContext = Node->IsA<UK2Node_CallParentFunction>();

				Statement.LHS = LHSTerm;
				Statement.RHS = RHSTerms;

				if (!bIsLatent)
				{
					// Fixup ubergraph calls
					if (pSrcEventNode)
					{
						UEdGraphPin* ExecOut = CompilerContext.GetSchema()->FindExecutionPin(**pSrcEventNode, EGPD_Output);

						check(CompilerContext.UbergraphContext);
						CompilerContext.UbergraphContext->GotoFixupRequestMap.Add(&Statement, ExecOut);
						Statement.UbergraphCallIndex = 0;
					}
				}
				else
				{
					// Fixup latent functions
					if (LatentTargetNode && (Target == ContextTerms.Last()))
					{
						check(LatentTargetParamIndex != INDEX_NONE);
						Statement.UbergraphCallIndex = LatentTargetParamIndex;
						Context.GotoFixupRequestMap.Add(&Statement, ThenExecPin);
						LatentStatement = &Statement;
					}
				}

				AdditionalCompiledStatementHandling(Context, Node, Statement);

				if(Statement.Type == KCST_CallFunction && Function->HasAnyFunctionFlags(FUNC_Delegate))
				{
					CompilerContext.MessageLog.Error(*LOCTEXT("CallingDelegate_Error", "@@ is trying to call a delegate function - delegates cannot be called directly").ToString(), Node);
					// Sanitize the statement, this would have ideally been detected earlier but we need
					// to run AdditionalCompiledStatementHandling to satisify the DelegateNodeHandler
					// implementation:
					Statement.Type = KCST_CallDelegate;
				}
			}

			{
				for (const auto& It : ModifiedCastInputs)
				{
					FBPTerminal* LocalRHSTerm = It.Get<0>();
					CastingUtils::FImplicitCastParams LocalInverseCastParams = It.Get<1>();

					CastingUtils::InsertImplicitCastStatement(Context, LocalInverseCastParams, LocalRHSTerm);
				}
			}

			// Create the exit from this node if there is one
			if (bIsLatent)
			{
				// End this thread of execution; the latent function will resume it at some point in the future
				FBlueprintCompiledStatement& PopStatement = Context.AppendStatementForNode(Node);
				PopStatement.Type = KCST_EndOfThread;
			}
			else
			{
				// Generate the output impulse from this node
				if (!UE::K2NodeCallFunction::Private::IsCalledFunctionPure(Node))
				{
					GenerateSimpleThenGoto(Context, *Node);
				}
			}
		}
	}
	else
	{
		FString WarningMessage = FText::Format(LOCTEXT("FindFunction_ErrorFmt", "Could not find the function '{0}' called from @@"), FText::FromString(GetFunctionNameFromNode(Node).ToString())).ToString();
		CompilerContext.MessageLog.Warning(*WarningMessage, Node);
	}
}

UClass* FKCHandler_CallFunction::GetCallingContext(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	// Find the calling scope
	UClass* SearchScope = Context.NewClass;
	
	if (UK2Node_CallParentFunction* ParentCall = Cast<UK2Node_CallParentFunction>(Node))
	{
		// Special Case: super call functions should search up their class hierarchy, and find the first legitimate implementation of the function
		const FName FuncName = ParentCall->FunctionReference.GetMemberName();
		UClass* SearchContext = Context.NewClass->GetSuperClass();

		UFunction* ParentFunc = nullptr;
		if (SearchContext)
		{
			ParentFunc = SearchContext->FindFunctionByName(FuncName);
		}

		return ParentFunc ? ParentFunc->GetOuterUClass() : nullptr;
	}
	else
	{
		if (UEdGraphPin* SelfPin = CompilerContext.GetSchema()->FindSelfPin(*Node, EGPD_Input))
		{
			SearchScope = Cast<UClass>(Context.GetScopeFromPinType(SelfPin->PinType, Context.NewClass));
		}
	}

	return SearchScope;
}

UClass* FKCHandler_CallFunction::GetTrueCallingClass(FKismetFunctionContext& Context, UEdGraphPin* SelfPin)
{
	if (SelfPin)
	{
		// TODO: here FBlueprintCompiledStatement::GetScopeFromPinType should be called, but since FEdGraphPinType::PinSubCategory is not always initialized properly that function works wrong
		// return Cast<UClass>(Context.GetScopeFromPinType(SelfPin->PinType, Context.NewClass));
		FEdGraphPinType& Type = SelfPin->PinType;
		if ((Type.PinCategory == UEdGraphSchema_K2::PC_Object) || (Type.PinCategory == UEdGraphSchema_K2::PC_Class) || (Type.PinCategory == UEdGraphSchema_K2::PC_Interface))
		{
			if (!Type.PinSubCategory.IsNone() && (Type.PinSubCategory != UEdGraphSchema_K2::PSC_Self))
			{
				return Cast<UClass>(Type.PinSubCategoryObject.Get());
			}
		}
	}
	return Context.NewClass;
}

void FKCHandler_CallFunction::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	check(Node);

	if (UFunction* Function = FindFunction(Context, Node))
	{
		TArray<FName> DefaultToSelfParamNames;
		TArray<FName> RequiresSetValue;

		if (Function->HasMetaData(FBlueprintMetadata::MD_DefaultToSelf))
		{
			const FName DefaltToSelfPinName = *Function->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);

			DefaultToSelfParamNames.Add(DefaltToSelfPinName);
		}
		if (Function->HasMetaData(FBlueprintMetadata::MD_WorldContext))
		{
			UEdGraphSchema_K2 const* K2Schema = CompilerContext.GetSchema();
			const bool bHasIntrinsicWorldContext = !K2Schema->IsStaticFunctionGraph(Context.SourceGraph) && FBlueprintEditorUtils::ImplementsGetWorld(Context.Blueprint);

			const FName WorldContextPinName = *Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);

			if (bHasIntrinsicWorldContext)
			{
				DefaultToSelfParamNames.Add(WorldContextPinName);
			}
			else if (!Function->HasMetaData(FBlueprintMetadata::MD_CallableWithoutWorldContext))
			{
				RequiresSetValue.Add(WorldContextPinName);
			}
		}

		for (UEdGraphPin* Pin : Node->Pins)
		{
			const bool bIsConnected = (Pin->LinkedTo.Num() != 0);

			// if this pin could use a default (it doesn't have a connection or default of its own)
			if (!bIsConnected && (Pin->DefaultObject == nullptr))
			{
				if (DefaultToSelfParamNames.Contains(Pin->PinName) && FKismetCompilerUtilities::ValidateSelfCompatibility(Pin, Context))
				{
					ensure(Pin->PinType.PinSubCategoryObject != nullptr);
					ensure((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface));

					FBPTerminal* Term = Context.RegisterLiteral(Pin);
					Term->Type.PinSubCategory = UEdGraphSchema_K2::PN_Self;
					Context.NetMap.Add(Pin, Term);
				}
				else if (RequiresSetValue.Contains(Pin->PinName))
				{
					CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "PinMustHaveConnection_Error", "Pin @@ must have a connection").ToString(), Pin);
				}
			}
		}
	}

	for (UEdGraphPin* Pin : Node->Pins)
	{
		check(Pin);

		if ((Pin->Direction != EGPD_Input) || (Pin->LinkedTo.Num() == 0))
		{
			continue;
		}

		// if we have an object plugged into an interface pin, let's create a 
		// term that'll be used as an intermediate, holding the result of a cast 
		// from object to interface
		if (((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) && (Pin->LinkedTo[0]->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)) ||
			((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) && (Pin->LinkedTo[0]->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)))
		{
			FBPTerminal* InterfaceTerm = Context.CreateLocalTerminal();
			InterfaceTerm->CopyFromPin(Pin, Context.NetNameMap->MakeValidName(Pin, TEXT("CastInput")));
			InterfaceTerm->Source = Node;

			InterfaceTermMap.Add(Pin, InterfaceTerm);
		}
	}

	FNodeHandlingFunctor::RegisterNets(Context, Node);
}

void FKCHandler_CallFunction::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	// This net is an output from a function call
	FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
	Context.NetMap.Add(Net, Term);
}

UFunction* FKCHandler_CallFunction::FindFunction(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UClass* CallingContext = GetCallingContext(Context, Node);

	if (CallingContext)
	{
		const UBlueprint* BlueprintContext = UBlueprint::GetBlueprintFromClass(CallingContext);

		// Redirect the calling context to the most up-to-date class (when not up-to-date,
		// this will redirect to the Blueprint's skeleton class)
		// It may be advisable to always do this branch in GetMostUpToDateClass, but
		// being conservative:
		if (!BlueprintContext || (BlueprintContext->bBeingCompiled && !FBlueprintCompilationManager::IsGeneratedClassLayoutReady()) || (!BlueprintContext->bBeingCompiled && !BlueprintContext->IsUpToDate()))
		{
			CallingContext = FBlueprintEditorUtils::GetMostUpToDateClass(CallingContext);
		}

		const FName FunctionName = GetFunctionNameFromNode(Node);
		return CallingContext->FindFunctionByName(FunctionName);
	}

	return nullptr;
}

void FKCHandler_CallFunction::Transform(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	// Add an object reference pin for this call
	UK2Node_CallFunction* CallFuncNode = Cast<UK2Node_CallFunction>(Node);
	if (!CallFuncNode)
	{
		return;
	}

	const bool bIsPure = CallFuncNode->IsNodePure();
	bool bIsPureAndNoUsedOutputs = false;
	if (bIsPure)
	{
		// Flag for removal if pure and there are no consumers of the outputs
		//@TODO: This isn't recursive (and shouldn't be here), it'll just catch the last node in a line of pure junk
		bool bAnyOutputsUsed = false;
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Node->Pins[PinIndex];
			if ((Pin->Direction == EGPD_Output) && (Pin->LinkedTo.Num() > 0))
			{
				bAnyOutputsUsed = true;
				break;
			}
		}

		if (!bAnyOutputsUsed)
		{
			//@TODO: Remove this node, not just warn about it
			bIsPureAndNoUsedOutputs = true;
		}
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Find the function, starting at the parent class
	if (UFunction* Function = FindFunction(Context, Node))
	{
		const bool bIsLatent = Function->HasMetaData(FBlueprintMetadata::MD_Latent);
		if (bIsLatent)
		{
			UEdGraphPin* OldOutPin = K2Schema->FindExecutionPin(*CallFuncNode, EGPD_Output);

			if ((OldOutPin != NULL) && (OldOutPin->LinkedTo.Num() > 0))
			{
				// Create a dummy execution sequence that will be the target of the return call from the latent action
				UK2Node_ExecutionSequence* DummyNode = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(CallFuncNode);
				DummyNode->AllocateDefaultPins();

				// Wire in the dummy node
				UEdGraphPin* NewInPin = K2Schema->FindExecutionPin(*DummyNode, EGPD_Input);
				UEdGraphPin* NewOutPin = K2Schema->FindExecutionPin(*DummyNode, EGPD_Output);

				if ((NewInPin != NULL) && (NewOutPin != NULL))
				{
					CompilerContext.MessageLog.NotifyIntermediatePinCreation(NewOutPin, OldOutPin);

					while (OldOutPin->LinkedTo.Num() > 0)
					{
						UEdGraphPin* LinkedPin = OldOutPin->LinkedTo[0];

						LinkedPin->BreakLinkTo(OldOutPin);
						LinkedPin->MakeLinkTo(NewOutPin);
					}

					OldOutPin->MakeLinkTo(NewInPin);
				}
			}
		}

		/**
		 * This code is for property dirty tracking.
		 * It works by injecting in extra nodes while compiling that will call UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex.
		 * See FKCPushModelHelpers::ConstructMarkDirtyNodeForProperty for node generation.
		 *
		 * If we're passing net properties as reference variables, there's no guarantee what the function will do,
		 * so we won't actually know whether or not the value has changed. In that case we'll go ahead
		 * and mark the property as dirty.
		 */

		// If the function is pure but won't actually be evaluated, if there are no out params,
		// or there are no input pins, then we don't need to worry about any extra generation
		// because there will either be no way to reference a NetProperty, or the node won't
		// have any effect.
		if (!bIsPureAndNoUsedOutputs && Function->NumParms > 0 && Function->HasAllFunctionFlags(FUNC_HasOutParms))
		{
			// We don't care about returns (non-ref out params, or const-ref out params), because those would
			// require a Set call to change any net property, and that will already have the appropriate nodes
			// generated.
			//
			// Additionally, we only need to care about pins that are actually connected to something.
			TArray<UEdGraphPin*> RemainingPins;
			RemainingPins.Reserve(Node->Pins.Num());
			for (UEdGraphPin* Pin : Node->Pins)
			{
				const bool bIsMutableContainerInput = 
					UE::K2NodeCallFunction::Private::IsMutableContainerInput(Pin->GetFName(), Function)
				;

				const bool bIsNonConstReference =
					(!Pin->PinType.bIsConst && Pin->PinType.bIsReference) ||
					bIsMutableContainerInput
				;

				if (Pin->Direction == EGPD_Input &&
					bIsNonConstReference &&
					Pin->LinkedTo.Num() > 0)
				{
					RemainingPins.Add(Pin);
				}
			}

			if (RemainingPins.Num() > 0)
			{
				FProperty* NetProperty = nullptr;
				FProperty* FieldNotifyProperty = nullptr;
				UEdGraphPin* PropertyObjectPin = nullptr;
				UEdGraphPin* OldThenPin = CallFuncNode->GetThenPin();

				// Note: This feels like it's going to be a very hot path during compilation as it will be hit for every
				// CallFunction node. Any optimizations that can be made here are probably worth.

				// Iterate the properties looking for Out Params that are tied to Net Properties.
				// This is similar to the loop in CreateCallFunction
				for (TFieldIterator<FProperty> It(Function); It; ++It)
				{
					if (UE::K2NodeCallFunction::Private::FindMatchingReferencedNetOrFieldNotifyPropertyAndPin(RemainingPins, *It, NetProperty, FieldNotifyProperty, PropertyObjectPin))
					{
						if (bIsPure)
						{
							// TODO: JDN - Reenable this when we can discern Const from Non Const Pure Refs.
							// Don't need to cause BP Spam since no one is using this right now.
							// 	CompilerContext.MessageLog.Warning(*NSLOCTEXT("KismetCompiler", "PurePushModel_Warning", "@@ is a pure node with references to a net property. The property may not be marked dirty in the correct frame. Consider making the function impure to solve the problem.").ToString(), Node);
							break;
						}

						if (FieldNotifyProperty)
						{
							const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
							check(Schema);
							const bool bNeedsFieldNotifyWarning =
								!UE::K2NodeCallFunction::Private::IsMutableContainerFunction(Function) ||
								Schema->bEnableMutableContainerWarnings
							;

							if (bNeedsFieldNotifyWarning)
							{
								CompilerContext.MessageLog.Warning(*NSLOCTEXT("KismetCompiler", "RefFieldNotifyProperty_Warning", "@@ references a Field Notify property. The property may not be Broadcast correctly. Consider using a temporary variable.").ToString(), Node);
							}
						}

						if (NetProperty)
						{
							if (UEdGraphNode* MarkPropertyDirtyNode = FKCPushModelHelpers::ConstructMarkDirtyNodeForProperty(Context, NetProperty, PropertyObjectPin))
							{
								// bool bWereNodesAdded = false;
								UEdGraphPin* NewThenPin = MarkPropertyDirtyNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
								UEdGraphPin* NewInPin = MarkPropertyDirtyNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute);

								if (ensure(NewThenPin) && ensure(NewInPin))
								{
									if (OldThenPin)
									{
										NewThenPin->CopyPersistentDataFromOldPin(*OldThenPin);
										OldThenPin->BreakAllPinLinks();
										OldThenPin->MakeLinkTo(NewInPin);

										OldThenPin = NewThenPin;
										// bWereNodesAdded = true;
									}
									else
									{
										// If there's no then pin, we'll instead insert the dirty nodes before the execution
										// of function with the reference.
										// This may do weird things with Latent Nodes, so warn about that.

										if (bIsLatent)
										{
											CompilerContext.MessageLog.Warning(*NSLOCTEXT("KismetCompiler", "LatentPushModel_Warning", "@@ is a latent node with references to a net property. The property may not be marked dirty in the correct frame.").ToString(), Node);
										}

										UEdGraphPin* OldInPin = CallFuncNode->FindPin(UEdGraphSchema_K2::PN_Execute);
										if (OldInPin)
										{
											NewInPin->CopyPersistentDataFromOldPin(*OldInPin);
											OldInPin->BreakAllPinLinks();

											NewThenPin->MakeLinkTo(OldInPin);
											OldThenPin = NewThenPin;
											// bWereNodesAdded = true;
										}
									}
								}

								/*
								if (!bWereNodesAdded)
								{
									// TODO: JDN - Reenable this once we have other edge cases worked out.
									// This warning is confusing / not necessarily actionable, and could contribute to spam,
									// but no one currently relies on these features.
									// CompilerContext.MessageLog.Warning(*NSLOCTEXT("KismetCompiler", "PushModelNoDirty_Warning", "@@ has reference to net properties, but we were unable to generate dirty nodes.").ToString(), Node);
								}
								*/
							}
						}
					}

					if (RemainingPins.Num() == 0)
					{
						break;
					}
				}
			}
		}
	}
}

void FKCHandler_CallFunction::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	check(NULL != Node);

	//@TODO: Can probably move this earlier during graph verification instead of compilation, but after island pruning
	if (!UE::K2NodeCallFunction::Private::IsCalledFunctionPure(Node))
	{
		// For imperative nodes, make sure the exec function was actually triggered and not just included due to an output data dependency
		UEdGraphPin* ExecTriggeringPin = CompilerContext.GetSchema()->FindExecutionPin(*Node, EGPD_Input);
		if (ExecTriggeringPin == NULL)
		{
			CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "NoValidExecutionPinForCallFunc_Error", "@@ must have a valid execution pin").ToString(), Node);
			return;
		}
		else if (ExecTriggeringPin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Warning(*NSLOCTEXT("KismetCompiler", "NodeNeverExecuted_Warning", "@@ will never be executed").ToString(), Node);
			return;
		}
	}

	// Validate the self pin again if it is disconnected, because pruning isolated nodes could have caused an invalid target
	UEdGraphPin* SelfPin = CompilerContext.GetSchema()->FindSelfPin(*Node, EGPD_Input);
	if (SelfPin && (SelfPin->LinkedTo.Num() == 0))
	{
		FEdGraphPinType SelfType;
		SelfType.PinCategory = UEdGraphSchema_K2::PC_Object;
		SelfType.PinSubCategory = UEdGraphSchema_K2::PSC_Self;

		if (!CompilerContext.GetSchema()->ArePinTypesCompatible(SelfType, SelfPin->PinType, Context.NewClass) && (SelfPin->DefaultObject == NULL))
		{
			CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "PinMustHaveConnectionPruned_Error", "Pin @@ must have a connection.  Self pins cannot be connected to nodes that are culled.").ToString(), SelfPin);
		}
	}

	// Make sure the function node is valid to call
	CreateFunctionCallStatement(Context, Node, SelfPin);
}

void FKCHandler_CallFunction::CheckIfFunctionIsCallable(UFunction* Function, FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	// Verify that the function is a Blueprint callable function (in case a BlueprintCallable specifier got removed)
	if (!Function->HasAnyFunctionFlags(FUNC_BlueprintCallable) && (Function->GetOuter() != Context.NewClass))
	{
		const bool bIsParentFunction = Node && Node->IsA<UK2Node_CallParentFunction>();
		if (!bIsParentFunction && Function->GetName().Find(UEdGraphSchema_K2::FN_ExecuteUbergraphBase.ToString()))
		{
			CompilerContext.MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "ShouldNotCallFromBlueprint_ErrorFmt", "Function '{0}' called from @@ should not be called from a Blueprint"), FText::FromString(Function->GetName())).ToString(), Node);
		}
	}
}

// Get the name of the function to call from the node
FName FKCHandler_CallFunction::GetFunctionNameFromNode(UEdGraphNode* Node) const
{
	UK2Node_CallFunction* CallFuncNode = Cast<UK2Node_CallFunction>(Node);
	if (CallFuncNode)
	{
		return CallFuncNode->FunctionReference.GetMemberName();
	}
	else
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "UnableResolveFunctionName_Error", "Unable to resolve function name for @@").ToString(), Node);
		return NAME_None;
	}
}

#ifdef _MSC_VER
#pragma warning(pop)
#endif

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
