// Copyright Epic Games, Inc. All Rights Reserved.

#include "DelegateNodeHandlers.h"

#include "BPTerminal.h"
#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallDelegate.h"
#include "K2Node_ClearDelegate.h"
#include "K2Node_CreateDelegate.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Templates/Casts.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

class UK2Node;

#define LOCTEXT_NAMESPACE "DelegateNodeHandlers"

struct FKCHandlerDelegateHelper
{
	static void CheckOutputsParametersInDelegateSignature(const UFunction* SignatureFunc, const UK2Node * DelegateNode, FCompilerResultsLog& MessageLog)
	{
		FKismetCompilerUtilities::DetectValuesReturnedByRef(SignatureFunc, DelegateNode, MessageLog);
	}

	static FMulticastDelegateProperty* FindAndCheckDelegateProperty(FKismetFunctionContext& Context, UK2Node_BaseMCDelegate * DelegateNode, FCompilerResultsLog& MessageLog, const UEdGraphSchema_K2* Schema)
	{
		check(NULL != DelegateNode);

		UEdGraphPin* Pin = Schema->FindSelfPin(*DelegateNode, EEdGraphPinDirection::EGPD_Input);
		UStruct* DelegateScope = Pin ? Context.GetScopeFromPinType(Pin->PinType, Context.NewClass) : NULL;

		// Early out if we have no pin.  That means the delegate is no longer valid, and we need to terminate gracefully
		if (!Pin || !DelegateScope)
		{
			MessageLog.Error(*LOCTEXT("NoDelegateProperty", "Event Dispatcher has no property @@").ToString(), DelegateNode);
			return NULL;
		}

		// Don't use DelegateNode->GetProperty(), because we don't want any property from skeletal class
		UClass* PropertyOwnerClass = CastChecked<UClass>(DelegateScope);
		FMulticastDelegateProperty* BoundProperty = NULL;
		for (TFieldIterator<FMulticastDelegateProperty> It(PropertyOwnerClass); It; ++It)
		{
			FMulticastDelegateProperty* Prop = *It;
			if (DelegateNode->GetPropertyName() == Prop->GetFName())
			{
				BoundProperty = Prop;
				break;
			}
		}

		if (!BoundProperty)
		{
			if (!FKismetCompilerUtilities::IsMissingMemberPotentiallyLoading(Context.Blueprint, DelegateNode->DelegateReference.GetMemberParentClass()))
			{
				FString const OwnerName = PropertyOwnerClass->GetName();
				FString const PropName = DelegateNode->GetPropertyName().ToString();

				MessageLog.Error(*FText::Format(LOCTEXT("DelegateNotFoundFmt", "Could not find an event-dispatcher named \"{0}\" in '{1}'.\nMake sure '{2}' has been compiled for @@"), FText::FromString(PropName), FText::FromString(OwnerName), FText::FromString(OwnerName)).ToString(), DelegateNode);
			}
			return NULL;
		}

		{
			// MulticastDelegateProperty from NewClass may have empty signature, but property from skeletal class should have it.
			const UFunction* OrgSignature = DelegateNode->GetDelegateSignature();
			if(const UEdGraphPin* DelegatePin = DelegateNode->GetDelegatePin())
			{
				const UFunction* PinSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(DelegatePin->PinType.PinSubCategoryMemberReference);
				if (!OrgSignature || !PinSignature || !OrgSignature->IsSignatureCompatibleWith(PinSignature))
				{
					MessageLog.Error(*LOCTEXT("WrongDelegate", "Wrong Event Dispatcher. Refresh node @@").ToString(), DelegateNode);
					return NULL;
				}
			}

			CheckOutputsParametersInDelegateSignature(OrgSignature, DelegateNode, MessageLog);
		}

		return BoundProperty;
	}

	static FBPTerminal* CreateInnerTerm(FKismetFunctionContext& Context, UEdGraphPin* SelfPin, UEdGraphPin* NetPin, FMulticastDelegateProperty* BoundProperty, UK2Node_BaseMCDelegate * DelegateNode, FCompilerResultsLog& MessageLog)
	{
		check(SelfPin && NetPin && BoundProperty && DelegateNode);

		FBPTerminal* Term = new FBPTerminal();
		Context.VariableReferences.Add(Term);
		Term->CopyFromPin(SelfPin, BoundProperty->GetName());
		Term->AssociatedVarProperty = BoundProperty;

		FBPTerminal** pContextTerm = Context.NetMap.Find(NetPin);
		if ((NULL == pContextTerm) && (SelfPin == NetPin))
		{
			Context.NetMap.Add(SelfPin, Term);
			pContextTerm = Context.NetMap.Find(NetPin);
		}

		if (pContextTerm)
		{
			if (Term != *pContextTerm)
			{
				Term->Context = *pContextTerm;
			}
		}
		else
		{
			MessageLog.Error(*FString(*LOCTEXT("FindDynamicallyBoundDelegate_Error", "Couldn't find target for dynamically bound delegate node @@").ToString()), DelegateNode);
		}
		return Term;
	}

	static void RegisterMultipleSelfAndMCDelegateProperty(FKismetFunctionContext& Context, UK2Node_BaseMCDelegate * DelegateNode, FCompilerResultsLog& MessageLog, 
		const UEdGraphSchema_K2* Schema, FDelegateOwnerId::FInnerTermMap& InnerTermMap)
	{
		if (DelegateNode && Schema)
		{
			FMulticastDelegateProperty* BoundProperty = FindAndCheckDelegateProperty(Context, DelegateNode, MessageLog, Schema);
			if (BoundProperty)
			{
				UEdGraphPin* SelfPin = Schema->FindSelfPin(*DelegateNode, EEdGraphPinDirection::EGPD_Input);
				check(SelfPin);

				if (0 == SelfPin->LinkedTo.Num())
				{
					FBPTerminal* Term = CreateInnerTerm(Context, SelfPin, FEdGraphUtilities::GetNetFromPin(SelfPin), BoundProperty, DelegateNode, MessageLog);
					Context.NetMap.Add(SelfPin, Term);
					InnerTermMap.Add(FDelegateOwnerId(SelfPin, DelegateNode), Term);
					return;
				}

				for (int32 LinkIndex = 0; LinkIndex < SelfPin->LinkedTo.Num(); ++LinkIndex)
				{
					UEdGraphPin* NetPin = SelfPin->LinkedTo[LinkIndex];
					FBPTerminal* Term = CreateInnerTerm(Context, SelfPin, NetPin, BoundProperty, DelegateNode, MessageLog);
					InnerTermMap.Add(FDelegateOwnerId(NetPin, DelegateNode), Term);
				}
			}
		}
	}

};


void FKCHandler_AddRemoveDelegate::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_BaseMCDelegate * DelegateNode = CastChecked<UK2Node_BaseMCDelegate>(Node);

	FKCHandlerDelegateHelper::RegisterMultipleSelfAndMCDelegateProperty(Context, DelegateNode, CompilerContext.MessageLog, CompilerContext.GetSchema(), InnerTermMap);

	UEdGraphPin* Pin = DelegateNode->GetDelegatePin();	
	check(NULL != Pin);
	if(0 == Pin->LinkedTo.Num())
	{
		CompilerContext.MessageLog.Error(*FString(*LOCTEXT("AddRemoveDelegate_NoDelegateInput", "Event Dispatcher pin is not connected @@").ToString()), DelegateNode);
	}
	UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(Pin);
	FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
	FBPTerminal* Term = FoundTerm ? *FoundTerm : NULL;
	if(NULL == Term)
	{
		Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
		Context.NetMap.Add(Net, Term);
	}
}

void FKCHandler_AddRemoveDelegate::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_BaseMCDelegate* DelegateNode = CastChecked<UK2Node_BaseMCDelegate>(Node);

	UEdGraphPin* DelegatePin = DelegateNode->GetDelegatePin();
	check(NULL != DelegatePin);
	FBPTerminal** DelegateInputTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(DelegatePin));
	check(DelegateInputTerm && *DelegateInputTerm);

	UEdGraphPin* SelfPin = CompilerContext.GetSchema()->FindSelfPin(*DelegateNode, EEdGraphPinDirection::EGPD_Input);
	check(SelfPin);

	TArray<UEdGraphPin*> Links = SelfPin->LinkedTo;
	if(!Links.Num())
	{
		Links.Add(SelfPin);
	}

	for (auto NetIt = Links.CreateIterator(); NetIt; ++NetIt)
	{
		UEdGraphPin* NetPin = *NetIt;
		check(NetPin);

		FBlueprintCompiledStatement& AddStatement = Context.AppendStatementForNode(DelegateNode);
		AddStatement.Type = Command;

		FBPTerminal** VarDelegate = InnerTermMap.Find(FDelegateOwnerId(NetPin, DelegateNode));
		check(VarDelegate && *VarDelegate);
		AddStatement.LHS = *VarDelegate;
		AddStatement.RHS.Add(*DelegateInputTerm);
	}

	GenerateSimpleThenGoto(Context, *DelegateNode, DelegateNode->FindPin(UEdGraphSchema_K2::PN_Then));
	FNodeHandlingFunctor::Compile(Context, DelegateNode);
}

void FKCHandler_CreateDelegate::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_CreateDelegate * DelegateNode = CastChecked<UK2Node_CreateDelegate>(Node);
	check(NULL != DelegateNode);

	const FName DelegateFunctionName = DelegateNode->GetFunctionName();
	if(DelegateFunctionName == NAME_None)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("NoDelegateFunctionName", "@@ : missing a function/event name.").ToString(), DelegateNode);
		return;
	}

	if(!DelegateNode->GetDelegateSignature())
	{
		const FString ErrorStr = LOCTEXT("NoDelegateFunction", "@@ : unable to determine expected signature - is the delegate pin connected?").ToString();
		CompilerContext.MessageLog.Error(*ErrorStr, DelegateNode);
		return;
	}

	{
		UEdGraphPin* InputPin = DelegateNode->GetObjectInPin();
		check(NULL != InputPin);

		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(InputPin);

		FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
		FBPTerminal* InputObjTerm = FoundTerm ? *FoundTerm : NULL;
		if(NULL == InputObjTerm)
		{
			if (InputPin->LinkedTo.Num() == 0)
			{
				InputObjTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
				InputObjTerm->Name = Context.NetNameMap->MakeValidName(Net);
				InputObjTerm->Type.PinSubCategory = UEdGraphSchema_K2::PN_Self;
			}
			else
			{
				InputObjTerm = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
			}

			Context.NetMap.Add(Net, InputObjTerm);
		}
	}

	{
		UEdGraphPin* OutPin = DelegateNode->GetDelegateOutPin();
		check(NULL != OutPin);
		if(!OutPin->LinkedTo.Num())
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoDelegateSignature", "No delegate signature @@").ToString(), DelegateNode);
			return;
		}
		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(OutPin);

		FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
		FBPTerminal* OutDelegateTerm = FoundTerm ? *FoundTerm : NULL;
		if(NULL == OutDelegateTerm)
		{
			OutDelegateTerm = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
			if (NULL == FMemberReference::ResolveSimpleMemberReference<UFunction>(OutDelegateTerm->Type.PinSubCategoryMemberReference))
			{
				FMemberReference::FillSimpleMemberReference<UFunction>(DelegateNode->GetDelegateSignature(), OutDelegateTerm->Type.PinSubCategoryMemberReference);
			}
			if (NULL == FMemberReference::ResolveSimpleMemberReference<UFunction>(OutDelegateTerm->Type.PinSubCategoryMemberReference))
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("UnconnectedDelegateSig", "Event Dispatcher has no signature @@").ToString(), OutPin);
				return;
			}
			Context.NetMap.Add(Net, OutDelegateTerm);
		}
	}
}

void FKCHandler_CreateDelegate::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_CreateDelegate * DelegateNode = CastChecked<UK2Node_CreateDelegate>(Node);
	check(NULL != DelegateNode);

	FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
	Statement.Type = KCST_BindDelegate;

	{
		UEdGraphPin* OutPin = DelegateNode->GetDelegateOutPin();
		check(NULL != OutPin);
		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(OutPin);
		check(NULL != Net);
		FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
		check(FoundTerm && *FoundTerm);
		Statement.LHS = *FoundTerm;
	}

	{
		FBPTerminal* DelegateNameTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		DelegateNameTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Name;
		DelegateNameTerm->Name = DelegateNode->GetFunctionName().ToString();
		DelegateNameTerm->bIsLiteral = true;
		Statement.RHS.Add(DelegateNameTerm);
	}

	{
		UEdGraphPin* InputPin = DelegateNode->GetObjectInPin();
		check(NULL != InputPin);
		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(InputPin);
		FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
		check(FoundTerm && *FoundTerm);
		Statement.RHS.Add(*FoundTerm);
	}

	FNodeHandlingFunctor::Compile(Context, Node);
}

void FKCHandler_ClearDelegate::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_ClearDelegate * DelegateNode = CastChecked<UK2Node_ClearDelegate>(Node);
	FKCHandlerDelegateHelper::RegisterMultipleSelfAndMCDelegateProperty(Context, DelegateNode, CompilerContext.MessageLog, CompilerContext.GetSchema(), InnerTermMap);
}

void FKCHandler_ClearDelegate::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_BaseMCDelegate* DelegateNode = CastChecked<UK2Node_BaseMCDelegate>(Node);

	UEdGraphPin* SelfPin = CompilerContext.GetSchema()->FindSelfPin(*DelegateNode, EEdGraphPinDirection::EGPD_Input);
	check(SelfPin);

	TArray<UEdGraphPin*> Links = SelfPin->LinkedTo;
	if(!Links.Num())
	{
		Links.Add(SelfPin);
	}

	for (auto NetIt = Links.CreateIterator(); NetIt; ++NetIt)
	{
		UEdGraphPin* NetPin = *NetIt;
		check(NetPin);

		FBlueprintCompiledStatement& AddStatement = Context.AppendStatementForNode(DelegateNode);
		AddStatement.Type = KCST_ClearMulticastDelegate;

		FBPTerminal** VarDelegate = InnerTermMap.Find(FDelegateOwnerId(NetPin, DelegateNode));
		check(VarDelegate && *VarDelegate);
		AddStatement.LHS = *VarDelegate;
	}

	GenerateSimpleThenGoto(Context, *DelegateNode, DelegateNode->FindPin(UEdGraphSchema_K2::PN_Then));
	FNodeHandlingFunctor::Compile(Context, DelegateNode);
}

//////////////////////////////////////////////////////////////////////////
// FKCHandler_CallDelegate

void FKCHandler_CallDelegate::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_CallDelegate * DelegateNode = CastChecked<UK2Node_CallDelegate>(Node);
	FKCHandlerDelegateHelper::RegisterMultipleSelfAndMCDelegateProperty(Context, DelegateNode, CompilerContext.MessageLog, CompilerContext.GetSchema(), InnerTermMap);

	FKCHandler_CallFunction::RegisterNets(Context, Node);
}	

void FKCHandler_CallDelegate::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	check(Node);
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>(); 
	const UFunction* SignatureFunction = FindFunction(Context, Node);
	if(!SignatureFunction)
	{
		UK2Node_CallDelegate* DelegateNode = CastChecked<UK2Node_CallDelegate>(Node);
		if (!FKismetCompilerUtilities::IsMissingMemberPotentiallyLoading(Context.Blueprint, DelegateNode->DelegateReference.GetMemberParentClass()))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("CallDelegateNoSignature_Error", "Cannot find signature function for @@").ToString(), Node);
		}
		return;
	}

	if(SignatureFunction->HasMetaData(FBlueprintMetadata::MD_DefaultToSelf))
	{
		CompilerContext.MessageLog.Error(
			*FText::Format(
				LOCTEXT("CallDelegateWrongMeta_ErrorFmt", "Signature function should not have {0} metadata. @@"),
				FText::FromString(FBlueprintMetadata::MD_DefaultToSelf.ToString())
			).ToString(),
			Node
		);
		return;
	}

	if(SignatureFunction->HasMetaData(FBlueprintMetadata::MD_WorldContext))
	{
		CompilerContext.MessageLog.Error(
			*FText::Format(
				LOCTEXT("CallDelegateWrongMeta_ErrorFmt", "Signature function should not have {0} metadata. @@"),
				FText::FromString(FBlueprintMetadata::MD_WorldContext.ToString())
			).ToString(),
			Node
		);
		return;
	}

	if(SignatureFunction->HasMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm))
	{
		CompilerContext.MessageLog.Error(
			*FText::Format(
				LOCTEXT("CallDelegateWrongMeta_ErrorFmt", "Signature function should not have {0} metadata. @@"),
				FText::FromString(FBlueprintMetadata::MD_AutoCreateRefTerm.ToString())
			).ToString(),
			Node
		);
		return;
	}

	FKCHandler_CallFunction::Compile(Context, Node);
}

UFunction* FKCHandler_CallDelegate::FindFunction(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	const UK2Node_CallDelegate* DelegateNode = CastChecked<UK2Node_CallDelegate>(Node);
	const UClass* TestClass = Context.NewClass;
	check(TestClass);
	const bool bIsSkeletonClass = TestClass->HasAnyFlags(RF_Transient) && TestClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
	return DelegateNode->GetDelegateSignature(!bIsSkeletonClass);
}

void FKCHandler_CallDelegate::AdditionalCompiledStatementHandling(FKismetFunctionContext& Context, UEdGraphNode* Node, FBlueprintCompiledStatement& Statement)
{
	if(NULL == Statement.FunctionContext)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("CallDelegateNoContext_Error", "Call delegate has no context. @@").ToString(), Node);
		return;
	}

	check(NULL != Statement.FunctionToCall);
	if (UClass* FunctionOwner = Statement.FunctionToCall->GetOwnerClass())
	{
		if (FunctionOwner != FunctionOwner->GetAuthoritativeClass())
		{
			CompilerContext.MessageLog.Warning(*LOCTEXT("CallDelegateWrongOwner", "Signature on delegate doesn't belong to authoritative class. @@").ToString(), Node);
		}
	}

	const UK2Node_BaseMCDelegate* DelegateNode = CastChecked<UK2Node_BaseMCDelegate>(Node);

	// Statement.FunctionContext is the term of DelegateOwner. It could be related with many keys (Pins) in Context.NetMap. 
	// We want to find the exact pin, that is connected with the DelegateNode
	FBPTerminal** VarDelegate = NULL;
	for (auto Pair : Context.NetMap)
	{
		if (Pair.Value == Statement.FunctionContext)
		{
			check(Pair.Key);
			VarDelegate = InnerTermMap.Find(FDelegateOwnerId(Pair.Key, DelegateNode));
			if (VarDelegate && *VarDelegate)
			{
				break;
			}
		}
	}
	check(VarDelegate && *VarDelegate);

	Statement.FunctionContext = *VarDelegate;
	Statement.Type = KCST_CallDelegate;
}

#undef LOCTEXT_NAMESPACE
