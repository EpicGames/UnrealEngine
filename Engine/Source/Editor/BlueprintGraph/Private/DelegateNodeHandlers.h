// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintCompiledStatement.h"
#include "CallFunctionHandler.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Crc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UFunction;
struct FBPTerminal;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_AddDelegate

struct FDelegateOwnerId
{
	typedef TMap<FDelegateOwnerId, FBPTerminal*> FInnerTermMap;

	const class UEdGraphPin* OutputPin;
	const class UK2Node_BaseMCDelegate* DelegateNode;

	FDelegateOwnerId(const class UEdGraphPin* InOutputPin, const class UK2Node_BaseMCDelegate* InDelegateNode) 
		: OutputPin(InOutputPin), DelegateNode(InDelegateNode)
	{
		ensure(OutputPin && DelegateNode);
	}

	bool operator==(const FDelegateOwnerId& Other) const
	{
		return (Other.OutputPin == OutputPin) && (Other.DelegateNode == DelegateNode);
	}

	friend uint32 GetTypeHash(const FDelegateOwnerId& DelegateOwnerId)
	{
		return FCrc::MemCrc_DEPRECATED(&DelegateOwnerId,sizeof(FDelegateOwnerId));
	}
};



class FKCHandler_AddRemoveDelegate : public FNodeHandlingFunctor
{
	EKismetCompiledStatementType Command;
	FDelegateOwnerId::FInnerTermMap InnerTermMap;

public:
	FKCHandler_AddRemoveDelegate(FKismetCompilerContext& InCompilerContext, EKismetCompiledStatementType InCommand)
		: FNodeHandlingFunctor(InCompilerContext)
		, Command(InCommand)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
};

//////////////////////////////////////////////////////////////////////////
// FKCHandler_CreateDelegate

class FKCHandler_CreateDelegate : public FNodeHandlingFunctor
{
public:
	FKCHandler_CreateDelegate(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
};

//////////////////////////////////////////////////////////////////////////
// FKCHandler_ClearDelegate

class FKCHandler_ClearDelegate : public FNodeHandlingFunctor
{
	FDelegateOwnerId::FInnerTermMap InnerTermMap;

public:
	FKCHandler_ClearDelegate(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
};

//////////////////////////////////////////////////////////////////////////
// FKCHandler_CallDelegate

class FKCHandler_CallDelegate : public FKCHandler_CallFunction
{
	FDelegateOwnerId::FInnerTermMap InnerTermMap;
public:

	FKCHandler_CallDelegate(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_CallFunction(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual UFunction* FindFunction(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void CheckIfFunctionIsCallable(UFunction* Function, FKismetFunctionContext& Context, UEdGraphNode* Node) override { }
	virtual void AdditionalCompiledStatementHandling(FKismetFunctionContext& Context, UEdGraphNode* Node, FBlueprintCompiledStatement& Statement) override;
};
