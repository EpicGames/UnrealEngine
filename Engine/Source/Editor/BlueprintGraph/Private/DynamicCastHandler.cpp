// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicCastHandler.h"

#include "BPTerminal.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_DynamicCast.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "DynamicCastHandler"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_DynamicCast

void FKCHandler_DynamicCast::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	FNodeHandlingFunctor::RegisterNets(Context, Node);

	if (const UK2Node_DynamicCast* DynamicCastNode = Cast<UK2Node_DynamicCast>(Node))
	{
		UEdGraphPin* BoolSuccessPin = DynamicCastNode->GetBoolSuccessPin();
		// this is to support backwards compatibility (when a cast node is generating code, but has yet to be reconstructed)
		// @TODO: remove this at some point, when backwards compatibility isn't a concern
		if (BoolSuccessPin == nullptr)
		{
			// Create a term to determine if the cast was successful or not
			FBPTerminal* BoolTerm = Context.CreateLocalTerminal();
			BoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
			BoolTerm->Source = Node;
			BoolTerm->Name = Context.NetNameMap->MakeValidName(Node, TEXT("CastSuccess"));
			BoolTermMap.Add(Node, BoolTerm);
		}
	}
	
}

void FKCHandler_DynamicCast::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
	Context.NetMap.Add(Net, Term);
}

void FKCHandler_DynamicCast::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	const UK2Node_DynamicCast* DynamicCastNode = CastChecked<UK2Node_DynamicCast>(Node);

	if (DynamicCastNode->TargetType == NULL)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("BadCastNoTargetType_Error", "Node @@ has an invalid target type, please delete and recreate it").ToString(), Node);
	}

	// Self Pin
	UEdGraphPin* SourceObjectPin = DynamicCastNode->GetCastSourcePin();
	UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(SourceObjectPin);
	FBPTerminal** ObjectToCast = Context.NetMap.Find(PinToTry);

	if (!ObjectToCast)
	{
		ObjectToCast = Context.LiteralHackMap.Find(PinToTry);

		if (!ObjectToCast || !(*ObjectToCast))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("InvalidConnectionOnNode_Error", "Node @@ has an invalid connection on @@").ToString(), Node, SourceObjectPin);
			return;
		}
	}

	// Output pin
	const UEdGraphPin* CastOutputPin = DynamicCastNode->GetCastResultPin();
	if( !CastOutputPin )
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InvalidDynamicCastClass_Error", "Node @@ has an invalid target class").ToString(), Node);
		return;
	}

	FBPTerminal** CastResultTerm = Context.NetMap.Find(CastOutputPin);
	if (!CastResultTerm || !(*CastResultTerm))
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InvalidDynamicCastClass_CompilerError", "Node @@ has an invalid target class. (Inner compiler error?)").ToString(), Node);
		return;
	}

	// Create a literal term from the class specified in the node
	FBPTerminal* ClassTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
	ClassTerm->Name = DynamicCastNode->TargetType->GetName();
	ClassTerm->bIsLiteral = true;
	ClassTerm->Source = Node;
	ClassTerm->ObjectLiteral = DynamicCastNode->TargetType;
	ClassTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Class;

	UClass const* const InputObjClass  = Cast<UClass>((*ObjectToCast)->Type.PinSubCategoryObject.Get());
	UClass const* const OutputObjClass = Cast<UClass>((*CastResultTerm)->Type.PinSubCategoryObject.Get());

	const bool bIsOutputInterface = ((OutputObjClass != NULL) && OutputObjClass->HasAnyClassFlags(CLASS_Interface));
	const bool bIsInputInterface  = ((InputObjClass != NULL) && InputObjClass->HasAnyClassFlags(CLASS_Interface));

	EKismetCompiledStatementType CastOpType = KCST_DynamicCast;
	if (bIsInputInterface)
	{
		if (bIsOutputInterface)
		{
			CastOpType = KCST_CrossInterfaceCast;
		}
		else
		{
			CastOpType = KCST_CastInterfaceToObj;
		}
	}
	else if (bIsOutputInterface)
	{
		CastOpType = KCST_CastObjToInterface;
	}

	if (KCST_MetaCast == CastType)
	{
		if (bIsInputInterface || bIsOutputInterface)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("InvalidClassDynamicCastClass_Error", "Node @@ has an invalid target class. Interfaces are not supported.").ToString(), Node);
			return;
		}
		CastOpType = KCST_MetaCast;
	}

	// Cast Statement
	FBlueprintCompiledStatement& CastStatement = Context.AppendStatementForNode(Node);
	CastStatement.Type = CastOpType;
	CastStatement.LHS = *CastResultTerm;
	CastStatement.RHS.Add(ClassTerm);
	CastStatement.RHS.Add(*ObjectToCast);

	FBPTerminal** BoolSuccessTerm = nullptr;
	if (UEdGraphPin* BoolSuccessPin = DynamicCastNode->GetBoolSuccessPin())
	{
		BoolSuccessTerm = Context.NetMap.Find(BoolSuccessPin);
	}
	else
	{
		BoolSuccessTerm = BoolTermMap.Find(DynamicCastNode);
	}	
	check(BoolSuccessTerm != nullptr);

	// Check result of cast statement
	FBlueprintCompiledStatement& CheckResultStatement = Context.AppendStatementForNode(Node);
	CheckResultStatement.Type = KCST_ObjectToBool;
	CheckResultStatement.LHS  = *BoolSuccessTerm;
	CheckResultStatement.RHS.Add(*CastResultTerm);

	UEdGraphPin* SuccessExecPin = DynamicCastNode->GetValidCastPin();
	bool const bIsPureCast = (SuccessExecPin == nullptr);
	if (!bIsPureCast)
	{
		UEdGraphPin* FailurePin = DynamicCastNode->GetInvalidCastPin();
		check(FailurePin != nullptr);
		// Failure condition...skip to the failed output
		FBlueprintCompiledStatement& FailCastGoto = Context.AppendStatementForNode(Node);
		FailCastGoto.Type = KCST_GotoIfNot;
		FailCastGoto.LHS  = *BoolSuccessTerm;
		Context.GotoFixupRequestMap.Add(&FailCastGoto, FailurePin);

		// Successful cast...hit the success output node
		FBlueprintCompiledStatement& SuccessCastGoto = Context.AppendStatementForNode(Node);
		SuccessCastGoto.Type = KCST_UnconditionalGoto;
		SuccessCastGoto.LHS  = *BoolSuccessTerm;
		Context.GotoFixupRequestMap.Add(&SuccessCastGoto, SuccessExecPin);
	}
}

#undef LOCTEXT_NAMESPACE
