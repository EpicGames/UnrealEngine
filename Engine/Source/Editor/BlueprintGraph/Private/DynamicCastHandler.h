// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintCompiledStatement.h"
#include "Containers/Map.h"
#include "KismetCompilerMisc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UEdGraphPin;
struct FBPTerminal;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_DynamicCast

class FKCHandler_DynamicCast : public FNodeHandlingFunctor
{
protected:
	TMap<UEdGraphNode*, FBPTerminal*> BoolTermMap; // should be deprecated at a certain point (left in for backwards compatibility)
	EKismetCompiledStatementType CastType;

public:
	FKCHandler_DynamicCast(FKismetCompilerContext& InCompilerContext, EKismetCompiledStatementType InCastType)
		: FNodeHandlingFunctor(InCompilerContext)
		, CastType(InCastType)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
};
