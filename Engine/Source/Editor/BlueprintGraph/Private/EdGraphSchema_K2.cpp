// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_K2.h"
#include "AssetBlueprintGraphActions.h"
#include "BlueprintCompilationManager.h"
#include "Kismet2/Breakpoint.h"
#include "Modules/ModuleManager.h"
#include "UObject/Interface.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/Blueprint.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/FieldPathProperty.h"
#include "Engine/MemberReference.h"
#include "Components/ActorComponent.h"
#include "Misc/Attribute.h"
#include "GameFramework/Actor.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/CollisionProfile.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Kismet/GameplayStatics.h"
#include "Engine/LevelScriptActor.h"
#include "Components/ChildActorComponent.h"
#include "Editor/EditorEngine.h"
#include "Engine/Font.h"
#include "Engine/Selection.h"
#include "Engine/UserDefinedEnum.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "GraphEditorSettings.h"
#include "K2Node.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Event.h"
#include "K2Node_ActorBoundEvent.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Variable.h"
#include "K2Node_BreakStruct.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CallParentFunction.h"
#include "K2Node_ComponentBoundEvent.h"
#include "K2Node_Tunnel.h"
#include "K2Node_Composite.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_FunctionTerminator.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_Knot.h"
#include "K2Node_Literal.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_MakeArray.h"
#include "K2Node_MakeStruct.h"
#include "K2Node_Select.h"
#include "K2Node_SpawnActor.h"
#include "K2Node_SpawnActorFromClass.h"
#include "K2Node_Switch.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "K2Node_SetFieldsInStruct.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor.h"
#include "Kismet/BlueprintMapLibrary.h"
#include "Kismet/BlueprintSetLibrary.h"
#include "Kismet/BlueprintTypeConversions.h"
#include "Kismet/KismetArrayLibrary.h"
#include "Kismet/KismetMathLibrary.h"
#include "GraphEditorActions.h"
#include "ScopedTransaction.h"
#include "ComponentAssetBroker.h"
#include "BlueprintEditorSettings.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/CompilerResultsLog.h"
#include "EdGraphUtilities.h"
#include "KismetCompiler.h"
#include "Misc/DefaultValueHelper.h"
#include "ObjectEditorUtils.h"
#include "ComponentTypeRegistry.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintComponentNodeSpawner.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "K2Node_CastByteToEnum.h"
#include "K2Node_ClassDynamicCast.h"
#include "K2Node_GetEnumeratorName.h"
#include "K2Node_GetEnumeratorNameAsString.h"
#include "K2Node_ConvertAsset.h"
#include "Framework/Commands/GenericCommands.h"

#include "BlueprintTypePromotion.h"
#include "K2Node_PromotableOperator.h"

#include "Editor/EditorPerProjectUserSettings.h"
#include "BlueprintPaletteFavorites.h"
#include "ObjectTools.h"

//////////////////////////////////////////////////////////////////////////

// How to display PC_Real pin types to users

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_K2)
enum class EBlueprintRealDisplayMode : uint8
{
	Real,
	Float,
	Number
};

namespace UE::EdGraphSchemaK2::Private
{
	template <class... T>
	constexpr bool TAlwaysFalse = false;

	template <typename TProperty>
	UClass* GetAuthoritativeClass(const TProperty& Property)
	{
		UClass* PropertyClass = nullptr;
		if constexpr (std::is_same_v<TProperty, FObjectPropertyBase>)
		{
			PropertyClass = Property.PropertyClass;
		}
		else if constexpr (std::is_same_v<TProperty, FSoftObjectProperty>)
		{
			PropertyClass = Property.PropertyClass;
		}
		else if constexpr (std::is_same_v<TProperty, FInterfaceProperty>)
		{
			PropertyClass = Property.InterfaceClass;
		}
		else if constexpr (std::is_same_v<TProperty, FClassProperty>)
		{
			PropertyClass = Property.MetaClass;
		}
		else if constexpr (std::is_same_v<TProperty, FSoftClassProperty>)
		{
			PropertyClass = Property.MetaClass;
		}
		else
		{
			static_assert(TAlwaysFalse<TProperty>, "Invalid property used.");
		}

		if (PropertyClass && PropertyClass->ClassGeneratedBy)
		{
			PropertyClass = PropertyClass->GetAuthoritativeClass();
		}

		if (PropertyClass && FKismetEditorUtilities::IsClassABlueprintSkeleton(PropertyClass))
		{
			UE_LOGF(LogBlueprint, Warning, "'%ls' is a skeleton class. SubCategoryObject will serialize to a null value.", *PropertyClass->GetFullName());
		}

		return PropertyClass;
	}
}

//////////////////////////////////////////////////////////////////////////
// FBlueprintMetadata

const FName FBlueprintMetadata::MD_AllowableBlueprintVariableType(TEXT("BlueprintType"));
const FName FBlueprintMetadata::MD_NotAllowableBlueprintVariableType(TEXT("NotBlueprintType"));

const FName FBlueprintMetadata::MD_BlueprintSpawnableComponent(TEXT("BlueprintSpawnableComponent"));
const FName FBlueprintMetadata::MD_IsBlueprintBase(TEXT("IsBlueprintBase"));
const FName FBlueprintMetadata::MD_RestrictedToClasses(TEXT("RestrictedToClasses"));
const FName FBlueprintMetadata::MD_ChildCanTick(TEXT("ChildCanTick"));
const FName FBlueprintMetadata::MD_ChildCannotTick(TEXT("ChildCannotTick"));
const FName FBlueprintMetadata::MD_IgnoreCategoryKeywordsInSubclasses(TEXT("IgnoreCategoryKeywordsInSubclasses"));


const FName FBlueprintMetadata::MD_Protected(TEXT("BlueprintProtected"));
const FName FBlueprintMetadata::MD_Latent(TEXT("Latent"));
const FName FBlueprintMetadata::MD_CustomThunkTemplates(TEXT("CustomThunkTemplates"));
const FName FBlueprintMetadata::MD_CustomThunk(TEXT("CustomThunk"));
const FName FBlueprintMetadata::MD_Variadic(TEXT("Variadic"));
const FName FBlueprintMetadata::MD_UnsafeForConstructionScripts(TEXT("UnsafeDuringActorConstruction"));
const FName FBlueprintMetadata::MD_FunctionCategory(TEXT("Category"));
const FName FBlueprintMetadata::MD_DeprecatedFunction(TEXT("DeprecatedFunction"));
const FName FBlueprintMetadata::MD_DeprecatedProperty(TEXT("DeprecatedProperty"));
const FName FBlueprintMetadata::MD_DeprecationMessage(TEXT("DeprecationMessage"));
const FName FBlueprintMetadata::MD_CompactNodeTitle(TEXT("CompactNodeTitle"));
const FName FBlueprintMetadata::MD_DisplayName(TEXT("DisplayName"));
const FName FBlueprintMetadata::MD_ReturnDisplayName(TEXT("ReturnDisplayName"));
const FName FBlueprintMetadata::MD_InternalUseParam(TEXT("InternalUseParam"));
const FName FBlueprintMetadata::MD_ForceAsFunction(TEXT("ForceAsFunction"));
const FName FBlueprintMetadata::MD_IgnoreTypePromotion(TEXT("IgnoreTypePromotion"));

const FName FBlueprintMetadata::MD_PropertyGetFunction(TEXT("BlueprintGetter"));
const FName FBlueprintMetadata::MD_PropertySetFunction(TEXT("BlueprintSetter"));

const FName FBlueprintMetadata::MD_ExposeOnSpawn(TEXT("ExposeOnSpawn"));
const FName FBlueprintMetadata::MD_HideSelfPin(TEXT("HideSelfPin"));
const FName FBlueprintMetadata::MD_HidePin(TEXT("HidePin"));
const FName FBlueprintMetadata::MD_DefaultToSelf(TEXT("DefaultToSelf"));
const FName FBlueprintMetadata::MD_WorldContext(TEXT("WorldContext"));
const FName FBlueprintMetadata::MD_CallableWithoutWorldContext(TEXT("CallableWithoutWorldContext"));
const FName FBlueprintMetadata::MD_DevelopmentOnly(TEXT("DevelopmentOnly"));
const FName FBlueprintMetadata::MD_AutoCreateRefTerm(TEXT("AutoCreateRefTerm"));
const FName FBlueprintMetadata::MD_HideAssetPicker(TEXT("HideAssetPicker"));
const FName FBlueprintMetadata::MD_HidePinAssetPicker(TEXT("HidePinAssetPicker"));

const FName FBlueprintMetadata::MD_ShowWorldContextPin(TEXT("ShowWorldContextPin"));
const FName FBlueprintMetadata::MD_Private(TEXT("BlueprintPrivate"));

const FName FBlueprintMetadata::MD_BlueprintInternalUseOnly(TEXT("BlueprintInternalUseOnly"));
const FName FBlueprintMetadata::MD_BlueprintInternalUseOnlyHierarchical(TEXT("BlueprintInternalUseOnlyHierarchical"));
const FName FBlueprintMetadata::MD_NeedsLatentFixup(TEXT("NeedsLatentFixup"));
const FName FBlueprintMetadata::MD_LatentInfo(TEXT("LatentInfo"));
const FName FBlueprintMetadata::MD_LatentCallbackTarget(TEXT("LatentCallbackTarget"));
const FName FBlueprintMetadata::MD_AllowPrivateAccess(TEXT("AllowPrivateAccess"));

const FName FBlueprintMetadata::MD_ExposeFunctionCategories(TEXT("ExposeFunctionCategories"));

const FName FBlueprintMetadata::MD_CannotImplementInterfaceInBlueprint(TEXT("CannotImplementInterfaceInBlueprint"));
const FName FBlueprintMetadata::MD_ProhibitedInterfaces(TEXT("ProhibitedInterfaces"));

const FName FBlueprintMetadata::MD_FunctionKeywords(TEXT("Keywords"));

const FName FBlueprintMetadata::MD_ExpandEnumAsExecs(TEXT("ExpandEnumAsExecs"));
const FName FBlueprintMetadata::MD_ExpandBoolAsExecs(TEXT("ExpandBoolAsExecs"));

const FName FBlueprintMetadata::MD_CommutativeAssociativeBinaryOperator(TEXT("CommutativeAssociativeBinaryOperator"));
const FName FBlueprintMetadata::MD_MaterialParameterCollectionFunction(TEXT("MaterialParameterCollectionFunction"));

const FName FBlueprintMetadata::MD_Tooltip(TEXT("Tooltip"));

const FName FBlueprintMetadata::MD_CallInEditor(TEXT("CallInEditor"));

const FName FBlueprintMetadata::MD_DataTablePin(TEXT("DataTablePin"));

const FName FBlueprintMetadata::MD_NativeMakeFunction(TEXT("HasNativeMake"));
const FName FBlueprintMetadata::MD_NativeBreakFunction(TEXT("HasNativeBreak"));
const FName FBlueprintMetadata::MD_NativeDisableSplitPin(TEXT("DisableSplitPin"));

const FName FBlueprintMetadata::MD_DynamicOutputType(TEXT("DeterminesOutputType"));
const FName FBlueprintMetadata::MD_DynamicOutputParam(TEXT("DynamicOutputParam"));

const FName FBlueprintMetadata::MD_CustomStructureParam(TEXT("CustomStructureParam"));

const FName FBlueprintMetadata::MD_ArrayParam(TEXT("ArrayParm"));
const FName FBlueprintMetadata::MD_ArrayDependentParam(TEXT("ArrayTypeDependentParams"));

const FName FBlueprintMetadata::MD_SetParam(TEXT("SetParam"));

// Each of these is a | separated list of param names:
const FName FBlueprintMetadata::MD_MapParam(TEXT("MapParam"));
const FName FBlueprintMetadata::MD_MapKeyParam(TEXT("MapKeyParam"));
const FName FBlueprintMetadata::MD_MapValueParam(TEXT("MapValueParam"));

const FName FBlueprintMetadata::MD_Bitmask(TEXT("Bitmask"));
const FName FBlueprintMetadata::MD_BitmaskEnum(TEXT("BitmaskEnum"));
const FName FBlueprintMetadata::MD_Bitflags(TEXT("Bitflags"));
const FName FBlueprintMetadata::MD_UseEnumValuesAsMaskValuesInEditor(TEXT("UseEnumValuesAsMaskValuesInEditor"));

const FName FBlueprintMetadata::MD_AnimBlueprintFunction(TEXT("AnimBlueprintFunction"));

const FName FBlueprintMetadata::MD_ShowDisplayNames(TEXT("ShowDisplayNames"));

const FName FBlueprintMetadata::MD_AllowAbstractClasses(TEXT("AllowAbstract"));
const FName FBlueprintMetadata::MD_AllowedClasses(TEXT("AllowedClasses"));
const FName FBlueprintMetadata::MD_GetOptions(TEXT("GetOptions"));

const FName FBlueprintMetadata::MD_Namespace(TEXT("Namespace"));

const FName FBlueprintMetadata::MD_ThreadSafe(TEXT("BlueprintThreadSafe"));
const FName FBlueprintMetadata::MD_NotThreadSafe(TEXT("NotBlueprintThreadSafe"));
const FName FBlueprintMetadata::MD_FieldNotify(TEXT("FieldNotify"));

//////////////////////////////////////////////////////////////////////////

#define LOCTEXT_NAMESPACE "KismetSchema"

/** Helpers for gathering pin type tree info for enums, structs, classes, and interfaces */
namespace GatherPinsImpl
{
	TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> FromAssetData(const FAssetData& InAsset, FName CategoryName, EObjectReferenceType ReferenceType);
	TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> FromObject(UField* Field, FName CategoryName, EObjectReferenceType ReferenceType);
	void SortPinTypes(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& PinArray);

	void FindEnums(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& Owner);
	void FindStructs(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& Owner);
	void FindObjectsAndInterfaces(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& ObjectsOwner, const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& InterfacesOwner);
}

TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> GatherPinsImpl::FromAssetData(const FAssetData& InAsset, FName CategoryName, EObjectReferenceType ReferenceType)
{
	return MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(
		FText::FromString(FName::NameToDisplayString(InAsset.AssetName.ToString(), false))
		, CategoryName
		, InAsset
		, FText::FromString(InAsset.GetObjectPathString())
		, false
		, (uint8)ReferenceType);
}

TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo> GatherPinsImpl::FromObject(UField* Field, FName CategoryName, EObjectReferenceType ReferenceType)
{
	return MakeShared<UEdGraphSchema_K2::FPinTypeTreeInfo>(
		CategoryName
		, Field
		, Field->GetToolTipText()
		, false
		, (uint8)ReferenceType);
}

void GatherPinsImpl::SortPinTypes(TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& PinArray)
{
	PinArray.Sort(
		[](const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& A, const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& B)
		{
			return A->GetCachedDescriptionString().Compare(B->GetCachedDescriptionString()) < 0;
		});
}

void GatherPinsImpl::FindEnums(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& Owner)
{
	TSet<FTopLevelAssetPath> ProcessedAssets;

	check(Owner->bReadOnly);
	// Generate a list of all potential enums which have "BlueprintType=true" in their metadata
	for (TObjectIterator<UEnum> EnumIt; EnumIt; ++EnumIt)
	{
		UEnum* CurrentEnum = *EnumIt;
		ProcessedAssets.Add(FTopLevelAssetPath(CurrentEnum));
		if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(CurrentEnum))
		{
			Owner->Children.Emplace(
				FromObject(CurrentEnum
					, UEdGraphSchema_K2::PC_Byte
					, EObjectReferenceType::NotAnObject));
		}
	}

	TArray<FAssetData> AssetData;
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByClass(UUserDefinedEnum::StaticClass()->GetClassPathName(), AssetData);

	for (const FAssetData& Asset : AssetData)
	{
		if (Asset.IsValid() && !ProcessedAssets.Contains(FTopLevelAssetPath(Asset.PackageName, Asset.AssetName)))
		{
			Owner->Children.Emplace(
				FromAssetData(Asset
					, UEdGraphSchema_K2::PC_Byte
					, EObjectReferenceType::NotAnObject));
		}
	}

	SortPinTypes(Owner->Children);
}

void GatherPinsImpl::FindStructs(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& Owner)
{
	check(Owner->bReadOnly);
	TSet<FTopLevelAssetPath> ProcessedAssets;

	// Find script structs marked with "BlueprintType=true" in their metadata, and add to the list
	for (TObjectIterator<UScriptStruct> StructIt; StructIt; ++StructIt)
	{
		UScriptStruct* ScriptStruct = *StructIt;
		ProcessedAssets.Add(FTopLevelAssetPath(ScriptStruct));
		if (UEdGraphSchema_K2::IsAllowableBlueprintVariableType(ScriptStruct))
		{
			Owner->Children.Emplace(
				FromObject(ScriptStruct
					, UEdGraphSchema_K2::PC_Struct
					, EObjectReferenceType::NotAnObject));
		}
	}

	TArray<FAssetData> AssetData;
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByClass(UUserDefinedStruct::StaticClass()->GetClassPathName(), AssetData);

	for (const FAssetData& Asset : AssetData)
	{
		if (Asset.IsValid() && !ProcessedAssets.Contains(FTopLevelAssetPath(Asset.PackageName, Asset.AssetName)))
		{
			Owner->Children.Emplace(
				FromAssetData(Asset
					, UEdGraphSchema_K2::PC_Struct
					, EObjectReferenceType::NotAnObject));
		}
	}

	SortPinTypes(Owner->Children);
}

void GatherPinsImpl::FindObjectsAndInterfaces(const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& ObjectsOwner, const TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& InterfacesOwner)
{
	check(ObjectsOwner->bReadOnly && InterfacesOwner->bReadOnly);
	TSet<FTopLevelAssetPath> ProcessedAssets;

	// Generate a list of all potential objects which have "BlueprintType=true" in their metadata
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* CurrentClass = *ClassIt;
		ProcessedAssets.Add(FTopLevelAssetPath(CurrentClass));
		ProcessedAssets.Add(FTopLevelAssetPath(CurrentClass->ClassGeneratedBy));
		const bool bIsInterface = CurrentClass->IsChildOf(UInterface::StaticClass());
		const bool bIsBlueprintType = UEdGraphSchema_K2::IsAllowableBlueprintVariableType(CurrentClass);
		const bool bIsDeprecated = CurrentClass->HasAnyClassFlags(CLASS_Deprecated);
		if (bIsBlueprintType && !bIsDeprecated)
		{
			if (bIsInterface)
			{
				InterfacesOwner->Children.Emplace(
					FromObject(CurrentClass
						, UEdGraphSchema_K2::PC_Interface
						, EObjectReferenceType::NotAnObject));
			}
			else
			{
				ObjectsOwner->Children.Emplace(
					FromObject(CurrentClass
						, UEdGraphSchema_K2::AllObjectTypes
						, EObjectReferenceType::AllTypes));
			}
		}
	}

	TArray<FAssetData> AssetData;
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), AssetData);

	const FStringView BPInterfaceTypeAllowed(TEXT("BPTYPE_Interface"));
	const FStringView BPNormalTypeAllowed(TEXT("BPTYPE_Normal"));

	for (const FAssetData& Asset : AssetData)
	{
		if (Asset.IsValid() && !ProcessedAssets.Contains(FTopLevelAssetPath(Asset.PackageName, Asset.AssetName)))
		{
			FAssetDataTagMapSharedView::FFindTagResult FoundValue = Asset.TagsAndValues.FindTag(FBlueprintTags::BlueprintType);
			if (!FoundValue.IsSet())
			{
				continue;
			}

			const bool bNormalBP = FoundValue.Equals(BPNormalTypeAllowed);
			const bool bInterfaceBP = FoundValue.Equals(BPInterfaceTypeAllowed);

			if (bNormalBP || bInterfaceBP)
			{
				const uint32 ClassFlags = Asset.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
				if (!(ClassFlags & CLASS_Deprecated))
				{
					if (bNormalBP)
					{
						ObjectsOwner->Children.Emplace(
							FromAssetData(Asset
								, UEdGraphSchema_K2::AllObjectTypes
								, EObjectReferenceType::AllTypes));
					}
					else if (bInterfaceBP)
					{
						InterfacesOwner->Children.Emplace(
							FromAssetData(Asset
								, UEdGraphSchema_K2::PC_Interface
								, EObjectReferenceType::NotAnObject));
					}
				}
			}
		}
	}

	SortPinTypes(InterfacesOwner->Children);
	SortPinTypes(ObjectsOwner->Children);
}

const FEdGraphPinType& UEdGraphSchema_K2::FPinTypeTreeInfo::GetPinType(bool bForceLoadedSubCategoryObject)
{
	// Only attempt to load the sub category object if we need to
	if (CachedAssetData.IsValid() && (!PinType.PinSubCategoryObject.IsValid() || FSoftObjectPath(PinType.PinSubCategoryObject.Get()) != CachedAssetData.GetSoftObjectPath()))
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		if (bForceLoadedSubCategoryObject || CachedAssetData.IsAssetLoaded())
		{
			UObject* LoadedObject = CachedAssetData.GetAsset();

			if (UBlueprint* BlueprintObject = Cast<UBlueprint>(LoadedObject))
			{
				PinType.PinSubCategoryObject = *BlueprintObject->GeneratedClass;
			}
			else
			{
				PinType.PinSubCategoryObject = LoadedObject;
			}
		}
	}

	return PinType;
}

UEdGraphSchema_K2::FPinTypeTreeInfo::FPinTypeTreeInfo(const FText& InFriendlyName, const FName CategoryName, const UEdGraphSchema_K2* Schema, const FText& InTooltip, bool bInReadOnly/*=false*/)
	: PossibleObjectReferenceTypes(0)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WILD_FPinTypeTreeInfo::Init);
	check( !CategoryName.IsNone() );
	check( Schema );
	check(!InFriendlyName.IsEmpty());

	Tooltip = InTooltip;
	PinType.PinCategory = (CategoryName == PC_Enum ? PC_Byte : CategoryName);
	PinType.PinSubCategory = (CategoryName == PC_Real ? PC_Double : NAME_None);
	PinType.PinSubCategoryObject = nullptr;

	bReadOnly = bInReadOnly;

	CachedDescription = InFriendlyName;
	CachedDescriptionString = MakeShared<FString>(CachedDescription.ToString());
}

UEdGraphSchema_K2::FPinTypeTreeInfo::FPinTypeTreeInfo(const FName CategoryName, UObject* SubCategoryObject, const FText& InTooltip, bool bInReadOnly/*=false*/, uint8 InPossibleObjectReferenceTypes)
	: PossibleObjectReferenceTypes(InPossibleObjectReferenceTypes)
{
	check( !CategoryName.IsNone() );
	check( SubCategoryObject );

	Tooltip = InTooltip;
	PinType.PinCategory = CategoryName;
	PinType.PinSubCategoryObject = SubCategoryObject;

	bReadOnly = bInReadOnly;
	CachedDescription = GenerateDescription();
	CachedDescriptionString = MakeShared<FString>(CachedDescription.ToString());
}

UEdGraphSchema_K2::FPinTypeTreeInfo::FPinTypeTreeInfo(const FText& InFriendlyName, const FName CategoryName, const FAssetData& AssetData, const FText& InTooltip, bool bInReadOnly, uint8 InPossibleObjectReferenceTypes)
	: PossibleObjectReferenceTypes(InPossibleObjectReferenceTypes)
{
	check(!CategoryName.IsNone());
	check(AssetData.IsValid());
	check(!InFriendlyName.IsEmpty());

	Tooltip = InTooltip;
	PinType.PinCategory = CategoryName;

	CachedAssetData = AssetData;

	bReadOnly = bInReadOnly;
	CachedDescription = InFriendlyName;
	CachedDescriptionString = MakeShared<FString>(CachedDescription.ToString());
}

UEdGraphSchema_K2::FPinTypeTreeInfo::FPinTypeTreeInfo(TSharedPtr<FPinTypeTreeInfo> InInfo)
{
	PinType = InInfo->PinType;
	bReadOnly = InInfo->bReadOnly;
	CachedAssetData = InInfo->CachedAssetData;
	Tooltip = InInfo->Tooltip;
	CachedDescription = InInfo->CachedDescription;
	CachedDescriptionString = InInfo->CachedDescriptionString;
	PossibleObjectReferenceTypes = InInfo->PossibleObjectReferenceTypes;
}

const FText& UEdGraphSchema_K2::FPinTypeTreeInfo::GetDescription() const
{
	return CachedDescription;
}

FText UEdGraphSchema_K2::FPinTypeTreeInfo::GenerateDescription()
{
	FText DisplayName;

	if (UObject* SubCategoryObject = PinType.PinSubCategoryObject.Get(); ensure(SubCategoryObject))
	{
		if (UField* SubCategoryField = Cast<UField>(SubCategoryObject))
		{
			DisplayName = SubCategoryField->GetDisplayNameText();
		}
		else
		{
			DisplayName = FText::FromString(FName::NameToDisplayString(SubCategoryObject->GetName(), PinType.PinCategory == PC_Boolean));
		}
	}

	return DisplayName;
}

const FAssetData& UEdGraphSchema_K2::FPinTypeTreeInfo::GetCachedAssetData() const
{
	return CachedAssetData;
}

const FName UEdGraphSchema_K2::PC_Exec(TEXT("exec"));
const FName UEdGraphSchema_K2::PC_Boolean(TEXT("bool"));
const FName UEdGraphSchema_K2::PC_Byte(TEXT("byte"));
const FName UEdGraphSchema_K2::PC_Class(TEXT("class"));
const FName UEdGraphSchema_K2::PC_Int(TEXT("int"));
const FName UEdGraphSchema_K2::PC_Int64(TEXT("int64"));
const FName UEdGraphSchema_K2::PC_Float(TEXT("float"));
const FName UEdGraphSchema_K2::PC_Double(TEXT("double"));
const FName UEdGraphSchema_K2::PC_Real(TEXT("real"));
const FName UEdGraphSchema_K2::PC_Name(TEXT("name"));
const FName UEdGraphSchema_K2::PC_Delegate(TEXT("delegate"));
const FName UEdGraphSchema_K2::PC_MCDelegate(TEXT("mcdelegate"));
const FName UEdGraphSchema_K2::PC_Object(TEXT("object"));
const FName UEdGraphSchema_K2::PC_Interface(TEXT("interface"));
const FName UEdGraphSchema_K2::PC_String(TEXT("string"));
const FName UEdGraphSchema_K2::PC_Text(TEXT("text"));
const FName UEdGraphSchema_K2::PC_Struct(TEXT("struct"));
const FName UEdGraphSchema_K2::PC_Wildcard(TEXT("wildcard"));
const FName UEdGraphSchema_K2::PC_FieldPath(TEXT("fieldpath"));
const FName UEdGraphSchema_K2::PC_Enum(TEXT("enum"));
const FName UEdGraphSchema_K2::PC_SoftObject(TEXT("softobject"));
const FName UEdGraphSchema_K2::PC_SoftClass(TEXT("softclass"));
const FName UEdGraphSchema_K2::PSC_Self(TEXT("self"));
const FName UEdGraphSchema_K2::PSC_Index(TEXT("index"));
const FName UEdGraphSchema_K2::PSC_Bitmask(TEXT("bitmask"));
const FName UEdGraphSchema_K2::PN_Execute(TEXT("execute"));
const FName UEdGraphSchema_K2::PN_Then(TEXT("then"));
const FName UEdGraphSchema_K2::PN_Completed(TEXT("Completed"));
const FName UEdGraphSchema_K2::PN_DelegateEntry(TEXT("delegate"));
const FName UEdGraphSchema_K2::PN_EntryPoint(TEXT("EntryPoint"));
const FName UEdGraphSchema_K2::PN_Self(TEXT("self"));
const FName UEdGraphSchema_K2::PN_Else(TEXT("else"));
const FName UEdGraphSchema_K2::PN_Loop(TEXT("loop"));
const FName UEdGraphSchema_K2::PN_After(TEXT("after"));
const FName UEdGraphSchema_K2::PN_ReturnValue(TEXT("ReturnValue"));
const FName UEdGraphSchema_K2::PN_ObjectToCast(TEXT("Object"));
const FName UEdGraphSchema_K2::PN_Condition(TEXT("Condition"));
const FName UEdGraphSchema_K2::PN_Start(TEXT("Start"));
const FName UEdGraphSchema_K2::PN_Stop(TEXT("Stop"));
const FName UEdGraphSchema_K2::PN_Index(TEXT("Index"));
const FName UEdGraphSchema_K2::PN_Item(TEXT("Item"));
const FName UEdGraphSchema_K2::PN_CastSucceeded(TEXT("then"));
const FName UEdGraphSchema_K2::PN_CastFailed(TEXT("CastFailed"));
const FString UEdGraphSchema_K2::PN_CastedValuePrefix(TEXT("As"));

const FName UEdGraphSchema_K2::FN_UserConstructionScript(TEXT("UserConstructionScript"));
const FName UEdGraphSchema_K2::FN_ExecuteUbergraphBase(TEXT("ExecuteUbergraph"));
const FName UEdGraphSchema_K2::GN_EventGraph(TEXT("EventGraph"));
const FName UEdGraphSchema_K2::GN_AnimGraph(TEXT("AnimGraph"));
const FText UEdGraphSchema_K2::VR_DefaultCategory(LOCTEXT("Default", "Default"));

const int32 UEdGraphSchema_K2::AG_LevelReference = 100;

const UScriptStruct* UEdGraphSchema_K2::VectorStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::Vector3fStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::Vector2DStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::RotatorStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::TransformStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::LinearColorStruct = nullptr;
const UScriptStruct* UEdGraphSchema_K2::ColorStruct = nullptr;

bool UEdGraphSchema_K2::bGeneratingDocumentation = false;
int32 UEdGraphSchema_K2::CurrentCacheRefreshID = 0;

const FName UEdGraphSchema_K2::AllObjectTypes(TEXT("ObjectTypes"));

namespace UEdGraphSchemaImpl
{
	bool ShouldActuallyTransact()
	{
		return !IsInAsyncLoadingThread();
	}
}

UEdGraphSchema_K2::UEdGraphSchema_K2(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Initialize cached static references to well-known struct types
	if (VectorStruct == nullptr)
	{
		VectorStruct = TBaseStructure<FVector>::Get();
		Vector3fStruct = TVariantStructure<FVector3f>::Get();
		Vector2DStruct = TBaseStructure<FVector2D>::Get();
		RotatorStruct = TBaseStructure<FRotator>::Get();
		TransformStruct = TBaseStructure<FTransform>::Get();
		LinearColorStruct = TBaseStructure<FLinearColor>::Get();
		ColorStruct = TBaseStructure<FColor>::Get();
	}
}

bool UEdGraphSchema_K2::DoesFunctionHaveOutParameters( const UFunction* Function ) const
{
	if ( Function != NULL )
	{
		for ( TFieldIterator<FProperty> PropertyIt(Function); PropertyIt; ++PropertyIt )
		{
			if ( PropertyIt->PropertyFlags & CPF_OutParm )
			{
				return true;
			}
		}
	}

	return false;
}

bool UEdGraphSchema_K2::ShouldShowPanelContextMenuForIncompatibleConnectionsImpl() const
{
	// Disabling this behavior for child types as it is experimental and
	// there are no tests that would exercise this codepath for them:
	return GetClass() == UEdGraphSchema_K2::StaticClass() && GetDefault<UBlueprintEditorSettings>()->bShowPanelContextMenuForIncompatibleConnections;
}

bool UEdGraphSchema_K2::CanFunctionBeUsedInGraph(const UClass* InClass, const UFunction* InFunction, const UEdGraph* InDestGraph, uint32 InAllowedFunctionTypes, bool bInCalledForEach, FText* OutReason) const
{
	if (CanUserKismetCallFunction(InFunction))
	{
		bool bLatentFuncsAllowed = true;
		bool bIsConstructionScript = false;

		if(InDestGraph != nullptr)
		{
			bLatentFuncsAllowed = (GetGraphType(InDestGraph) == GT_Ubergraph || (GetGraphType(InDestGraph) == GT_Macro));
			bIsConstructionScript = IsConstructionScript(InDestGraph);
		}

		const bool bIsPureFunc = (InFunction->HasAnyFunctionFlags(FUNC_BlueprintPure) != false);
		if (bIsPureFunc)
		{
			const bool bAllowPureFuncs = (InAllowedFunctionTypes & FT_Pure) != 0;
			if (!bAllowPureFuncs)
			{
				if(OutReason != nullptr)
				{
					*OutReason = LOCTEXT("PureFunctionsNotAllowed", "Pure functions are not allowed.");
				}

				return false;
			}
		}
		else
		{
			const bool bAllowImperativeFuncs = (InAllowedFunctionTypes & FT_Imperative) != 0;
			if (!bAllowImperativeFuncs)
			{
				if(OutReason != nullptr)
				{
					*OutReason = LOCTEXT("ImpureFunctionsNotAllowed", "Impure functions are not allowed.");
				}

				return false;
			}
		}

		const bool bIsConstFunc = (InFunction->HasAnyFunctionFlags(FUNC_Const) != false);
		const bool bAllowConstFuncs = (InAllowedFunctionTypes & FT_Const) != 0;
		if (bIsConstFunc && !bAllowConstFuncs)
		{
			if(OutReason != nullptr)
			{
				*OutReason = LOCTEXT("ConstFunctionsNotAllowed", "Const functions are not allowed.");
			}

			return false;
		}

		const bool bIsLatent = InFunction->HasMetaData(FBlueprintMetadata::MD_Latent);
		if (bIsLatent && !bLatentFuncsAllowed)
		{
			if(OutReason != nullptr)
			{
				*OutReason = LOCTEXT("LatentFunctionsNotAllowed", "Latent functions cannot be used here.");
			}

			return false;
		}

		const bool bIsNotNative = !FBlueprintEditorUtils::IsNativeSignature(InFunction);
		if(bIsNotNative)
		{
			// Blueprint functions visibility flags can be enforced in blueprints - native functions
			// are often using these flags to only hide functionality from other native functions:
			const bool bIsProtected = (InFunction->FunctionFlags & FUNC_Protected) != 0;
			const bool bFuncBelongsToSubClass = InClass && InClass->IsChildOf(InFunction->GetOuterUClass()->GetSuperStruct());
			if (bIsProtected)
			{
				const bool bAllowProtectedFuncs = (InAllowedFunctionTypes & FT_Protected) != 0;
				if (!bAllowProtectedFuncs)
				{
					if(OutReason != nullptr)
					{
						*OutReason = LOCTEXT("ProtectedFunctionsNotAllowed", "Protected functions are not allowed.");
					}

					return false;
				}

				if (!bFuncBelongsToSubClass)
				{
					if(OutReason != nullptr)
					{
						*OutReason = LOCTEXT("ProtectedFunctionInaccessible", "Function is protected and inaccessible.");
					}

					return false;
				}
			}

			const bool bIsPrivate = (InFunction->FunctionFlags & FUNC_Private) != 0;
			const bool bFuncBelongsToClass = bFuncBelongsToSubClass && (InClass->GetSuperStruct() == InFunction->GetOuterUClass()->GetSuperStruct());
			if ( bIsPrivate && !bFuncBelongsToClass)
			{
				if(OutReason != nullptr)
				{
					*OutReason = LOCTEXT("PrivateFunctionInaccessible", "Function is private and inaccessible.");
				}

				return false;
			}
		}

		const bool bIsUnsafeForConstruction = InFunction->GetBoolMetaData(FBlueprintMetadata::MD_UnsafeForConstructionScripts);	
		if (bIsUnsafeForConstruction && bIsConstructionScript)
		{
			if(OutReason != nullptr)
			{
				*OutReason = LOCTEXT("FunctionUnsafeForConstructionScript", "Function cannot be used in a Construction Script.");
			}

			return false;
		}

		// We no longer block functions that require world contexts as it will add the pin when needed

		const bool bFunctionStatic = InFunction->HasAllFunctionFlags(FUNC_Static);
		const bool bHasReturnParams = (InFunction->GetReturnProperty() != NULL);
		const bool bHasArrayPointerParms = InFunction->HasMetaData(FBlueprintMetadata::MD_ArrayParam);

		const bool bAllowForEachCall = !bFunctionStatic && !bIsLatent && !bIsPureFunc && !bIsConstFunc && !bHasReturnParams && !bHasArrayPointerParms;
		if (bInCalledForEach && !bAllowForEachCall)
		{
			if(OutReason != nullptr)
			{
				if(bFunctionStatic)
				{
					*OutReason = LOCTEXT("StaticFunctionsNotAllowedInForEachContext", "Static functions cannot be used within a ForEach context.");
				}
				else if(bIsLatent)
				{
					*OutReason = LOCTEXT("LatentFunctionsNotAllowedInForEachContext", "Latent functions cannot be used within a ForEach context.");
				}
				else if(bIsPureFunc)
				{
					*OutReason = LOCTEXT("PureFunctionsNotAllowedInForEachContext", "Pure functions cannot be used within a ForEach context.");
				}
				else if(bIsConstFunc)
				{
					*OutReason = LOCTEXT("ConstFunctionsNotAllowedInForEachContext", "Const functions cannot be used within a ForEach context.");
				}
				else if(bHasReturnParams)
				{
					*OutReason = LOCTEXT("FunctionsWithReturnValueNotAllowedInForEachContext", "Functions that return a value cannot be used within a ForEach context.");
				}
				else if(bHasArrayPointerParms)
				{
					*OutReason = LOCTEXT("FunctionsWithArrayParmsNotAllowedInForEachContext", "Functions with array parameters cannot be used within a ForEach context.");
				}
				else
				{
					*OutReason = LOCTEXT("FunctionNotAllowedInForEachContext", "Function cannot be used within a ForEach context.");
				}
			}

			return false;
		}

		return true;
	}

	if(OutReason != nullptr)
	{
		*OutReason = LOCTEXT("FunctionInvalid", "Invalid function.");
	}

	return false;
}

UFunction* UEdGraphSchema_K2::GetCallableParentFunction(UFunction* Function)
{
	if( Function && Cast<UClass>(Function->GetOuter()) )
	{
		const FName FunctionName = Function->GetFName();

		// Search up the parent scopes
		UClass* ParentClass = CastChecked<UClass>(Function->GetOuter())->GetSuperClass();
		UFunction* ClassFunction = ParentClass->FindFunctionByName(FunctionName);

		return ClassFunction;
	}

	return NULL;
}

bool UEdGraphSchema_K2::CanUserKismetCallFunction(const UFunction* Function)
{
	return Function && 
		(Function->HasAllFunctionFlags(FUNC_BlueprintCallable) && !Function->HasAllFunctionFlags(FUNC_Delegate) && !Function->GetBoolMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly) && (!Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction) || GetDefault<UBlueprintEditorSettings>()->bExposeDeprecatedFunctions));
}

bool UEdGraphSchema_K2::CanKismetOverrideFunction(const UFunction* Function)
{
	return  
		Function && 
		(
			Function->HasAllFunctionFlags(FUNC_BlueprintEvent)
			&& !Function->HasAllFunctionFlags(FUNC_Delegate) && 
			!Function->GetBoolMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly) && 
			(!Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction) || GetDefault<UBlueprintEditorSettings>()->bExposeDeprecatedFunctions)
		);
}

bool UEdGraphSchema_K2::HasFunctionAnyOutputParameter(const UFunction* InFunction)
{
	check(InFunction);
	for (TFieldIterator<FProperty> PropIt(InFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* FuncParam = *PropIt;
		if (FuncParam->HasAnyPropertyFlags(CPF_ReturnParm) || (FuncParam->HasAnyPropertyFlags(CPF_OutParm) && !FuncParam->HasAnyPropertyFlags(CPF_ReferenceParm) && !FuncParam->HasAnyPropertyFlags(CPF_ConstParm)))
		{
			return true;
		}
	}

	return false;
}

bool UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(const UFunction* InFunction)
{
	// First check we are override-able, non-static, non-const and not marked thread safe
	if (!InFunction || !CanKismetOverrideFunction(InFunction) || InFunction->HasAnyFunctionFlags(FUNC_Static|FUNC_Const) || FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(InFunction))
	{
		return false;
	}

	// Check if meta data has been set to force this to appear as blueprint function even if it doesn't return a value.
	if (InFunction->HasAllFunctionFlags(FUNC_BlueprintEvent) && InFunction->HasMetaData(FBlueprintMetadata::MD_ForceAsFunction))
	{
		return false;
	}

	// Then look to see if we have any output, return, or reference params
	return !HasFunctionAnyOutputParameter(InFunction);
}

bool UEdGraphSchema_K2::FunctionCanBeUsedInDelegate(const UFunction* InFunction)
{	
	if (!InFunction || 
		!CanUserKismetCallFunction(InFunction) ||
		InFunction->HasMetaData(FBlueprintMetadata::MD_Latent) ||
		InFunction->HasAllFunctionFlags(FUNC_BlueprintPure))
	{
		return false;
	}

	return true;
}

FText UEdGraphSchema_K2::GetFriendlySignatureName(const UFunction* Function)
{
	return ObjectTools::GetUserFacingFunctionName( Function );
}

void UEdGraphSchema_K2::GetAutoEmitTermParameters(const UFunction* Function, TArray<FString>& AutoEmitParameterNames)
{
	AutoEmitParameterNames.Reset();

	const FString& MetaData = Function->GetMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm);
	if (!MetaData.IsEmpty())
	{
		MetaData.ParseIntoArray(AutoEmitParameterNames, TEXT(","), true);

		for (int32 NameIndex = 0; NameIndex < AutoEmitParameterNames.Num();)
		{
			FString& ParameterName = AutoEmitParameterNames[NameIndex];
			ParameterName.TrimStartAndEndInline();
			if (ParameterName.IsEmpty())
			{
				AutoEmitParameterNames.RemoveAtSwap(NameIndex);
			}
			else
			{
				++NameIndex;
			}
		}
	}

	// Allow any params that are blueprint defined to be autocreated:
	if (!FBlueprintEditorUtils::IsNativeSignature(Function))
	{
		for (	TFieldIterator<FProperty> ParamIter(Function, EFieldIterationFlags::Default); 
				ParamIter && (ParamIter->PropertyFlags & CPF_Parm); 
				++ParamIter)
		{
			FProperty* Param = *ParamIter;
			if(Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				AutoEmitParameterNames.Add(Param->GetName());
			}
		}
	}
}

bool UEdGraphSchema_K2::FunctionHasParamOfType(const UFunction* InFunction, UEdGraph const* InGraph, const FEdGraphPinType& DesiredPinType, bool bWantOutput) const
{
	TSet<FName> HiddenPins;
	FBlueprintEditorUtils::GetHiddenPinsForFunction(InGraph, InFunction, HiddenPins);

	// Iterate over all params of function
	for (TFieldIterator<FProperty> PropIt(InFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* FuncParam = *PropIt;

		// Ensure that this isn't a hidden parameter
		if (!HiddenPins.Contains(FuncParam->GetFName()))
		{
			// See if this is the direction we want (input or output)
			const bool bIsFunctionInput = !FuncParam->HasAnyPropertyFlags(CPF_ReturnParm) && (!FuncParam->HasAnyPropertyFlags(CPF_OutParm) || FuncParam->HasAnyPropertyFlags(CPF_ReferenceParm));
			if (bIsFunctionInput != bWantOutput)
			{
				// See if this pin has compatible types
				FEdGraphPinType ParamPinType;
				bool bConverted = ConvertPropertyToPinType(FuncParam, ParamPinType);
				if (bConverted)
				{
					UClass* Context = nullptr;
					UBlueprint* Blueprint = Cast<UBlueprint>(InGraph->GetOuter());
					if (Blueprint)
					{
						Context = Blueprint->GeneratedClass;
					}

					if (bIsFunctionInput && ArePinTypesCompatible(DesiredPinType, ParamPinType, Context))
					{
						return true;
					}
					else if (!bIsFunctionInput && ArePinTypesCompatible(ParamPinType, DesiredPinType, Context))
					{
						return true;
					}
				}
			}
		}
	}

	// Boo, no pin of this type
	return false;
}

void UEdGraphSchema_K2::AddExtraFunctionFlags(const UEdGraph* CurrentGraph, int32 ExtraFlags) const
{
	for (UEdGraphNode* Node : CurrentGraph->Nodes)
	{
		if (UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(Node))
		{
			EntryNode->AddExtraFlags(ExtraFlags);
		}
	}
}

void UEdGraphSchema_K2::MarkFunctionEntryAsEditable(const UEdGraph* CurrentGraph, bool bNewEditable) const
{
	for (UEdGraphNode* Node : CurrentGraph->Nodes)
	{
		if (UK2Node_EditablePinBase* EditableNode = Cast<UK2Node_EditablePinBase>(Node))
		{
			EditableNode->Modify();
			EditableNode->bIsEditable = bNewEditable;
		}
	}
}

bool UEdGraphSchema_K2::IsActorValidForLevelScriptRefs(const AActor* TestActor, const UBlueprint* Blueprint) const
{
	check(Blueprint);

	return TestActor
		&& FBlueprintEditorUtils::IsLevelScriptBlueprint(Blueprint)
		&& (TestActor->GetLevel() == FBlueprintEditorUtils::GetLevelFromBlueprint(Blueprint))
		&& FKismetEditorUtilities::IsActorValidForLevelScript(TestActor);
}

void UEdGraphSchema_K2::ReplaceSelectedNode(UEdGraphNode* SourceNode, AActor* TargetActor)
{
	check(SourceNode);

	if (TargetActor != NULL)
	{
		UK2Node_Literal* LiteralNode = (UK2Node_Literal*)(SourceNode);
		if (LiteralNode)
		{
			const FScopedTransaction Transaction( LOCTEXT("ReplaceSelectedNodeUndoTransaction", "Replace Selected Node"), UEdGraphSchemaImpl::ShouldActuallyTransact());

			LiteralNode->Modify();
			LiteralNode->SetObjectRef( TargetActor );
			LiteralNode->ReconstructNode();
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CastChecked<UEdGraph>(SourceNode->GetOuter()));
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

void UEdGraphSchema_K2::AddSelectedReplaceableNodes(FToolMenuSection& Section, UBlueprint* Blueprint, const UEdGraphNode* InGraphNode) const
{
	//Only allow replace object reference functionality for literal nodes
	const UK2Node_Literal* LiteralNode = Cast<UK2Node_Literal>(InGraphNode);
	if (LiteralNode)
	{
		USelection* SelectedActors = GEditor->GetSelectedActors();
		for(FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
		{
			// We only care about actors that are referenced in the world for literals, and also in the same level as this blueprint
			AActor* Actor = Cast<AActor>(*Iter);
			if( LiteralNode->GetObjectRef() != Actor && IsActorValidForLevelScriptRefs(Actor, Blueprint) )
			{
				FText Description = FText::Format( LOCTEXT("ChangeToActorName", "Change to <{0}>"), FText::FromString( Actor->GetActorLabel() ) );
				FText ToolTip = LOCTEXT("ReplaceNodeReferenceToolTip", "Replace node reference");
				Section.AddMenuEntry(NAME_None, Description, ToolTip, FSlateIcon(), FUIAction(
					FExecuteAction::CreateUObject((UEdGraphSchema_K2*const)this, &UEdGraphSchema_K2::ReplaceSelectedNode, const_cast< UEdGraphNode* >(InGraphNode), Actor) ) );
			}
		}
	}
}



bool UEdGraphSchema_K2::CanUserKismetAccessVariable(const FProperty* Property, const UClass* InClass, EDelegateFilterMode FilterMode)
{
	const bool bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
	const bool bIsAccessible = Property->HasAllPropertyFlags(CPF_BlueprintVisible);
	const bool bIsAssignableOrCallable = Property->HasAnyPropertyFlags(CPF_BlueprintAssignable | CPF_BlueprintCallable);
	
	const bool bPassesDelegateFilter = (bIsAccessible && !bIsDelegate && (FilterMode != MustBeDelegate)) || 
		(bIsAssignableOrCallable && bIsDelegate && (FilterMode != CannotBeDelegate));

	const bool bHidden = FObjectEditorUtils::IsVariableCategoryHiddenFromClass(Property, InClass);

	return !Property->HasAnyPropertyFlags(CPF_Parm) && bPassesDelegateFilter && !bHidden;
}

bool UEdGraphSchema_K2::ClassHasBlueprintAccessibleMembers(const UClass* InClass) const
{
	// @TODO Don't show other blueprints yet...
	UBlueprint* ClassBlueprint = UBlueprint::GetBlueprintFromClass(InClass);
	if (!InClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists) && (ClassBlueprint == NULL))
	{
		// Find functions
		for (TFieldIterator<UFunction> FunctionIt(InClass, EFieldIteratorFlags::IncludeSuper); FunctionIt; ++FunctionIt)
		{
			UFunction* Function = *FunctionIt;
			const bool bIsBlueprintProtected = Function->GetBoolMetaData(FBlueprintMetadata::MD_Protected);
			const bool bHidden = FObjectEditorUtils::IsFunctionHiddenFromClass(Function, InClass);
			if (UEdGraphSchema_K2::CanUserKismetCallFunction(Function) && !bIsBlueprintProtected && !bHidden)
			{
				return true;
			}
		}

		// Find vars
		for (TFieldIterator<FProperty> PropertyIt(InClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			if (CanUserKismetAccessVariable(Property, InClass, CannotBeDelegate))
			{
				return true;
			}
		}
	}

	return false;
}

bool UEdGraphSchema_K2::IsAllowableBlueprintVariableType(const UEnum* InEnum)
{
	return InEnum && (InEnum->GetBoolMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType) || InEnum->IsA<UUserDefinedEnum>());
}

bool UEdGraphSchema_K2::IsAllowableBlueprintVariableType(const UClass* InClass, bool bAssumeBlueprintType)
{
	if (InClass)
	{
		// No Skeleton classes or reinstancing classes (they would inherit the BlueprintType metadata)
		if (FKismetEditorUtilities::IsClassABlueprintSkeleton(InClass)
			|| InClass->HasAnyClassFlags(CLASS_NewerVersionExists))
		{
			return false;
		}

		// No Blueprint Macro Libraries
		if (FKismetEditorUtilities::IsClassABlueprintMacroLibrary(InClass))
		{
			return false;
		}

		// UObject is an exception, and is always a blueprint-able type
		if(InClass == UObject::StaticClass())
		{
			return true;
		}
		
		// cannot have level script variables
		if (InClass->IsChildOf(ALevelScriptActor::StaticClass()))
		{
			return false;
		}

		const UClass* ParentClass = InClass;
		while(ParentClass)
		{
			// Climb up the class hierarchy and look for "BlueprintType" and "NotBlueprintType" to see if this class is allowed.
			if(ParentClass->GetBoolMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType)
				|| ParentClass->HasMetaData(FBlueprintMetadata::MD_BlueprintSpawnableComponent))
			{
				return true;
			}
			else if(ParentClass->GetBoolMetaData(FBlueprintMetadata::MD_NotAllowableBlueprintVariableType))
			{
				return false;
			}
			ParentClass = ParentClass->GetSuperClass();
		}
	}
	
	return bAssumeBlueprintType;
}

bool UEdGraphSchema_K2::IsAllowableBlueprintVariableType(const UScriptStruct* InStruct, const bool bForInternalUse)
{
	if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(InStruct))
	{
		if (EUserDefinedStructureStatus::UDSS_UpToDate != UDStruct->Status.GetValue())
		{
			return false;
		}

		// User-defined structs are always allowed as BP variable types.
		return true;
	}

	// struct needs to be marked as BP type
	if (InStruct && InStruct->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_AllowableBlueprintVariableType))
	{
		// for internal use, all BP types are allowed
		if (bForInternalUse)
		{
			return true;
		}

		// for user-facing use case, only allow structs that don't have the internal-use-only tag
		// struct itself should not be tagged
		if (!InStruct->GetBoolMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly))
		{
			// struct's base structs should not be tagged
			if (!InStruct->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_BlueprintInternalUseOnlyHierarchical))
			{
				return true;
			}
		}
	}

	return false;
}

bool UEdGraphSchema_K2::DoesGraphSupportImpureFunctions(const UEdGraph* InGraph) const
{
	const EGraphType GraphType = GetGraphType(InGraph);
	const bool bAllowImpureFuncs = GraphType != GT_Animation; //@TODO: It's really more nuanced than this (e.g., in a function someone wants to write as pure)

	return bAllowImpureFuncs;
}

bool UEdGraphSchema_K2::IsGraphMarkedThreadSafe(const UEdGraph* InGraph) const
{
	TArray<UK2Node_FunctionEntry*> EntryNodes;
	InGraph->GetNodesOfClass(EntryNodes);

	for(UK2Node_FunctionEntry* EntryNode : EntryNodes)
	{
		if(EntryNode->MetaData.bThreadSafe)
		{
			return true;
		}
		else if(UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(EntryNode))
		{
			if(FBlueprintEditorUtils::HasFunctionBlueprintThreadSafeMetaData(Function))
			{
				return true;
			}
		}
	}

	return false;
}

bool UEdGraphSchema_K2::IsPropertyExposedOnSpawn(const FProperty* Property)
{
	Property = FBlueprintEditorUtils::GetMostUpToDateProperty(Property);
	if (Property)
	{
		const bool bMeta = Property->HasMetaData(FBlueprintMetadata::MD_ExposeOnSpawn);
		const bool bFlag = Property->HasAllPropertyFlags(CPF_ExposeOnSpawn);
		if (bMeta != bFlag)
		{
			const FCoreTexts& CoreTexts = FCoreTexts::Get();

			UE_LOGF(LogBlueprint, Warning
				, "ExposeOnSpawn ambiguity. Property '%ls', MetaData '%ls', Flag '%ls'"
				, *Property->GetFullName()
				, bMeta ? *CoreTexts.True.ToString() : *CoreTexts.False.ToString()
				, bFlag ? *CoreTexts.True.ToString() : *CoreTexts.False.ToString());
		}
		return bMeta || bFlag;
	}
	return false;
}

bool UEdGraphSchema_K2::IsPropertyDeprecated(const FProperty* Property)
{
	Property = FBlueprintEditorUtils::GetMostUpToDateProperty(Property);
	if (Property)
	{
		const bool bMeta = Property->HasMetaData(FBlueprintMetadata::MD_DeprecatedProperty);
		const bool bFlag = Property->HasAllPropertyFlags(CPF_Deprecated);
		return bMeta || bFlag;
	}
	return false;
}

// if node is a get/set variable and the variable it refers to does not exist
static bool IsUsingNonExistantVariable(const UEdGraphNode* InGraphNode, UBlueprint* OwnerBlueprint)
{
	bool bNonExistantVariable = false;
	const bool bBreakOrMakeStruct = 
		InGraphNode->IsA(UK2Node_BreakStruct::StaticClass()) || 
		InGraphNode->IsA(UK2Node_MakeStruct::StaticClass());
	if (!bBreakOrMakeStruct)
	{
		if (const UK2Node_Variable* Variable = Cast<const UK2Node_Variable>(InGraphNode))
		{
			if (Variable->VariableReference.IsSelfContext())
			{
				TSet<FName> CurrentVars;
				FBlueprintEditorUtils::GetClassVariableList(OwnerBlueprint, CurrentVars);
				if ( false == CurrentVars.Contains(Variable->GetVarName()) )
				{
					bNonExistantVariable = true;
				}
			}
			else if(Variable->VariableReference.IsLocalScope())
			{
				// If there is no member scope, or we can't find the local variable in the member scope, then it's non-existant
				UStruct* MemberScope = Variable->VariableReference.GetMemberScope(Variable->GetBlueprintClassFromNode());
				if (MemberScope == nullptr || !FBlueprintEditorUtils::FindLocalVariable(OwnerBlueprint, MemberScope, Variable->GetVarName()))
				{
					bNonExistantVariable = true;
				}
			}
		}
	}
	return bNonExistantVariable;
}

bool UEdGraphSchema_K2::PinHasSplittableStructType(const UEdGraphPin* InGraphPin) const
{
	const FEdGraphPinType& PinType = InGraphPin->PinType;
	bool bCanSplit = (!PinType.IsContainer() && PinType.PinCategory == PC_Struct);

	if (bCanSplit)
	{
		UScriptStruct* StructType = Cast<UScriptStruct>(InGraphPin->PinType.PinSubCategoryObject.Get());

		// Check if the user has explicitly disabled split pins
		const bool bDisableSplit = StructType ? StructType->HasMetaData(FBlueprintMetadata::MD_NativeDisableSplitPin) : false;

		if (StructType && !bDisableSplit)
		{
			if (InGraphPin->Direction == EGPD_Input)
			{
				UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InGraphPin->GetOwningNode());
				bCanSplit = UK2Node_MakeStruct::CanBeSplit(StructType, Blueprint);
				if (!bCanSplit)
				{
					const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeMakeFunction);
					UFunction* Function = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);
					bCanSplit = (Function != nullptr);
				}
			}
			else
			{
				bCanSplit = UK2Node_BreakStruct::CanBeSplit(StructType);
				if (!bCanSplit)
				{
					const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeBreakFunction);
					UFunction* Function = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);
					bCanSplit = (Function != nullptr);
				}
			}
		}
		else
		{
			// If the struct type of a split struct pin no longer exists this can happen
			bCanSplit = false;
		}
	}

	return bCanSplit;
}

bool UEdGraphSchema_K2::PinDefaultValueIsEditable(const UEdGraphPin& InGraphPin) const
{
	// Array types are not currently assignable without a 'make array' node:
	if( InGraphPin.PinType.IsContainer() )
	{
		return false;
	}

	// User defined structures (from code or from data) cannot accept default values:
	if( InGraphPin.PinType.PinCategory == PC_Struct )
	{
		// Only the built in struct types are editable as 'default' values on a pin.
		// See FNodeFactory::CreatePinWidget for justification of the above statement!
		UObject const& SubCategoryObject = *InGraphPin.PinType.PinSubCategoryObject;
		return &SubCategoryObject == VectorStruct 
			|| &SubCategoryObject == Vector3fStruct
			|| &SubCategoryObject == Vector2DStruct
			|| &SubCategoryObject == RotatorStruct
			|| &SubCategoryObject == TransformStruct
			|| &SubCategoryObject == LinearColorStruct
			|| &SubCategoryObject == ColorStruct
			|| &SubCategoryObject == FCollisionProfileName::StaticStruct();
	}

	return true;
}

bool UEdGraphSchema_K2::PinHasCustomDefaultFormat(const UEdGraphPin& InGraphPin) const
{
	if (InGraphPin.PinType.PinCategory == PC_Struct)
	{
		// Some struct types have custom formats for default value for historical reasons
		UObject const& SubCategoryObject = *InGraphPin.PinType.PinSubCategoryObject;
		return &SubCategoryObject == VectorStruct
			|| &SubCategoryObject == Vector3fStruct
			|| &SubCategoryObject == RotatorStruct
			|| &SubCategoryObject == TransformStruct
			|| &SubCategoryObject == LinearColorStruct;
	}
	return false;
}

void UEdGraphSchema_K2::SelectAllNodesInDirection(TEnumAsByte<enum EEdGraphPinDirection> InDirection, UEdGraph* Graph, UEdGraphPin* InGraphPin)
{
	/** Traverses the node graph out from the specified pin, logging each node that it visits along the way. */
	struct FDirectionalNodeVisitor
	{
		FDirectionalNodeVisitor(UEdGraphPin* StartingPin, EEdGraphPinDirection TargetDirection)
			: Direction(TargetDirection)
		{
			TraversePin(StartingPin);
		}

		/** If the pin is the right direction, visits each of its attached nodes */
		void TraversePin(UEdGraphPin* Pin)
		{
			if (Pin->Direction == Direction)
			{
				for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
				{
					VisitNode(LinkedPin->GetOwningNode());
				}
			}
		}

		/** If the node has already been visited, does nothing. Otherwise it traverses each of its pins. */
		void VisitNode(UEdGraphNode* Node)
		{
			bool bAlreadyVisited = false;
			VisitedNodes.Add(Node, &bAlreadyVisited);

			if (!bAlreadyVisited)
			{
				for (UEdGraphPin* Pin : Node->Pins)
				{
					TraversePin(Pin);
				}
			}
		}

		EEdGraphPinDirection Direction;
		TSet<UEdGraphNode*>  VisitedNodes;
	};

	FDirectionalNodeVisitor NodeVisitor(InGraphPin, InDirection);
	for (UEdGraphNode* Node : NodeVisitor.VisitedNodes)
	{
		FKismetEditorUtilities::AddToSelection(Graph, Node);
	}
}

void UEdGraphSchema_K2::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	const UEdGraph* CurrentGraph = Context->Graph;
	const UEdGraphNode* InGraphNode = Context->Node;
	const UEdGraphPin* InGraphPin = Context->Pin;
	const bool bIsDebugging = Context->bIsDebugging;
	check(CurrentGraph);
	UBlueprint* OwnerBlueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(CurrentGraph);

	if (InGraphPin)
	{
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("EdGraphSchemaPinActions");
			if (!bIsDebugging)
			{	
				// Add the change pin type action, if this is a select node
				if (InGraphNode->IsA(UK2Node_Select::StaticClass()))
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().ChangePinType);
				}
	
				// Conditionally add the var promotion pin if this is an output pin and it's not an exec pin
				if (InGraphPin->PinType.PinCategory != PC_Exec)
				{
					Section.AddMenuEntry( FGraphEditorCommands::Get().PromoteToVariable );

					if (FBlueprintEditorUtils::DoesSupportLocalVariables(CurrentGraph))
					{
						Section.AddMenuEntry( FGraphEditorCommands::Get().PromoteToLocalVariable );
					}
				}
	
				if (InGraphPin->PinType.PinCategory == PC_Struct && InGraphNode->CanSplitPin(InGraphPin))
				{
					// If the pin cannot be split, create an error tooltip to use
					FText Tooltip;
					if (PinHasSplittableStructType(InGraphPin))
					{
						Tooltip = FGraphEditorCommands::Get().SplitStructPin->GetDescription();
					}
					else
					{
						Tooltip = LOCTEXT("SplitStructPin_Error", "Cannot split the struct pin, it may be missing Blueprint exposed properties!");
					}
					Section.AddMenuEntry( FGraphEditorCommands::Get().SplitStructPin, FGraphEditorCommands::Get().SplitStructPin->GetLabel(), Tooltip );
				}

				if (InGraphPin->ParentPin != NULL)
				{
					Section.AddMenuEntry( FGraphEditorCommands::Get().RecombineStructPin );
				}
	
				// Conditionally add the execution path pin options if this is an execution branching node
				if( InGraphPin->Direction == EGPD_Output && InGraphPin->GetOwningNode())
				{
					if (CastChecked<UK2Node>(InGraphPin->GetOwningNode())->CanEverInsertExecutionPin())
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().InsertExecutionPinBefore);
						Section.AddMenuEntry(FGraphEditorCommands::Get().InsertExecutionPinAfter);
					}

					if (CastChecked<UK2Node>(InGraphPin->GetOwningNode())->CanEverRemoveExecutionPin())
					{
						Section.AddMenuEntry( FGraphEditorCommands::Get().RemoveExecutionPin );
					}
				}

				if (UK2Node_SetFieldsInStruct::ShowCustomPinActions(InGraphPin, true))
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().RemoveThisStructVarPin);
					Section.AddMenuEntry(FGraphEditorCommands::Get().RemoveOtherStructVarPins);
				}

				if (InGraphPin->PinType.PinCategory != PC_Exec && InGraphPin->Direction == EGPD_Input && InGraphPin->LinkedTo.Num() == 0 && !ShouldHidePinDefaultValue(const_cast<UEdGraphPin*>(InGraphPin)))
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().ResetPinToDefaultValue);
				}
			}
		}

		// Add the watch pin / unwatch pin menu items
		{
			FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaWatches", LOCTEXT("WatchesHeader", "Watches"));
			if (!IsMetaPin(*InGraphPin))
			{
				const UEdGraphPin* WatchedPin = ((InGraphPin->Direction == EGPD_Input) && (InGraphPin->LinkedTo.Num() > 0)) ? InGraphPin->LinkedTo[0] : InGraphPin;
				if (FKismetDebugUtilities::IsPinBeingWatched(OwnerBlueprint, WatchedPin))
				{
					Section.AddMenuEntry( FGraphEditorCommands::Get().StopWatchingPin );
				}
				else
				{
					Section.AddMenuEntry( FGraphEditorCommands::Get().StartWatchingPin );
				}
			}
		}
	}
	else if (InGraphNode != NULL)
	{
		if (IsUsingNonExistantVariable(InGraphNode, OwnerBlueprint))
		{
			{
				FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
				GetNonExistentVariableMenu(Section, InGraphNode, OwnerBlueprint);
			}
		}
		else
		{
			{
				FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeActions", LOCTEXT("NodeActionsMenuHeader", "Node Actions"));
				if (!bIsDebugging)
				{
					// Replaceable node display option
					AddSelectedReplaceableNodes(Section, OwnerBlueprint, InGraphNode);

					// Node contextual actions
					Section.AddMenuEntry( FGenericCommands::Get().Delete );
					Section.AddMenuEntry( FGenericCommands::Get().Cut );
					Section.AddMenuEntry( FGenericCommands::Get().Copy );
					Section.AddMenuEntry( FGenericCommands::Get().Duplicate );
					Section.AddMenuEntry( FGraphEditorCommands::Get().ReconstructNodes );
					Section.AddMenuEntry( FGraphEditorCommands::Get().BreakNodeLinks );

					// Conditionally add the action to add an execution pin, if this is an execution node
					if( InGraphNode->IsA(UK2Node_ExecutionSequence::StaticClass()) || InGraphNode->IsA(UK2Node_Switch::StaticClass()) )
					{
						Section.AddMenuEntry( FGraphEditorCommands::Get().AddExecutionPin );
					}

					// Conditionally add the action to create a super function call node, if this is an event or function entry
					if( InGraphNode->IsA(UK2Node_Event::StaticClass()) || InGraphNode->IsA(UK2Node_FunctionEntry::StaticClass()) )
					{
						Section.AddMenuEntry( FGraphEditorCommands::Get().AddParentNode );
					}

					// Conditionally add the actions to add or remove an option pin, if this is a select node
					if (InGraphNode->IsA(UK2Node_Select::StaticClass()))
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().AddOptionPin);
						Section.AddMenuEntry(FGraphEditorCommands::Get().RemoveOptionPin);
					}

					// Don't show the "Assign selected Actor" option if more than one actor is selected
					if (InGraphNode->IsA(UK2Node_ActorBoundEvent::StaticClass()) && GEditor->GetSelectedActorCount() == 1)
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().AssignReferencedActor);
					}

					// Conditionally show the "Create Matching Function" option if it is an unresolved CallFunction node
					if (const UK2Node_CallFunction* FuncNode = Cast<UK2Node_CallFunction>(InGraphNode))
					{
						if (!FuncNode->GetTargetFunction())
						{
							Section.AddMenuEntry(FGraphEditorCommands::Get().CreateMatchingFunction);
						}
					}
				}

				// If the node has an associated definition (for some loose sense of the word), allow going to it (same action as double-clicking on a node)
				if (InGraphNode->CanJumpToDefinition())
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().GoToDefinition);
				}

				// Show search for references for everyone. Depending on context, it's an action or a submenu.
				const bool bIsFuncOrVarNode = InGraphNode->IsA<UK2Node_CallFunction>() || InGraphNode->IsA<UK2Node_Event>() || InGraphNode->IsA<UK2Node_FunctionTerminator>() || InGraphNode->IsA<UK2Node_Variable>(); 
				const bool bExpandFindReferences = bIsFuncOrVarNode;
				if (!bExpandFindReferences)
				{
					Section.AddMenuEntry(FGraphEditorCommands::Get().FindReferences);
				}
				else
				{
					// Expandable menu: insert sub-menu here
					Section.AddSubMenu(
						FName("FindReferenceSubMenu"),
						LOCTEXT("FindReferences_Label", "Find References"),
						LOCTEXT("FindReferences_Tooltip", "Options for finding references to class members"),
						FNewToolMenuChoice(FNewMenuDelegate::CreateStatic(&FGraphEditorCommands::BuildFindReferencesMenu))
					);
				}

				if (!bIsDebugging)
				{
					if (InGraphNode->IsA(UK2Node_Variable::StaticClass()))
					{
						GetReplaceVariableMenu(Section, InGraphNode, OwnerBlueprint, true);
					}

					if (InGraphNode->IsA(UK2Node_SetFieldsInStruct::StaticClass()))
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().RestoreAllStructVarPins);
					}

					Section.AddMenuEntry(FGenericCommands::Get().Rename, LOCTEXT("Rename", "Rename"), LOCTEXT("Rename_Tooltip", "Renames selected function or variable in blueprint.") );
				}

				// Select referenced actors in the level
				Section.AddMenuEntry(FGraphEditorCommands::Get().SelectReferenceInLevel);
			}

			{
				FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaNodeNavigation", LOCTEXT("NodeNavigationMenuHeader", "Navigation"));
				// Opens a function, macro or collapsed graph in a new tab
				Section.AddMenuEntry(FGraphEditorCommands::Get().OpenInNewTab);
			}

			if (!bIsDebugging)
			{
				// Collapse/expand nodes
				{
					FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaOrganization", LOCTEXT("OrganizationHeader", "Organization"));
					Section.AddMenuEntry( FGraphEditorCommands::Get().CollapseNodes );
					Section.AddMenuEntry( FGraphEditorCommands::Get().CollapseSelectionToFunction );
					Section.AddMenuEntry( FGraphEditorCommands::Get().CollapseSelectionToMacro );
					Section.AddMenuEntry( FGraphEditorCommands::Get().ExpandNodes );

					if (InGraphNode->IsA(UK2Node_FunctionEntry::StaticClass()))
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().ConvertFunctionToEvent);
					}

					if (InGraphNode->IsA(UK2Node_Event::StaticClass()))
					{
						Section.AddMenuEntry(FGraphEditorCommands::Get().ConvertEventToFunction);
					}

					if(InGraphNode->IsA(UK2Node_Composite::StaticClass()))
					{
						Section.AddMenuEntry( FGraphEditorCommands::Get().PromoteSelectionToFunction );
						Section.AddMenuEntry( FGraphEditorCommands::Get().PromoteSelectionToMacro );
					}

					Section.AddSubMenu("Alignment", LOCTEXT("AlignmentHeader", "Alignment"), FText(), FNewToolMenuDelegate::CreateLambda([](UToolMenu* AlignmentMenu)
					{
						{
							FToolMenuSection& InSection = AlignmentMenu->AddSection("EdGraphSchemaAlignment", LOCTEXT("AlignHeader", "Align"));
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesTop);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesMiddle);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesBottom);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesLeft);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesCenter);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().AlignNodesRight);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().StraightenConnections);
						}

						{
							FToolMenuSection& InSection = AlignmentMenu->AddSection("EdGraphSchemaDistribution", LOCTEXT("DistributionHeader", "Distribution"));
							InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesHorizontally);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().DistributeNodesVertically);
						}

						{
							FToolMenuSection& InSection = AlignmentMenu->AddSection("EdGraphSchemaStack", LOCTEXT("StackHeader", "Stack"));
							InSection.AddMenuEntry(FGraphEditorCommands::Get().StackNodesHorizontally);
							InSection.AddMenuEntry(FGraphEditorCommands::Get().StackNodesVertically);						
						}
					}));
				}
			}

			if (const UK2Node* K2Node = Cast<const UK2Node>(InGraphNode))
			{
				if (!K2Node->IsNodePure())
				{
					if (!bIsDebugging && GetDefault<UBlueprintEditorSettings>()->bAllowExplicitImpureNodeDisabling)
					{
						// Don't expose the enabled state for disabled nodes that were not explicitly disabled by the user
						if (!K2Node->IsAutomaticallyPlacedGhostNode())
						{
							// Add compile options
							{
								FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaCompileOptions", LOCTEXT("CompileOptionsHeader", "Compile Options"));
								Section.AddMenuEntry(
									FGraphEditorCommands::Get().DisableNodes,
									LOCTEXT("DisableCompile", "Disable (Do Not Compile)"),
									LOCTEXT("DisableCompileToolTip", "Selected node(s) will not be compiled."));

								{
									const FUIAction* SubMenuUIAction = Menu->Context.GetActionForCommand(FGraphEditorCommands::Get().EnableNodes);
									if(ensure(SubMenuUIAction))
									{
										Section.AddSubMenu(
											"EnableCompileSubMenu",
											LOCTEXT("EnableCompileSubMenu", "Enable Compile"),
											LOCTEXT("EnableCompileSubMenuToolTip", "Options to enable selected node(s) for compile."),
											FNewToolMenuDelegate::CreateLambda([](UToolMenu* SubMenu)
											{
												FToolMenuSection& SubMenuSection = SubMenu->AddSection("Section");
												SubMenuSection.AddMenuEntry(
													FGraphEditorCommands::Get().EnableNodes_Always,
													LOCTEXT("EnableCompileAlways", "Always"),
													LOCTEXT("EnableCompileAlwaysToolTip", "Always compile selected node(s)."));
												SubMenuSection.AddMenuEntry(
													FGraphEditorCommands::Get().EnableNodes_DevelopmentOnly,
													LOCTEXT("EnableCompileDevelopmentOnly", "Development Only"),
													LOCTEXT("EnableCompileDevelopmentOnlyToolTip", "Compile selected node(s) for development only."));
											}),
											*SubMenuUIAction,
											FGraphEditorCommands::Get().EnableNodes->GetUserInterfaceType());
									}
								}
							}
						}
					}
					
					// Add breakpoint actions
					if (K2Node->CanPlaceBreakpoints())
					{
						FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaBreakpoints", LOCTEXT("BreakpointsHeader", "Breakpoints"));
						Section.AddMenuEntry( FGraphEditorCommands::Get().ToggleBreakpoint );
						Section.AddMenuEntry( FGraphEditorCommands::Get().AddBreakpoint );
						Section.AddMenuEntry( FGraphEditorCommands::Get().RemoveBreakpoint );
						Section.AddMenuEntry( FGraphEditorCommands::Get().EnableBreakpoint );
						Section.AddMenuEntry( FGraphEditorCommands::Get().DisableBreakpoint );
					}
				}
			}
		}
	}

	FToolMenuSection& Section = Menu->AddSection("EdGraphSchemaDocumentation", LOCTEXT("DocumentationHeader", "Documentation"));
	Section.AddMenuEntry(FGraphEditorCommands::Get().GoToDocumentation);

	Super::GetContextMenuActions(Menu, Context);
}


void UEdGraphSchema_K2::OnCreateNonExistentVariable( UK2Node_Variable* Variable,  UBlueprint* OwnerBlueprint)
{
	if (UEdGraphPin* Pin = Variable->FindPin(Variable->GetVarName()))
	{
		const FScopedTransaction Transaction( LOCTEXT("CreateMissingVariable", "Create Missing Variable"), UEdGraphSchemaImpl::ShouldActuallyTransact());

		if (FBlueprintEditorUtils::AddMemberVariable(OwnerBlueprint,Variable->GetVarName(), Pin->PinType))
		{
			FGuid Guid = FBlueprintEditorUtils::FindMemberVariableGuidByName(OwnerBlueprint, Variable->GetVarName());
			Variable->VariableReference.SetSelfMember( Variable->GetVarName(), Guid );
		}
	}	
}

void UEdGraphSchema_K2::OnCreateNonExistentLocalVariable( UK2Node_Variable* Variable,  UBlueprint* OwnerBlueprint)
{
	if (UEdGraphPin* Pin = Variable->FindPin(Variable->GetVarName()))
	{
		const FScopedTransaction Transaction( LOCTEXT("CreateMissingLocalVariable", "Create Missing Local Variable"), UEdGraphSchemaImpl::ShouldActuallyTransact());

		FName VarName = Variable->GetVarName();
		if (FBlueprintEditorUtils::AddLocalVariable(OwnerBlueprint, Variable->GetGraph(), VarName, Pin->PinType))
		{
			FGuid LocalVarGuid = FBlueprintEditorUtils::FindLocalVariableGuidByName(OwnerBlueprint, Variable->GetGraph(), VarName);
			if (LocalVarGuid.IsValid())
			{
				// Loop through every variable in the graph, check if the variable references are the same, and update them
				FMemberReference OldReference = Variable->VariableReference;
				TArray<UK2Node_Variable*> VariableNodeList;
				Variable->GetGraph()->GetNodesOfClass(VariableNodeList);
				for( UK2Node_Variable* VariableNode : VariableNodeList)
				{
					if (VariableNode->VariableReference.IsSameReference(OldReference))
					{
						VariableNode->VariableReference.SetLocalMember(VarName, FBlueprintEditorUtils::GetTopLevelGraph(Variable->GetGraph())->GetName(), LocalVarGuid);
						VariableNode->ReconstructNode();
					}
				}
			}
		}
	}	
}

void UEdGraphSchema_K2::OnReplaceVariableForVariableNode( UK2Node_Variable* Variable, UBlueprint* OwnerBlueprint, FName VariableName, bool bIsSelfMember)
{
	if(UEdGraphPin* Pin = Variable->FindPin(Variable->GetVarName()))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_ReplaceVariable", "Replace Variable"), UEdGraphSchemaImpl::ShouldActuallyTransact());
		Variable->Modify();
		Pin->Modify();

		if (bIsSelfMember)
		{
			FGuid Guid = FBlueprintEditorUtils::FindMemberVariableGuidByName(OwnerBlueprint, VariableName);
			Variable->VariableReference.SetSelfMember( VariableName, Guid );
		}
		else
		{
			UEdGraph* FunctionGraph = FBlueprintEditorUtils::GetTopLevelGraph(Variable->GetGraph());
			Variable->VariableReference.SetLocalMember( VariableName, FunctionGraph->GetName(), FBlueprintEditorUtils::FindLocalVariableGuidByName(OwnerBlueprint, FunctionGraph, VariableName));
		}
		Pin->PinName = VariableName;
		Variable->ReconstructNode();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(OwnerBlueprint);
	}
}

void UEdGraphSchema_K2::GetReplaceVariableMenu(UToolMenu* Menu, UK2Node_Variable* Variable,  UBlueprint* OwnerBlueprint, bool bReplaceExistingVariable/*=false*/)
{
	if (UEdGraphPin* Pin = Variable->FindPin(Variable->GetVarName()))
	{
		FName ExistingVariableName = bReplaceExistingVariable? Variable->GetVarName() : NAME_None;

		FText ReplaceVariableWithTooltipFormat;
		if(!bReplaceExistingVariable)
		{
			ReplaceVariableWithTooltipFormat = LOCTEXT("ReplaceNonExistantVarToolTip", "Variable '{OldVariable}' does not exist, replace with matching variable '{AlternateVariable}'?");
		}
		else
		{
			ReplaceVariableWithTooltipFormat = LOCTEXT("ReplaceExistantVarToolTip", "Replace Variable '{OldVariable}' with matching variable '{AlternateVariable}'?");
		}

		TArray<FName> Variables;
		FBlueprintEditorUtils::GetNewVariablesOfType(OwnerBlueprint, Pin->PinType, Variables);

		{
			FToolMenuSection& Section = Menu->AddSection(NAME_None, LOCTEXT("Variables", "Variables"));
			for (TArray<FName>::TIterator VarIt(Variables); VarIt; ++VarIt)
			{
				if (*VarIt != ExistingVariableName)
				{
					const FText AlternativeVar = FText::FromName(*VarIt);

					FFormatNamedArguments TooltipArgs;
					TooltipArgs.Add(TEXT("OldVariable"), Variable->GetVarNameText());
					TooltipArgs.Add(TEXT("AlternateVariable"), AlternativeVar);
					const FText Desc = FText::Format(ReplaceVariableWithTooltipFormat, TooltipArgs);

					Section.AddMenuEntry(NAME_None, AlternativeVar, Desc, FSlateIcon(), FUIAction(
						FExecuteAction::CreateStatic(&UEdGraphSchema_K2::OnReplaceVariableForVariableNode, const_cast<UK2Node_Variable*>(Variable), OwnerBlueprint, *VarIt, /*bIsSelfMember=*/true)));
				}
			}
		}

		FText ReplaceLocalVariableWithTooltipFormat;
		if(!bReplaceExistingVariable)
		{
			ReplaceLocalVariableWithTooltipFormat = LOCTEXT("ReplaceNonExistantLocalVarToolTip", "Variable '{OldVariable}' does not exist, replace with matching local variable '{AlternateVariable}'?");
		}
		else
		{
			ReplaceLocalVariableWithTooltipFormat = LOCTEXT("ReplaceExistantLocalVarToolTip", "Replace Variable '{OldVariable}' with matching local variable '{AlternateVariable}'?");
		}

		TArray<FName> LocalVariables;
		FBlueprintEditorUtils::GetLocalVariablesOfType(Variable->GetGraph(), Pin->PinType, LocalVariables);

		{
			FToolMenuSection& Section = Menu->AddSection(NAME_None, LOCTEXT("LocalVariables", "LocalVariables"));
			for (TArray<FName>::TIterator VarIt(LocalVariables); VarIt; ++VarIt)
			{
				if (*VarIt != ExistingVariableName)
				{
					const FText AlternativeVar = FText::FromName(*VarIt);

					FFormatNamedArguments TooltipArgs;
					TooltipArgs.Add(TEXT("OldVariable"), Variable->GetVarNameText());
					TooltipArgs.Add(TEXT("AlternateVariable"), AlternativeVar);
					const FText Desc = FText::Format(ReplaceLocalVariableWithTooltipFormat, TooltipArgs);

					Section.AddMenuEntry(NAME_None, AlternativeVar, Desc, FSlateIcon(), FUIAction(
						FExecuteAction::CreateStatic(&UEdGraphSchema_K2::OnReplaceVariableForVariableNode, const_cast<UK2Node_Variable*>(Variable), OwnerBlueprint, *VarIt, /*bIsSelfMember=*/false)));
				}
			}
		}
	}
}

void UEdGraphSchema_K2::GetNonExistentVariableMenu(FToolMenuSection& Section, const UEdGraphNode* InGraphNode, UBlueprint* OwnerBlueprint) const
{

	if (const UK2Node_Variable* Variable = Cast<const UK2Node_Variable>(InGraphNode))
	{
		// Creating missing variables should never occur in a Macro Library or Interface, they do not support variables
		if(OwnerBlueprint->BlueprintType != BPTYPE_MacroLibrary && OwnerBlueprint->BlueprintType != BPTYPE_Interface )
		{		
			// Creating missing member variables should never occur in a Function Library, they do not support variables
			if(OwnerBlueprint->BlueprintType != BPTYPE_FunctionLibrary)
			{
				// create missing variable
				const FText Label = FText::Format( LOCTEXT("CreateNonExistentVar", "Create variable '{0}'"), Variable->GetVarNameText());
				const FText Desc = FText::Format( LOCTEXT("CreateNonExistentVarToolTip", "Variable '{0}' does not exist, create it?"), Variable->GetVarNameText());
				Section.AddMenuEntry("CreateNonExistentVar", Label, Desc, FSlateIcon(), FUIAction(
					FExecuteAction::CreateStatic( &UEdGraphSchema_K2::OnCreateNonExistentVariable, const_cast<UK2Node_Variable* >(Variable),OwnerBlueprint) ) );
			}

			// Only allow creating missing local variables if in a function graph
			if(InGraphNode->GetGraph()->GetSchema()->GetGraphType(InGraphNode->GetGraph()) == GT_Function)
			{
				const FText Label = FText::Format( LOCTEXT("CreateNonExistentLocalVar", "Create local variable '{0}'"), Variable->GetVarNameText());
				const FText Desc = FText::Format( LOCTEXT("CreateNonExistentLocalVarToolTip", "Local variable '{0}' does not exist, create it?"), Variable->GetVarNameText());
				Section.AddMenuEntry("CreateNonExistentLocalVar", Label, Desc, FSlateIcon(), FUIAction(
					FExecuteAction::CreateStatic( &UEdGraphSchema_K2::OnCreateNonExistentLocalVariable, const_cast<UK2Node_Variable* >(Variable),OwnerBlueprint) ) );
			}
		}

		// delete this node
		{			
			const FText Desc = FText::Format( LOCTEXT("DeleteNonExistentVarToolTip", "Referenced variable '{0}' does not exist, delete this node?"), Variable->GetVarNameText());
			Section.AddMenuEntry("DeleteNonExistentVar", FGenericCommands::Get().Delete, FGenericCommands::Get().Delete->GetLabel(), Desc, FSlateIcon());
		}

		GetReplaceVariableMenu(Section, InGraphNode, OwnerBlueprint);
	}
}

void UEdGraphSchema_K2::GetReplaceVariableMenu(FToolMenuSection& Section, const UEdGraphNode* InGraphNode, UBlueprint* InOwnerBlueprint, bool bInReplaceExistingVariable/* = false*/) const
{
	if (const UK2Node_Variable* Variable = Cast<const UK2Node_Variable>(InGraphNode))
	{
		// replace with matching variables
		if (UEdGraphPin* Pin = Variable->FindPin(Variable->GetVarName()))
		{
			FName ExistingVariableName = bInReplaceExistingVariable? Variable->GetVarName() : NAME_None;

			TArray<FName> Variables;
			FBlueprintEditorUtils::GetNewVariablesOfType(InOwnerBlueprint, Pin->PinType, Variables);
			Variables.RemoveSwap(ExistingVariableName);

			TArray<FName> LocalVariables;
			FBlueprintEditorUtils::GetLocalVariablesOfType(Variable->GetGraph(), Pin->PinType, LocalVariables);
			LocalVariables.RemoveSwap(ExistingVariableName);

			if (Variables.Num() > 0 || LocalVariables.Num() > 0)
			{
				FText ReplaceVariableWithTooltip;
				if(bInReplaceExistingVariable)
				{
					ReplaceVariableWithTooltip = LOCTEXT("ReplaceVariableWithToolTip", "Replace Variable '{0}' with another variable?");
				}
				else
				{
					ReplaceVariableWithTooltip = LOCTEXT("ReplaceMissingVariableWithToolTip", "Variable '{0}' does not exist, replace with another variable?");
				}

				Section.AddSubMenu(
					"ReplaceVariableWith",
					FText::Format( LOCTEXT("ReplaceVariableWith", "Replace variable '{0}' with..."), Variable->GetVarNameText()),
					FText::Format( ReplaceVariableWithTooltip, Variable->GetVarNameText()),
					FNewToolMenuDelegate::CreateStatic( &UEdGraphSchema_K2::GetReplaceVariableMenu,
						const_cast<UK2Node_Variable*>(Variable), InOwnerBlueprint, bInReplaceExistingVariable));
			}
		}
	}
}

const FPinConnectionResponse UEdGraphSchema_K2::DetermineConnectionResponseOfCompatibleTypedPins(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UEdGraphPin* InputPin, const UEdGraphPin* OutputPin) const
{
	// Now check to see if there are already connections and this is an 'exclusive' connection
	const bool bBreakExistingDueToExecOutput = IsExecPin(*OutputPin) && (OutputPin->LinkedTo.Num() > 0);
	const bool bBreakExistingDueToDataInput = !IsExecPin(*InputPin) && (InputPin->LinkedTo.Num() > 0);

	bool bMultipleSelfException = false;
	const UK2Node* OwningNode = Cast<UK2Node>(InputPin->GetOwningNode());
	if (bBreakExistingDueToDataInput && 
		IsSelfPin(*InputPin) && 
		OwningNode &&
		OwningNode->AllowMultipleSelfs(false) &&
		!InputPin->PinType.IsContainer() &&
		!OutputPin->PinType.IsContainer() )
	{
		//check if the node wont be expanded as foreach call, if there is a link to an array
		bool bAnyArrayInput = false;
		for(int InputLinkIndex = 0; InputLinkIndex < InputPin->LinkedTo.Num(); InputLinkIndex++)
		{
			if(const UEdGraphPin* Pin = InputPin->LinkedTo[InputLinkIndex])
			{
				if(Pin->PinType.IsArray())
				{
					bAnyArrayInput = true;
					break;
				}
			}
		}
		bMultipleSelfException = !bAnyArrayInput;
	}

	if (bBreakExistingDueToExecOutput)
	{
		const ECanCreateConnectionResponse ReplyBreakOutputs = (PinA == OutputPin) ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B;
		return FPinConnectionResponse(ReplyBreakOutputs, TEXT("Replace existing output connections"));
	}
	else if (bBreakExistingDueToDataInput && !bMultipleSelfException)
	{
		const ECanCreateConnectionResponse ReplyBreakInputs = (PinA == InputPin) ? CONNECT_RESPONSE_BREAK_OTHERS_A : CONNECT_RESPONSE_BREAK_OTHERS_B;
		return FPinConnectionResponse(ReplyBreakInputs, TEXT("Replace existing input connections"));
	}
	else
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_MAKE, TEXT(""));
	}
}

static FText GetPinIncompatibilityReason(const UEdGraphPin* PinA, const UEdGraphPin* PinB, bool* bIsFatalOut = nullptr)
{
	const FEdGraphPinType& PinAType = PinA->PinType;
	const FEdGraphPinType& PinBType = PinB->PinType;

	FFormatNamedArguments MessageArgs;
	MessageArgs.Add(TEXT("PinAName"), PinA->GetDisplayName());
	MessageArgs.Add(TEXT("PinBName"), PinB->GetDisplayName());
	MessageArgs.Add(TEXT("PinAType"), UEdGraphSchema_K2::TypeToText(PinAType));
	MessageArgs.Add(TEXT("PinBType"), UEdGraphSchema_K2::TypeToText(PinBType));

	const UEdGraphPin* InputPin = (PinA->Direction == EGPD_Input) ? PinA : PinB;
	const FEdGraphPinType& InputType  = InputPin->PinType;
	const UEdGraphPin* OutputPin = (InputPin == PinA) ? PinB : PinA;
	const FEdGraphPinType& OutputType = OutputPin->PinType;

	FText MessageFormat = LOCTEXT("DefaultPinIncompatibilityMessage", "{PinAType} is not compatible with {PinBType}.");

	if (bIsFatalOut != nullptr)
	{
		// the incompatible pins should generate an error by default
		*bIsFatalOut = true;
	}

	if (OutputType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if (InputType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			MessageFormat = LOCTEXT("StructsIncompatible", "Only exactly matching structures are considered compatible.");

			const UStruct* OutStruct = Cast<const UStruct>(OutputType.PinSubCategoryObject.Get());
			const UStruct* InStruct  = Cast<const UStruct>(InputType.PinSubCategoryObject.Get());
			if ((OutStruct != nullptr) && (InStruct != nullptr) && OutStruct->IsChildOf(InStruct))
			{
				MessageFormat = LOCTEXT("ChildStructIncompatible", "Only exactly matching structures are considered compatible. Derived structures are disallowed.");
			}
		}
	}
	else if (OutputType.PinCategory == UEdGraphSchema_K2::PC_Class)
	{
		if ((InputType.PinCategory == UEdGraphSchema_K2::PC_Object) || 
			(InputType.PinCategory == UEdGraphSchema_K2::PC_Interface))
		{
			MessageArgs.Add(TEXT("OutputName"), OutputPin->GetDisplayName());
			MessageArgs.Add(TEXT("InputName"),  InputPin->GetDisplayName());
			MessageFormat = LOCTEXT("ClassObjectIncompatible", "'{PinAName}' and '{PinBName}' are incompatible ('{OutputName}' is an object type, and '{InputName}' is a reference to an object instance).");

			if ((InputType.PinCategory == UEdGraphSchema_K2::PC_Object) && (bIsFatalOut != nullptr))
			{
				// under the hood class is an object, so it's not fatal
				*bIsFatalOut = false;
			}
		}
	}
	else if ((OutputType.PinCategory == UEdGraphSchema_K2::PC_Object) )//|| (OutputType.PinCategory == UEdGraphSchema_K2::PC_Interface))
	{
		if (InputType.PinCategory == UEdGraphSchema_K2::PC_Class)
		{
			MessageArgs.Add(TEXT("OutputName"), OutputPin->GetDisplayName());
			MessageArgs.Add(TEXT("InputName"),  InputPin->GetDisplayName());
			MessageArgs.Add(TEXT("InputType"),  UEdGraphSchema_K2::TypeToText(InputType));

			MessageFormat = LOCTEXT("CannotGetClass", "'{PinAName}' and '{PinBName}' are not inherently compatible ('{InputName}' is an object type, and '{OutputName}' is a reference to an object instance).\nWe cannot use {OutputName}'s class because it is not a child of {InputType}.");
		}
		else if (InputType.PinCategory == UEdGraphSchema_K2::PC_Object)
		{
			if (bIsFatalOut != nullptr)
			{
				*bIsFatalOut = true;
			}
		}
	}

	return FText::Format(MessageFormat, MessageArgs);
}

const FPinConnectionResponse UEdGraphSchema_K2::CanCreateConnection(const UEdGraphPin* PinA, const UEdGraphPin* PinB) const
{
	check(PinA);
	check(PinB);

	const UK2Node* OwningNodeA = Cast<UK2Node>(PinA->GetOwningNodeUnchecked());
	const UK2Node* OwningNodeB = Cast<UK2Node>(PinB->GetOwningNodeUnchecked());

	if (!OwningNodeA || !OwningNodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Invalid nodes"));
	}

	// Make sure the pins are not on the same node
	if (OwningNodeA == OwningNodeB)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Both are on the same node"));
	}

	if (PinA->bOrphanedPin || PinB->bOrphanedPin)
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Cannot make new connections to orphaned pin"));
	}

	FString NodeResponseMessage;
	// node can disallow the connection
	{
		if(OwningNodeA && OwningNodeA->IsConnectionDisallowed(PinA, PinB, NodeResponseMessage))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NodeResponseMessage);
		}
		if(OwningNodeB && OwningNodeB->IsConnectionDisallowed(PinB, PinA, NodeResponseMessage))
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, NodeResponseMessage);
		}
	}

	// Compare the directions
	const UEdGraphPin* InputPin = nullptr;
	const UEdGraphPin* OutputPin = nullptr;

	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, TEXT("Directions are not compatible"));
	}

	check(InputPin);
	check(OutputPin);

	bool bIgnoreArray = false;
	if (const UK2Node* OwningNode = Cast<UK2Node>(InputPin->GetOwningNode()))
	{
		const bool bAllowMultipleSelfs = OwningNode->AllowMultipleSelfs(true); // it applies also to ForEachCall
		const bool bNotAContainer = !InputPin->PinType.IsContainer();
		const bool bSelfPin = IsSelfPin(*InputPin);
		if (bAllowMultipleSelfs && bNotAContainer && bSelfPin)
		{
			// Indicates whether or not we will allow an array to be connected to a non-array input. This applies to nodes that support a foreach expansion of array inputs.
			bIgnoreArray = OutputPin->PinType.IsArray();
		}
	}

	// Find the calling context in case one of the pins is of type object and has a value of Self
	UClass* CallingContext = nullptr;
	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(PinA->GetOwningNodeUnchecked());
	if (Blueprint)
	{
		CallingContext = (Blueprint->GeneratedClass != NULL) ? Blueprint->GeneratedClass : Blueprint->ParentClass;
	}

	// Compare the types
	const bool bTypesMatch = ArePinsCompatible(OutputPin, InputPin, CallingContext, bIgnoreArray);

	if (bTypesMatch)
	{
		FPinConnectionResponse ConnectionResponse = DetermineConnectionResponseOfCompatibleTypedPins(PinA, PinB, InputPin, OutputPin);
		if (ConnectionResponse.Message.IsEmpty())
		{
			ConnectionResponse.Message = FText::FromString(NodeResponseMessage);
		}
		else if (!NodeResponseMessage.IsEmpty())
		{
			ConnectionResponse.Message = FText::Format(LOCTEXT("MultiMsgConnectionResponse", "{0} - {1}"), ConnectionResponse.Message, FText::FromString(NodeResponseMessage));
		}
		return ConnectionResponse;
	}
	else
	{
		// Promotable types in blueprints! Only if the Cvar is set and the node is of a special type. Eventually we want this for all
		if (TypePromoDebug::IsTypePromoEnabled() && InputPin->GetOwningNode()->IsA<UK2Node_PromotableOperator>())
		{
			if (FTypePromotion::IsValidPromotion(InputPin->PinType, OutputPin->PinType) || FTypePromotion::HasStructConversion(InputPin, OutputPin))
			{
				// Set the Text here correctly based on which pin type is higher
				return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_PROMOTION, FString::Printf(TEXT("Promote %s to %s"), *TypeToText(InputPin->PinType).ToString(), *TypeToText(OutputPin->PinType).ToString()));
			}
		}

		// Autocasting
		const bool bCanAutocast = SearchForAutocastFunction(OutputPin->PinType, InputPin->PinType).IsSet();
		const bool bCanAutoConvert = FindSpecializedConversionNode(OutputPin->PinType, *InputPin, false).IsSet();

		if (bCanAutocast || bCanAutoConvert)
		{
			return FPinConnectionResponse(CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE, FString::Printf(TEXT("Convert %s to %s"), *TypeToText(OutputPin->PinType).ToString(), *TypeToText(InputPin->PinType).ToString()));
		}
		else
		{
			bool bIsFatal = true;
			FText IncompatibilityReasonText = GetPinIncompatibilityReason(PinA, PinB, &bIsFatal);

			FPinConnectionResponse ConnectionResponse(CONNECT_RESPONSE_DISALLOW, IncompatibilityReasonText.ToString());
			if (bIsFatal)
			{
				ConnectionResponse.SetFatal();
			}
			return ConnectionResponse;
		}
	}
}

bool UEdGraphSchema_K2::TryCreateConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(PinA->GetOwningNode());

	bool bModified = UEdGraphSchema::TryCreateConnection(PinA, PinB);

	if (bModified && !PinA->IsPendingKill())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	return bModified;
}

struct FAutocastFunctionMap : private FNoncopyable
{
private:
	static FAutocastFunctionMap* AutocastFunctionMap;

	TMap<FString, TWeakObjectPtr<UFunction>> InnerMap;
	FDelegateHandle OnReloadCompleteDelegateHandle;
	FDelegateHandle OnModulesChangedDelegateHandle;

	static FString GenerateTypeData(const FEdGraphPinType& PinType)
	{
		UObject* Obj = PinType.PinSubCategoryObject.Get();
		FString PinSubCategory = PinType.PinSubCategory.ToString();
		if (PinSubCategory.StartsWith(UEdGraphSchema_K2::PSC_Bitmask.ToString()))
		{
			// Exclude the bitmask subcategory string from integral types so that autocast will work.
			PinSubCategory.Reset();
		}
		
		FString TypeString = FString::Printf(TEXT("%s;%s;%s;%d"), *PinType.PinCategory.ToString(), *PinSubCategory, Obj ? *Obj->GetPathName() : TEXT(""), (int32)PinType.ContainerType);

		if (PinType.ContainerType == EPinContainerType::Map)
		{
			// Add value type to string
			Obj = PinType.PinValueType.TerminalSubCategoryObject.Get();
			PinSubCategory = PinType.PinValueType.TerminalSubCategory.ToString();
			if (PinSubCategory.StartsWith(UEdGraphSchema_K2::PSC_Bitmask.ToString()))
			{
				PinSubCategory.Reset();
			}
			return FString::Printf(TEXT("%s;%s;%s;%s"), *TypeString, *PinType.PinValueType.TerminalCategory.ToString(), *PinSubCategory, Obj ? *Obj->GetPathName() : TEXT(""));
		}

		return TypeString;
	}

	static FString GenerateCastData(const FEdGraphPinType& InputPinType, const FEdGraphPinType& OutputPinType)
	{
		return FString::Printf(TEXT("%s;%s"), *GenerateTypeData(InputPinType), *GenerateTypeData(OutputPinType));
	}

	static bool IsInputParam(uint64 PropertyFlags)
	{
		const uint64 ConstOutParamFlag = CPF_OutParm | CPF_ConstParm;
		const uint64 IsConstOut = PropertyFlags & ConstOutParamFlag;
		return (CPF_Parm == (PropertyFlags & (CPF_Parm | CPF_ReturnParm)))
			&& ((0 == IsConstOut) || (ConstOutParamFlag == IsConstOut));
	}

	static const FProperty* GetFirstInputProperty(const UFunction* Function)
	{
		for (const FProperty* Property : TFieldRange<const FProperty>(Function))
		{
			if (Property && IsInputParam(Property->PropertyFlags))
			{
				return Property;
			}
		}
		return nullptr;
	}

	void InsertFunction(UFunction* Function, const UEdGraphSchema_K2* Schema)
	{
		FEdGraphPinType InputPinType;
		Schema->ConvertPropertyToPinType(GetFirstInputProperty(Function), InputPinType);

		FEdGraphPinType OutputPinType;
		Schema->ConvertPropertyToPinType(Function->GetReturnProperty(), OutputPinType);

		// If the output pin is an object pin, iterate through all possible super classes to add them as viable auto cast functions
		UStruct* StructObject = Cast<UStruct>(OutputPinType.PinSubCategoryObject.Get());
		const bool bIterateHierarchy = OutputPinType.PinCategory == UEdGraphSchema_K2::PC_Object && StructObject;
		if (bIterateHierarchy)
		{
			FEdGraphPinType OutputPinTypeCopy = OutputPinType;
			for (UStruct* OutputPinObject = StructObject; OutputPinObject != nullptr; OutputPinObject = OutputPinObject->GetSuperStruct())
			{
				OutputPinTypeCopy.PinSubCategoryObject = OutputPinObject;
				InnerMap.Add(GenerateCastData(InputPinType, OutputPinTypeCopy), Function);
			}
		}
		else
		{
			InnerMap.Add(GenerateCastData(InputPinType, OutputPinType), Function);
		}
	}
public:

	static bool IsAutocastFunction(const UFunction* Function)
	{
		const FName BlueprintAutocast(TEXT("BlueprintAutocast"));
		return Function
			&& Function->HasMetaData(BlueprintAutocast)
			&& Function->HasAllFunctionFlags(FUNC_Static | FUNC_Native | FUNC_Public | FUNC_BlueprintPure)
			&& Function->GetReturnProperty()
			&& GetFirstInputProperty(Function);
	}

	void Refresh()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WILD_FAutocastFunctionMap::Refresh);

#ifdef SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME
		static_assert(false, "Macro redefinition.");
#endif
#define SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME 0
#if SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME
		const double StartTime = FPlatformTime::Seconds();
#endif //SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME

		InnerMap.Empty();

		TArray<UClass*> Libraries;
		GetDerivedClasses(UBlueprintFunctionLibrary::StaticClass(), Libraries);
		AddLibraries(Libraries);

#if SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME
		const double EndTime = FPlatformTime::Seconds();
		UE_LOGF(LogBlueprint, Warning, "FAutocastFunctionMap::Refresh took %fs", EndTime - StartTime);
#endif //SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME
#undef SCHEMA_K2_AUTOCASTFUNCTIONMAP_LOG_TIME
	}

	void AddLibrariesFromModule(FName ModuleThatChanged)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WILD_FAutocastFunctionMap::AddLibrariesFromModule);

		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		if (UPackage* ModuleScriptPacakge = FindPackage(nullptr, *FString::Printf(TEXT("/Script/%s"), *ModuleThatChanged.ToString())))
		{
			TArray<UClass*> Libraries;
			ForEachObjectWithPackage(ModuleScriptPacakge, [&Libraries](UObject* Obj) -> bool
				{
					if (UClass* Class = Cast<UClass>(Obj))
					{
						if (Class->IsChildOf(UBlueprintFunctionLibrary::StaticClass()))
						{
							Libraries.Add(Class);
						}
					}

					return true;
				}, EGetObjectsFlags::None);

			AddLibraries(Libraries);
		}
	}

	void AddLibraries(const TArray<UClass*>& Libraries)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		for (UClass* Library : Libraries)
		{
			if (Library && (CLASS_Native == (Library->ClassFlags & (CLASS_Native | CLASS_Deprecated | CLASS_NewerVersionExists))))
			{
				for (UFunction* Function : TFieldRange<UFunction>(Library, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated))
				{
					if (IsAutocastFunction(Function))
					{
						InsertFunction(Function, Schema);
					}
				}
			}
		}
	}

	UFunction* Find(const FEdGraphPinType& InputPinType, const FEdGraphPinType& OutputPinType) const
	{
		// If the input pin is an object pin, iterate through all possible super classes to check for auto cast availability
		UStruct* StructObject = Cast<UStruct>(InputPinType.PinSubCategoryObject.Get());
		const bool bIterateHierarchy = InputPinType.PinCategory == UEdGraphSchema_K2::PC_Object && StructObject;
		if (bIterateHierarchy)
		{
			FEdGraphPinType InputPinTypeCopy = InputPinType;
			for (UStruct* InputPinObject = StructObject; InputPinObject != nullptr; InputPinObject = InputPinObject->GetSuperStruct())
			{
				InputPinTypeCopy.PinSubCategoryObject = InputPinObject;
				
				const TWeakObjectPtr<UFunction>* FuncPtr = InnerMap.Find(GenerateCastData(InputPinTypeCopy, OutputPinType));
				if (FuncPtr)
				{
					return FuncPtr->Get();
				}
			}
			return nullptr;
		}
		
		const TWeakObjectPtr<UFunction>* FuncPtr = InnerMap.Find(GenerateCastData(InputPinType, OutputPinType));
		return FuncPtr ? FuncPtr->Get() : nullptr;
	}

	static FAutocastFunctionMap& Get()
	{
		if (AutocastFunctionMap == nullptr)
		{
			AutocastFunctionMap = new FAutocastFunctionMap();
		}
		return *AutocastFunctionMap;
	}

	static void Shutdown()
	{
		delete AutocastFunctionMap;
		AutocastFunctionMap = nullptr;
	}

	static void OnReloadComplete(EReloadCompleteReason Reaosn)
	{
		if (AutocastFunctionMap)
		{
			AutocastFunctionMap->Refresh();
		}
	}

	static void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
	{
		if (AutocastFunctionMap)
		{
			if (ReasonForChange == EModuleChangeReason::ModuleLoaded)
			{
				AutocastFunctionMap->AddLibrariesFromModule(ModuleThatChanged);
			}
			else if (ReasonForChange == EModuleChangeReason::ModuleUnloaded)
			{
				AutocastFunctionMap->Refresh();
			}
		}
	}

	void RegisterDelegates()
	{
		OnReloadCompleteDelegateHandle = FCoreUObjectDelegates::ReloadCompleteDelegate.AddStatic(&FAutocastFunctionMap::OnReloadComplete);
		OnModulesChangedDelegateHandle = FModuleManager::Get().OnModulesChanged().AddStatic(&OnModulesChanged);
		FCoreDelegates::OnEnginePreExit.AddLambda([]() { FAutocastFunctionMap::Get().UnregisterDelegates(); });
	}

	void UnregisterDelegates()
	{
		FCoreUObjectDelegates::ReloadCompleteDelegate.Remove(OnReloadCompleteDelegateHandle);
		FModuleManager::Get().OnModulesChanged().Remove(OnModulesChangedDelegateHandle);
	}

	FAutocastFunctionMap()
	{
		Refresh();
		RegisterDelegates();
	}

	~FAutocastFunctionMap()
	{
		UnregisterDelegates();
	}
};

FAutocastFunctionMap* FAutocastFunctionMap::AutocastFunctionMap = nullptr;

void UEdGraphSchema_K2::Shutdown()
{
	FAutocastFunctionMap::Shutdown();
}

TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> UEdGraphSchema_K2::SearchForAutocastFunction(const FEdGraphPinType& OutputPinType, const FEdGraphPinType& InputPinType) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WILD_UEdGraphSchema_K2::SearchForAutocastFunction);

	// NOTE: Under no circumstances should anyone *ever* add a questionable cast to this function.
	// If it could be at all confusing why a function is provided, to even a novice user, err on the side of do not cast!!!
	// This includes things like string->int (does it do length, atoi, or what?) that would be autocasts in a traditional scripting language

	TOptional<FSearchForAutocastFunctionResults> Result;

	if (OutputPinType.ContainerType != InputPinType.ContainerType)
	{
		if (OutputPinType.IsSet() && InputPinType.IsArray())
		{
			const UFunction* Function = UBlueprintSetLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UBlueprintSetLibrary, Set_ToArray));
			Result = { Function->GetFName(), Function->GetOwnerClass() };
		}

		// Skip the other special cases if container check fails, but allow checking the autocast map
	}
	else
	{
		// SPECIAL CASES, not supported by FAutocastFunctionMap.
		if ((OutputPinType.PinCategory == PC_Interface) && (InputPinType.PinCategory == PC_Object))
		{
			const UClass* InputClass = Cast<const UClass>(InputPinType.PinSubCategoryObject.Get());

			const bool bInputIsUObject = (InputClass && (InputClass == UObject::StaticClass()));
			if (bInputIsUObject)
			{
				UFunction* Function = UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, Conv_InterfaceToObject));
				Result = { Function->GetFName(), Function->GetOwnerClass() };
			}
		}
		else if (OutputPinType.PinCategory == PC_Object)
		{
			UClass const* OutputClass = Cast<UClass const>(OutputPinType.PinSubCategoryObject.Get());
			if (InputPinType.PinCategory == PC_Class)
			{
				UClass const* InputClass = Cast<UClass const>(InputPinType.PinSubCategoryObject.Get());
				if ((OutputClass != nullptr) &&
					(InputClass != nullptr) &&
					OutputClass->IsChildOf(InputClass))
				{
					UFunction* Function = UGameplayStatics::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UGameplayStatics, GetObjectClass));
					Result = { Function->GetFName(), Function->GetOwnerClass() };
				}
			}
			else if (InputPinType.PinCategory == PC_String)
			{
				UFunction* Function = UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, GetDisplayName));
				Result = { Function->GetFName(), Function->GetOwnerClass() };
			}
		}
		else if (OutputPinType.PinCategory == PC_Class)
		{
			if (InputPinType.PinCategory == PC_String)
			{
				UFunction* Function = UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, GetClassDisplayName));
				Result = { Function->GetFName(), Function->GetOwnerClass() };
			}
		}
		else if (OutputPinType.PinCategory == PC_Struct)
		{
			const UScriptStruct* OutputStructType = Cast<const UScriptStruct>(OutputPinType.PinSubCategoryObject.Get());
			if (OutputStructType == TBaseStructure<FRotator>::Get())
			{
				const UScriptStruct* InputStructType = Cast<const UScriptStruct>(InputPinType.PinSubCategoryObject.Get());
				if ((InputPinType.PinCategory == PC_Struct) && (InputStructType == TBaseStructure<FTransform>::Get()))
				{
					UFunction* Function = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetMathLibrary, MakeTransform));
					Result = { Function->GetFName(), Function->GetOwnerClass() };
				}
			}
		}
	}

	// Try looking for a marked up autocast if we've not found a built-in one that works
	if (!Result.IsSet())
	{
		auto FindAndSetCastFunction = [&Result](const FEdGraphPinType& OutputPinType, const FEdGraphPinType& InputPinType)
		{
			const FAutocastFunctionMap& AutocastFunctionMap = FAutocastFunctionMap::Get();
			if (const UFunction* Function = AutocastFunctionMap.Find(OutputPinType, InputPinType))
			{
				Result = { Function->GetFName(), Function->GetOwnerClass() };
				return true;
			}
			return false;
		};

		if (!FindAndSetCastFunction(OutputPinType, InputPinType))
		{
			// Since single-precision float interfaces have been deprecated,
			// we should try to find a double-precision equivalent.
			if (OutputPinType.PinSubCategory == PC_Float)
			{
				FEdGraphPinType PinTypeCopy(OutputPinType);
				PinTypeCopy.PinSubCategory = PC_Double;
				FindAndSetCastFunction(PinTypeCopy, InputPinType);
			}
			else if (InputPinType.PinSubCategory == PC_Float)
			{
				FEdGraphPinType PinTypeCopy(InputPinType);
				PinTypeCopy.PinSubCategory = PC_Double;
				FindAndSetCastFunction(OutputPinType, PinTypeCopy);
			}
		}
	}

	return Result;
}

TOptional<UEdGraphSchema_K2::FFindSpecializedConversionNodeResults> UEdGraphSchema_K2::FindSpecializedConversionNode(const FEdGraphPinType& OutputPinType, const UEdGraphPin& InputPin, bool bCreateNode) const
{
	TOptional<UEdGraphSchema_K2::FFindSpecializedConversionNodeResults> Result;
	FEdGraphPinType InputPinType = InputPin.PinType;

	const bool bConvertScalarToArray =
		!OutputPinType.IsContainer() &&
		InputPinType.IsArray() &&
		ArePinTypesCompatible(OutputPinType, InputPinType, nullptr, true);

	const bool bTryAlternateObjectProperty =
		InputPin.GetOwningNode()->IsA(UK2Node_CallFunction::StaticClass()) &&
		IsSelfPin(InputPin) &&
		((OutputPinType.PinCategory == PC_Object) || ((OutputPinType.PinCategory == PC_Interface) && !OutputPinType.IsContainer()));

	if (bConvertScalarToArray)
	{
		Result = { nullptr };
		if (bCreateNode)
		{
			Result->TargetNode = NewObject<UK2Node_MakeArray>();
		}
	}
	// If connecting an object to a 'call function' self pin, and not currently compatible, see if there is a property we can call a function on
	else if (bTryAlternateObjectProperty)
	{
		const UK2Node_CallFunction* CallFunctionNode = CastChecked<UK2Node_CallFunction>(InputPin.GetOwningNode());
		const UClass* OutputPinClass = Cast<UClass>(OutputPinType.PinSubCategoryObject.Get());
		const UClass* FunctionClass = CallFunctionNode->FunctionReference.GetMemberParentClass(CallFunctionNode->GetBlueprintClassFromNode());

		if(FunctionClass != nullptr && OutputPinClass != nullptr)
		{
			// Iterate over object properties..
			for (TFieldIterator<FObjectProperty> PropIt(OutputPinClass, EFieldIteratorFlags::IncludeSuper); PropIt; ++PropIt)
			{
				FObjectProperty* ObjProp = *PropIt;
				// .. if we have a blueprint visible var, and is of the type which contains this function..
				if(ObjProp->HasAllPropertyFlags(CPF_BlueprintVisible) && ObjProp->PropertyClass && ObjProp->PropertyClass->IsChildOf(FunctionClass))
				{
					Result = { nullptr };
					if (bCreateNode)
					{
						UK2Node_VariableGet* GetNode = NewObject<UK2Node_VariableGet>();
						GetNode->VariableReference.SetFromField<FProperty>(ObjProp, false);
						Result->TargetNode = GetNode;
					}
				}
			}

		}	
	}

	if (!Result.IsSet())
	{
		// CHECK ENUM TO NAME CAST
		const bool bInputMatch = !InputPin.PinType.IsContainer() && ((PC_Name == InputPin.PinType.PinCategory) || (PC_String == InputPin.PinType.PinCategory));
		const bool bOutputMatch = !OutputPinType.IsContainer() && (PC_Byte == OutputPinType.PinCategory) && (nullptr != Cast<UEnum>(OutputPinType.PinSubCategoryObject.Get()));
		if(bOutputMatch && bInputMatch)
		{
			Result = { nullptr };
			if (bCreateNode)
			{
				if(PC_Name == InputPin.PinType.PinCategory)
				{
					Result->TargetNode = NewObject<UK2Node_GetEnumeratorName>();
				}
				else if(PC_String == InputPin.PinType.PinCategory)
				{
					Result->TargetNode = NewObject<UK2Node_GetEnumeratorNameAsString>();
				}
			}
		}
	}

	if (!Result.IsSet())
	{
		// CHECK BYTE TO ENUM CAST
		UEnum* Enum = Cast<UEnum>(InputPinType.PinSubCategoryObject.Get());
		const bool bInputIsEnum = !InputPinType.IsContainer() && (PC_Byte == InputPinType.PinCategory) && Enum;
		const bool bOutputIsByte = !OutputPinType.IsContainer() && (PC_Byte == OutputPinType.PinCategory);
		if (bInputIsEnum && bOutputIsByte)
		{
			Result = { nullptr };
			if(bCreateNode)
			{
				UK2Node_CastByteToEnum* CastByteToEnum = NewObject<UK2Node_CastByteToEnum>();
				CastByteToEnum->Enum = Enum;
				CastByteToEnum->bSafe = true;
				Result->TargetNode = CastByteToEnum;
			}
		}
		else if (!OutputPinType.IsContainer())
		{
			// Note: Cast nodes do not support a ForEach-style expansion, so we exclude container types here.
		
			const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(InputPin.GetOwningNode());
			UClass* BlueprintClass = (Blueprint->GeneratedClass != nullptr) ? Blueprint->GeneratedClass : Blueprint->ParentClass;

			UClass* InputClass = Cast<UClass>(InputPin.PinType.PinSubCategoryObject.Get());

			if ((InputClass == nullptr) && (InputPin.PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self))
			{
				InputClass = BlueprintClass;
			}

			const UClass* OutputClass = Cast<UClass>(OutputPinType.PinSubCategoryObject.Get());

			if ((OutputClass == nullptr) && (OutputPinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self))
			{
				OutputClass = BlueprintClass;
			}

			bool bNeedsDynamicCast = false;
			if ((OutputPinType.PinCategory == PC_Interface) && (InputPinType.PinCategory == PC_Object))
			{
				bNeedsDynamicCast = (InputClass && OutputClass) && (InputClass->ImplementsInterface(OutputClass) || OutputClass->IsChildOf(InputClass));
			}
			else if (OutputPinType.PinCategory == PC_Object)
			{
				UBlueprintEditorSettings const* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
				if ((InputPinType.PinCategory == PC_Object) && BlueprintSettings->bAutoCastObjectConnections)
				{
					bNeedsDynamicCast = (InputClass && OutputClass) && InputClass->IsChildOf(OutputClass);
				}
			}

			if (bNeedsDynamicCast)
			{
				Result = { nullptr };
				if (bCreateNode)
				{
					UK2Node_DynamicCast* DynCastNode = NewObject<UK2Node_DynamicCast>();
					DynCastNode->TargetType = InputClass;
					DynCastNode->SetPurity(true);
					Result->TargetNode = DynCastNode;
				}
			}

			if (!bNeedsDynamicCast && InputClass && OutputClass && OutputClass->IsChildOf(InputClass))
			{
				const bool bConvertAsset = (OutputPinType.PinCategory == PC_SoftObject) && (InputPinType.PinCategory == PC_Object);
				const bool bConvertAssetClass = (OutputPinType.PinCategory == PC_SoftClass) && (InputPinType.PinCategory == PC_Class);
				const bool bConvertToAsset = (OutputPinType.PinCategory == PC_Object) && (InputPinType.PinCategory == PC_SoftObject);
				const bool bConvertToAssetClass = (OutputPinType.PinCategory == PC_Class) && (InputPinType.PinCategory == PC_SoftClass);

				if (bConvertAsset || bConvertAssetClass || bConvertToAsset || bConvertToAssetClass)
				{
					Result = { nullptr };
					if (bCreateNode)
					{
						UK2Node_ConvertAsset* ConvertAssetNode = NewObject<UK2Node_ConvertAsset>();
						Result->TargetNode = ConvertAssetNode;
					}
				}
			}
		}
	}

	return Result;
}

void UEdGraphSchema_K2::AutowireConversionNode(UEdGraphPin* InputPin, UEdGraphPin* OutputPin, UEdGraphNode* ConversionNode) const
{
	bool bAllowInputConnections = true;
	bool bAllowOutputConnections = true;

	for (int32 PinIndex = 0; PinIndex < ConversionNode->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* TestPin = ConversionNode->Pins[PinIndex];

		UClass* Context = nullptr;
		UK2Node* K2Node = Cast<UK2Node>(OutputPin->GetOwningNode());
		if (K2Node != nullptr)
		{
			UBlueprint* Blueprint = K2Node->GetBlueprint();
			if (Blueprint)
			{
				Context = Blueprint->GeneratedClass;
			}
		}

		if ((TestPin->Direction == EGPD_Input) && (ArePinTypesCompatible(OutputPin->PinType, TestPin->PinType, Context)))
		{
			if(bAllowOutputConnections && TryCreateConnection(TestPin, OutputPin))
			{
				// Successful connection, do not allow more output connections
				bAllowOutputConnections = false;
			}
		}
		else if ((TestPin->Direction == EGPD_Output) && (ArePinTypesCompatible(TestPin->PinType, InputPin->PinType, Context)))
		{
			if(bAllowInputConnections && TryCreateConnection(TestPin, InputPin))
			{
				// Successful connection, do not allow more input connections
				bAllowInputConnections = false;
			}
		}
	}
}

bool UEdGraphSchema_K2::CreateAutomaticConversionNodeAndConnections(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	// Determine which pin is an input and which pin is an output
	UEdGraphPin* InputPin = nullptr;
	UEdGraphPin* OutputPin = nullptr;
	if (!CategorizePinsByDirection(PinA, PinB, /*out*/ InputPin, /*out*/ OutputPin))
	{
		return false;
	}

	check(InputPin);
	check(OutputPin);

	UK2Node* TemplateConversionNode = nullptr;

	if (TOptional<FSearchForAutocastFunctionResults> AutocastResult = SearchForAutocastFunction(OutputPin->PinType, InputPin->PinType))
	{
		// Create a new call function node for the casting operator
		UK2Node_CallFunction* TemplateNode = NewObject<UK2Node_CallFunction>();
		TemplateNode->FunctionReference.SetExternalMember(AutocastResult->TargetFunction, AutocastResult->FunctionOwner);

		TemplateConversionNode = TemplateNode;
	}
	else if (TOptional<FFindSpecializedConversionNodeResults> ConversionResult = FindSpecializedConversionNode(OutputPin->PinType, *InputPin, true))
	{
		TemplateConversionNode = ConversionResult->TargetNode;
	}

	if (TemplateConversionNode != nullptr)
	{
		// Determine where to position the new node (assuming it isn't going to get beaded)
		FVector2D AverageLocation = CalculateAveragePositionBetweenNodes(InputPin, OutputPin);

		UK2Node* ConversionNode = FEdGraphSchemaAction_K2NewNode::SpawnNodeFromTemplate<UK2Node>(InputPin->GetOwningNode()->GetGraph(), TemplateConversionNode, AverageLocation);

		// Connect the cast node up to the output/input pins
		AutowireConversionNode(InputPin, OutputPin, ConversionNode);

		return true;
	}

	return false;
}

bool UEdGraphSchema_K2::CreatePromotedConnection(UEdGraphPin* PinA, UEdGraphPin* PinB) const
{
	PinA->Modify();
	PinB->Modify();

	PinA->MakeLinkTo(PinB);

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(PinA->GetOwningNode());

	if (!PinA->IsPendingKill())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	return true;
}

FString UEdGraphSchema_K2::IsPinDefaultValid(const UEdGraphPin* Pin, const FString& NewDefaultValue, TObjectPtr<UObject> NewDefaultObject, const FText& InNewDefaultText) const
{
	check(Pin);

	FFormatNamedArguments MessageArgs;
	MessageArgs.Add(TEXT("PinName"), Pin->GetDisplayName());

	const UBlueprint* OwningBP = FBlueprintEditorUtils::FindBlueprintForNode(Pin->GetOwningNodeUnchecked());

	const bool bIsArray = Pin->PinType.IsArray();
	const bool bIsSet = Pin->PinType.IsSet();
	const bool bIsMap = Pin->PinType.IsMap();
	const bool bIsReference = Pin->PinType.bIsReference;
	const bool bIsAutoCreateRefTerm = IsAutoCreateRefTerm(Pin);

	if (OwningBP == nullptr || OwningBP->BlueprintType != BPTYPE_Interface)
	{
		if( !bIsAutoCreateRefTerm )
		{
			// No harm in leaving a function result node input (aka function output) unconnected - the property will be initialized correctly
			// as empty:
			bool bIsFunctionOutput = false;
			if(UK2Node_FunctionResult* Result = Cast<UK2Node_FunctionResult>(Pin->GetOwningNode()))
			{
				if(ensure(Pin->Direction == EEdGraphPinDirection::EGPD_Input))
				{
					bIsFunctionOutput = true;
				}
			}

			if(!bIsFunctionOutput)
			{
				if( bIsArray )
				{
					FText MsgFormat = LOCTEXT("BadArrayDefaultVal", "Array inputs (like '{PinName}') must have an input wired into them (try connecting a MakeArray node).");
					return FText::Format(MsgFormat, MessageArgs).ToString();
				}
				else if( bIsSet )
				{
					FText MsgFormat = LOCTEXT("BadSetDefaultVal", "Set inputs (like '{PinName}') must have an input wired into them (try connecting a MakeSet node).");
					return FText::Format(MsgFormat, MessageArgs).ToString();
				}
				else if ( bIsMap )
				{
					FText MsgFormat = LOCTEXT("BadMapDefaultVal", "Map inputs (like '{PinName}') must have an input wired into them (try connecting a MakeMap node).");
					return FText::Format(MsgFormat, MessageArgs).ToString();
				}
				else if( bIsReference )
				{
					FText MsgFormat = LOCTEXT("BadRefDefaultVal", "'{PinName}' in action '{ActionName}' must have an input wired into it (\"by ref\" params expect a valid input to operate on).");
					MessageArgs.Add(TEXT("ActionName"), Pin->GetOwningNode()->GetNodeTitle(ENodeTitleType::ListView));
					return FText::Format(MsgFormat, MessageArgs).ToString();
				}
			}
		}
	}

	FString ReturnMsg;
	DefaultValueSimpleValidation(Pin->PinType, Pin->PinName, NewDefaultValue, NewDefaultObject, InNewDefaultText, &ReturnMsg);
	return ReturnMsg;
}

bool UEdGraphSchema_K2::DoesSupportPinWatching() const
{
	return true;
}

bool UEdGraphSchema_K2::IsPinBeingWatched(UEdGraphPin const* Pin) const
{
	// Note: If you crash here; it is likely that you forgot to call Blueprint->OnBlueprintChanged.Broadcast(Blueprint) to invalidate the cached UI state
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Pin ? Pin->GetOwningNodeUnchecked() : nullptr);
	return (Blueprint ? FKismetDebugUtilities::IsPinBeingWatched(Blueprint, Pin) : false);
}

void UEdGraphSchema_K2::ClearPinWatch(UEdGraphPin const* Pin) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Pin->GetOwningNode());
	FKismetDebugUtilities::RemovePinWatch(Blueprint, Pin);
}

FLinearColor UEdGraphSchema_K2::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	const FName TypeName = PinType.PinCategory;
	const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();

	if (TypeName == PC_Exec)
	{
		return Settings->ExecutionPinTypeColor;
	}
	else if (TypeName == PC_Object || TypeName == PC_FieldPath)
	{
		return Settings->ObjectPinTypeColor;
	}
	else if (TypeName == PC_Interface)
	{
		return Settings->InterfacePinTypeColor;
	}
	else if (TypeName == PC_Real)
	{
		return Settings->RealPinTypeColor;
	}
	else if (TypeName == PC_Boolean)
	{
		return Settings->BooleanPinTypeColor;
	}
	else if (TypeName == PC_Byte || TypeName == PC_Enum)
	{
		return Settings->BytePinTypeColor;
	}
	else if (TypeName == PC_Int)
	{
		return Settings->IntPinTypeColor;
	}
	else if (TypeName == PC_Int64)
	{
		return Settings->Int64PinTypeColor;
	}
	else if (TypeName == PC_Struct)
	{
		if ((PinType.PinSubCategoryObject == VectorStruct) || (PinType.PinSubCategoryObject == Vector3fStruct))
		{
			// vector
			return Settings->VectorPinTypeColor;
		}
		else if (PinType.PinSubCategoryObject == RotatorStruct)
		{
			// rotator
			return Settings->RotatorPinTypeColor;
		}
		else if (PinType.PinSubCategoryObject == TransformStruct)
		{
			// transform
			return Settings->TransformPinTypeColor;
		}
		else
		{
			return Settings->StructPinTypeColor;
		}
	}
	else if (TypeName == PC_String)
	{
		return Settings->StringPinTypeColor;
	}
	else if (TypeName == PC_Text)
	{
		return Settings->TextPinTypeColor;
	}
	else if (TypeName == PC_Wildcard)
	{
		if (PinType.PinSubCategory == PSC_Index)
		{
			return Settings->IndexPinTypeColor;
		}
		else
		{
			return Settings->WildcardPinTypeColor;
		}
	}
	else if (TypeName == PC_Name)
	{
		return Settings->NamePinTypeColor;
	}
	else if (TypeName == PC_SoftObject)
	{
		return Settings->SoftObjectPinTypeColor;
	}
	else if (TypeName == PC_SoftClass)
	{
		return Settings->SoftClassPinTypeColor;
	}
	else if (TypeName == PC_Delegate)
	{
		return Settings->DelegatePinTypeColor;
	}
	else if (TypeName == PC_Class)
	{
		return Settings->ClassPinTypeColor;
	}

	// Type does not have a defined color!
	return Settings->DefaultPinTypeColor;
}

FLinearColor UEdGraphSchema_K2::GetSecondaryPinTypeColor(const FEdGraphPinType& PinType) const
{
	if (PinType.IsMap())
	{
		FEdGraphPinType FakePrimary = PinType;
		FakePrimary.PinCategory = FakePrimary.PinValueType.TerminalCategory;
		FakePrimary.PinSubCategory = FakePrimary.PinValueType.TerminalSubCategory;
		FakePrimary.PinSubCategoryObject = FakePrimary.PinValueType.TerminalSubCategoryObject;

		return GetPinTypeColor(FakePrimary);
	}
	else
	{
		const UGraphEditorSettings* Settings = GetDefault<UGraphEditorSettings>();
		return Settings->WildcardPinTypeColor;
	}
}

FText UEdGraphSchema_K2::GetPinDisplayName(const UEdGraphPin* Pin) const 
{
	FText DisplayName = FText::GetEmpty();

	if (Pin)
	{
		UEdGraphNode* Node = Pin->GetOwningNode();
		if (Node->ShouldOverridePinNames())
		{
			DisplayName = Node->GetPinNameOverride(*Pin);
		}
		else
		{
			DisplayName = Super::GetPinDisplayName(Pin);
	
			// bit of a hack to hide 'execute' and 'then' pin names
			if (Pin->PinType.PinCategory == PC_Exec)
			{
				FName DisplayFName(*DisplayName.ToString(), FNAME_Find);
				if ((DisplayFName == PN_Execute) || (DisplayFName == PN_Then))
				{
					DisplayName = FText::GetEmpty();
				}
			}
		}

		if( GEditor && GetDefault<UEditorStyleSettings>()->bShowFriendlyNames && Pin->bAllowFriendlyName )
		{
			DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(DisplayName.ToString(), Pin->PinType.PinCategory == PC_Boolean));
		}
	}
	return DisplayName;
}

void UEdGraphSchema_K2::ConstructBasicPinTooltip(const UEdGraphPin& Pin, const FText& PinDescription, FString& TooltipOut) const
{
	if (Pin.bWasTrashed)
	{
		return;
	}

	if (bGeneratingDocumentation)
	{
		TooltipOut = PinDescription.ToString();
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinType"), TypeToText(Pin.PinType));

		if (Pin.bDeprecated && !Pin.DeprecationMessage.IsEmpty())
		{
			Args.Add(TEXT("DeprecationWarning"), FText::FromString(Pin.DeprecationMessage));
			Args.Add(TEXT("LineFeed1"), FText::FromString(TEXT("\n")));
		}
		else if (Pin.bDeprecated)
		{
			Args.Add(TEXT("DeprecationWarning"), LOCTEXT("Deprecated", "Deprecated"));
			Args.Add(TEXT("LineFeed1"), FText::FromString(TEXT("\n")));
		}
		else
		{
			Args.Add(TEXT("DeprecationWarning"), FText::GetEmpty());
			Args.Add(TEXT("LineFeed1"), FText::GetEmpty());
		}

		if (UEdGraphNode* PinNode = Pin.GetOwningNode())
		{
			UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(PinNode->GetSchema());
			if (ensure(K2Schema != NULL)) // ensure that this node belongs to this schema
			{
				Args.Add(TEXT("DisplayName"), GetPinDisplayName(&Pin));
				Args.Add(TEXT("LineFeed2"), FText::FromString(TEXT("\n")));
			}
		}
		else
		{
				Args.Add(TEXT("DisplayName"), FText::GetEmpty());
				Args.Add(TEXT("LineFeed2"), FText::GetEmpty());
		}


		if (!PinDescription.IsEmpty())
		{
			Args.Add(TEXT("Description"), PinDescription);
			Args.Add(TEXT("LineFeed3"), FText::FromString(TEXT("\n\n")));
		}
		else
		{
			Args.Add(TEXT("Description"), FText::GetEmpty());
			Args.Add(TEXT("LineFeed3"), FText::GetEmpty());
		}
	
		TooltipOut = FText::Format(LOCTEXT("PinTooltip", "{DeprecationWarning}{LineFeed1}{DisplayName}{LineFeed2}{PinType}{LineFeed3}{Description}"), Args).ToString(); 
	}
}

EGraphType UEdGraphSchema_K2::GetGraphType(const UEdGraph* TestEdGraph) const
{
	if (TestEdGraph)
	{
		//@TODO: Should there be a GT_Subgraph type?	
		UEdGraph* GraphToTest = const_cast<UEdGraph*>(TestEdGraph);

		for (UObject* TestOuter = GraphToTest; TestOuter; TestOuter = TestOuter->GetOuter())
		{
			// reached up to the blueprint for the graph
			if (UBlueprint* Blueprint = Cast<UBlueprint>(TestOuter))
			{
				if (Blueprint->BlueprintType == BPTYPE_MacroLibrary ||
					Blueprint->MacroGraphs.Contains(GraphToTest))
				{
					return GT_Macro;
				}
				else if (Blueprint->UbergraphPages.Contains(GraphToTest))
				{
					return GT_Ubergraph;
				}
				else if (Blueprint->FunctionGraphs.Contains(GraphToTest))
				{
					return GT_Function; 
				}
			}
			else
			{
				GraphToTest = Cast<UEdGraph>(TestOuter);
			}
		}
	}
	
	return Super::GetGraphType(TestEdGraph);
}

bool UEdGraphSchema_K2::IsTitleBarPin(const UEdGraphPin& Pin) const
{
	return IsExecPin(Pin);
}

void UEdGraphSchema_K2::CreateMacroGraphTerminators(UEdGraph& Graph, UClass* Class) const
{
	const FName GraphName = Graph.GetFName();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(&Graph);
	
	// Create the entry/exit tunnels
	{
		FGraphNodeCreator<UK2Node_Tunnel> EntryNodeCreator(Graph);
		UK2Node_Tunnel* EntryNode = EntryNodeCreator.CreateNode();
		EntryNode->bCanHaveOutputs = true;
		EntryNodeCreator.Finalize();
		SetNodeMetaData(EntryNode, FNodeMetadata::DefaultGraphNode);
	}

	{
		FGraphNodeCreator<UK2Node_Tunnel> ExitNodeCreator(Graph);
		UK2Node_Tunnel* ExitNode = ExitNodeCreator.CreateNode();
		ExitNode->bCanHaveInputs = true;
		ExitNode->NodePosX = 240;
		ExitNodeCreator.Finalize();
		SetNodeMetaData(ExitNode, FNodeMetadata::DefaultGraphNode);
	}
}

void UEdGraphSchema_K2::LinkDataPinFromOutputToInput(UEdGraphNode* InOutputNode, UEdGraphNode* InInputNode) const
{
	for (UEdGraphPin* OutputPin : InOutputNode->Pins)
	{
		if ((OutputPin->Direction == EGPD_Output) && (!IsExecPin(*OutputPin)))
		{
			UEdGraphPin* const InputPin = InInputNode->FindPinChecked(OutputPin->PinName);
			OutputPin->MakeLinkTo(InputPin);
		}
	}
}

void UEdGraphSchema_K2::CreateFunctionGraphTerminators(UEdGraph& Graph, UClass* Class) const
{
	const FName GraphName = Graph.GetFName();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(&Graph);
	check(Blueprint->BlueprintType != BPTYPE_MacroLibrary);

	// Get the function GUID from the most up-to-date class
	FGuid GraphGuid;
	FBlueprintEditorUtils::GetFunctionGuidFromClassByFieldName(FBlueprintEditorUtils::GetMostUpToDateClass(Class), GraphName, GraphGuid);

	// Create a function entry node
	FGraphNodeCreator<UK2Node_FunctionEntry> FunctionEntryCreator(Graph);
	UK2Node_FunctionEntry* EntryNode = FunctionEntryCreator.CreateNode();
	EntryNode->FunctionReference.SetExternalMember(GraphName, Class, GraphGuid);
	FunctionEntryCreator.Finalize();
	SetNodeMetaData(EntryNode, FNodeMetadata::DefaultGraphNode);

	// See if we need to implement a return node
	UFunction* InterfaceToImplement = FindUField<UFunction>(Class, GraphName);
	if (InterfaceToImplement)
	{
		// Add modifier flags from the declaration
		EntryNode->AddExtraFlags(InterfaceToImplement->FunctionFlags & (FUNC_Const | FUNC_Static | FUNC_BlueprintPure));

		UK2Node* NextNode = EntryNode;
		UEdGraphPin* NextExec = FindExecutionPin(*EntryNode, EGPD_Output);
		bool bHasParentNode = false;
		// Create node for call parent function
		if (((Class->GetClassFlags() & CLASS_Interface) == 0)  &&
			(InterfaceToImplement->FunctionFlags & FUNC_BlueprintCallable))
		{
			FGraphNodeCreator<UK2Node_CallParentFunction> FunctionParentCreator(Graph);
			UK2Node_CallParentFunction* ParentNode = FunctionParentCreator.CreateNode();
			ParentNode->SetFromFunction(InterfaceToImplement);
			ParentNode->NodePosX = EntryNode->NodePosX + EntryNode->NodeWidth + 256;
			ParentNode->NodePosY = EntryNode->NodePosY;
			FunctionParentCreator.Finalize();

			UEdGraphPin* ParentNodeExec = FindExecutionPin(*ParentNode, EGPD_Input); 

			// If the parent node has an execution pin, then we should as well (we're overriding them, after all)
			// but perhaps this assumption is not valid in the case where a function becomes pure after being
			// initially declared impure - for that reason I'm checking for validity on both ParentNodeExec and NextExec
			if (ParentNodeExec && NextExec)
			{
				NextExec->MakeLinkTo(ParentNodeExec);
				NextExec = FindExecutionPin(*ParentNode, EGPD_Output);
			}

			// Link any params from the function entry node to the parent node inputs
			LinkDataPinFromOutputToInput(EntryNode, ParentNode);

			NextNode = ParentNode;
			bHasParentNode = true;
		}

		// See if any function params are marked as out
		bool bHasOutParam =  false;
		for( TFieldIterator<FProperty> It(InterfaceToImplement); It && (It->PropertyFlags & CPF_Parm); ++It )
		{
			if( It->PropertyFlags & CPF_OutParm )
			{
				bHasOutParam = true;
				break;
			}
		}

		if( bHasOutParam )
		{
			FGraphNodeCreator<UK2Node_FunctionResult> NodeCreator(Graph);
			UK2Node_FunctionResult* ReturnNode = NodeCreator.CreateNode();
			ReturnNode->FunctionReference = EntryNode->FunctionReference;
			ReturnNode->NodePosX = NextNode->NodePosX + NextNode->NodeWidth + 256;
			ReturnNode->NodePosY = EntryNode->NodePosY;
			NodeCreator.Finalize();
			SetNodeMetaData(ReturnNode, FNodeMetadata::DefaultGraphNode);

			// Auto-connect the pins for entry and exit, so that by default the signature is properly generated
			UEdGraphPin* ResultNodeExec = FindExecutionPin(*ReturnNode, EGPD_Input);
			if (ResultNodeExec && NextExec)
			{
				NextExec->MakeLinkTo(ResultNodeExec);
			}

			if (bHasParentNode)
			{
				LinkDataPinFromOutputToInput(NextNode, ReturnNode);
			}
		}
	}
}

void UEdGraphSchema_K2::CreateFunctionGraphTerminators(UEdGraph& Graph, const UFunction* FunctionSignature) const
{
	const FName GraphName = Graph.GetFName();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(&Graph);
	check(Blueprint->BlueprintType != BPTYPE_MacroLibrary);

	// Create a function entry node
	FGraphNodeCreator<UK2Node_FunctionEntry> FunctionEntryCreator(Graph);
	UK2Node_FunctionEntry* EntryNode = FunctionEntryCreator.CreateNode();
	EntryNode->FunctionReference.SetExternalMember(GraphName, nullptr);
	FunctionEntryCreator.Finalize();
	SetNodeMetaData(EntryNode, FNodeMetadata::DefaultGraphNode);

	// We don't have a signature class to base this on permanently, because it's not an override function.
	// so we need to define the pins as user defined so that they are serialized.

	EntryNode->CreateUserDefinedPinsForFunctionEntryExit(FunctionSignature, /*bIsFunctionEntry=*/ true);

	// See if any function params are marked as out
	bool bHasOutParam =  false;
	for ( TFieldIterator<FProperty> It(FunctionSignature); It && ( It->PropertyFlags & CPF_Parm ); ++It )
	{
		if ( It->PropertyFlags & CPF_OutParm )
		{
			bHasOutParam = true;
			break;
		}
	}

	if ( bHasOutParam )
	{
		FGraphNodeCreator<UK2Node_FunctionResult> NodeCreator(Graph);
		UK2Node_FunctionResult* ReturnNode = NodeCreator.CreateNode();
		ReturnNode->FunctionReference = EntryNode->FunctionReference;
		ReturnNode->NodePosX = EntryNode->NodePosX + EntryNode->NodeWidth + 256;
		ReturnNode->NodePosY = EntryNode->NodePosY;
		NodeCreator.Finalize();
		SetNodeMetaData(ReturnNode, FNodeMetadata::DefaultGraphNode);

		ReturnNode->CreateUserDefinedPinsForFunctionEntryExit(FunctionSignature, /*bIsFunctionEntry=*/ false);

		// Auto-connect the pins for entry and exit, so that by default the signature is properly generated
		UEdGraphPin* EntryNodeExec = FindExecutionPin(*EntryNode, EGPD_Output);
		UEdGraphPin* ResultNodeExec = FindExecutionPin(*ReturnNode, EGPD_Input);
		EntryNodeExec->MakeLinkTo(ResultNodeExec);
	}
}

bool UEdGraphSchema_K2::GetPropertyCategoryInfo(const FProperty* TestProperty, FName& OutCategory, FName& OutSubCategory, UObject*& OutSubCategoryObject, bool& bOutIsWeakPointer)
{
	using namespace UE::EdGraphSchemaK2::Private;

	if (const FInterfaceProperty* InterfaceProperty = CastField<const FInterfaceProperty>(TestProperty))
	{
		OutCategory = PC_Interface;
		OutSubCategoryObject = GetAuthoritativeClass(*InterfaceProperty);
	}
	else if (const FClassProperty* ClassProperty = CastField<const FClassProperty>(TestProperty))
	{
		OutCategory = PC_Class;
		OutSubCategoryObject = GetAuthoritativeClass(*ClassProperty);
	}
	else if (const FSoftClassProperty* SoftClassProperty = CastField<const FSoftClassProperty>(TestProperty))
	{
		OutCategory = PC_SoftClass;
		OutSubCategoryObject = GetAuthoritativeClass(*SoftClassProperty);
	}
	else if (const FSoftObjectProperty* SoftObjectProperty = CastField<const FSoftObjectProperty>(TestProperty))
	{
		OutCategory = PC_SoftObject;
		OutSubCategoryObject = GetAuthoritativeClass(*SoftObjectProperty);
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(TestProperty))
	{
		OutCategory = PC_Object;
		OutSubCategoryObject = GetAuthoritativeClass(*ObjectProperty);
		bOutIsWeakPointer = TestProperty->IsA(FWeakObjectProperty::StaticClass());
	}
	else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(TestProperty))
	{
		OutCategory = PC_Struct;
		OutSubCategoryObject = StructProperty->Struct;
		// Match IsTypeCompatibleWithProperty and erase REINST_ structs here:
		if(UUserDefinedStruct* UDS = Cast<UUserDefinedStruct>(StructProperty->Struct))
		{
			UUserDefinedStruct* RealStruct = UDS->PrimaryStruct.Get();
			if(RealStruct)
			{
				OutSubCategoryObject = RealStruct;
			}
		}
	}
	else if (TestProperty->IsA<FFloatProperty>())
	{
		OutCategory = PC_Real;
		OutSubCategory = PC_Float;
	}
	else if (TestProperty->IsA<FDoubleProperty>())
	{
		OutCategory = PC_Real;
		OutSubCategory = PC_Double;
	}
	else if (TestProperty->IsA<FInt64Property>())
	{
		OutCategory = PC_Int64;
	}
	else if (TestProperty->IsA<FIntProperty>())
	{
		OutCategory = PC_Int;

		if (TestProperty->HasMetaData(FBlueprintMetadata::MD_Bitmask))
		{
			OutSubCategory = PSC_Bitmask;
		}
	}
	else if (const FByteProperty* ByteProperty = CastField<const FByteProperty>(TestProperty))
	{
		OutCategory = PC_Byte;

		if (TestProperty->HasMetaData(FBlueprintMetadata::MD_Bitmask))
		{
			OutSubCategory = PSC_Bitmask;
		}
		else
		{
			OutSubCategoryObject = ByteProperty->Enum;
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<const FEnumProperty>(TestProperty))
	{
		// K2 only supports byte enums right now - any violations should have been caught by UHT or the editor
		if (!EnumProperty->GetUnderlyingProperty()->IsA<FByteProperty>())
		{
			OutCategory = TEXT("unsupported_enum_type: enum size is larger than a byte");
			return false;
		}

		OutCategory = PC_Byte;

		if (TestProperty->HasMetaData(FBlueprintMetadata::MD_Bitmask))
		{
			OutSubCategory = PSC_Bitmask;
		}
		else
		{
			OutSubCategoryObject = EnumProperty->GetEnum();
		}
	}
	else if (TestProperty->IsA<FNameProperty>())
	{
		OutCategory = PC_Name;
	}
	else if (TestProperty->IsA<FBoolProperty>())
	{
		OutCategory = PC_Boolean;
	}
	else if (TestProperty->IsA<FStrProperty>())
	{
		OutCategory = PC_String;
	}
	else if (TestProperty->IsA<FTextProperty>())
	{
		OutCategory = PC_Text;
	}
	else if (const FFieldPathProperty* FieldPathProperty = CastField<const FFieldPathProperty>(TestProperty))
	{
		OutCategory = PC_FieldPath;
		//OutSubCategoryObject = SoftObjectProperty->PropertyClass; @todo: FProp
	}
	else
	{
		OutCategory = TEXT("bad_type");
		return false;
	}

	return true;
}

bool UEdGraphSchema_K2::ConvertPropertyToPinType(const FProperty* Property, /*out*/ FEdGraphPinType& TypeOut) const
{
	if (Property == nullptr)
	{
		TypeOut.PinCategory = TEXT("bad_type");
		return false;
	}

	TypeOut.PinSubCategory = NAME_None;
	
	// Handle whether or not this is an array property
	const FMapProperty* MapProperty = CastField<const FMapProperty>(Property);
	const FSetProperty* SetProperty = CastField<const FSetProperty>(Property);
	const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property);
	const FProperty* TestProperty = Property;
	if (MapProperty)
	{
		TestProperty = MapProperty->KeyProp;

		// set up value property:
		UObject* SubCategoryObject = nullptr;
		bool bIsWeakPtr = false;
		bool bResult = GetPropertyCategoryInfo(MapProperty->ValueProp, TypeOut.PinValueType.TerminalCategory, TypeOut.PinValueType.TerminalSubCategory, SubCategoryObject, bIsWeakPtr);
		TypeOut.PinValueType.TerminalSubCategoryObject = SubCategoryObject;

		if (bIsWeakPtr)
		{
			return false;
		}

		if (!bResult)
		{
			return false;
		}

		// Ensure that the value term will be identified as a wrapper type if the source property has that flag set.
		if(MapProperty->ValueProp->HasAllPropertyFlags(CPF_UObjectWrapper))
		{
			TypeOut.PinValueType.bTerminalIsUObjectWrapper = true;
		}
	}
	else if (SetProperty)
	{
		TestProperty = SetProperty->ElementProp;
	}
	else if (ArrayProperty)
	{
		TestProperty = ArrayProperty->Inner;
	}
	TypeOut.ContainerType = FEdGraphPinType::ToPinContainerType(ArrayProperty != nullptr, SetProperty != nullptr, MapProperty != nullptr);
	TypeOut.bIsReference = Property->HasAllPropertyFlags(CPF_OutParm|CPF_ReferenceParm);
	TypeOut.bIsConst     = Property->HasAllPropertyFlags(CPF_ConstParm);

	// This flag will be set on the key/inner property for container types, so check the test property.
	TypeOut.bIsUObjectWrapper = TestProperty->HasAllPropertyFlags(CPF_UObjectWrapper);

	// Check to see if this is the wildcard property for the target container type
	if(IsWildcardProperty(Property))
	{
		TypeOut.PinCategory = PC_Wildcard;
		if(MapProperty)
		{
			TypeOut.PinValueType.TerminalCategory = PC_Wildcard;
		}
	}
	else if (const FMulticastDelegateProperty* MulticastDelegateProperty = CastField<const FMulticastDelegateProperty>(TestProperty))
	{
		TypeOut.PinCategory = PC_MCDelegate;
		FMemberReference::FillSimpleMemberReference<UFunction>(MulticastDelegateProperty->SignatureFunction, TypeOut.PinSubCategoryMemberReference);
	}
	else if (const FDelegateProperty* DelegateProperty = CastField<const FDelegateProperty>(TestProperty))
	{
		TypeOut.PinCategory = PC_Delegate;
		FMemberReference::FillSimpleMemberReference<UFunction>(DelegateProperty->SignatureFunction, TypeOut.PinSubCategoryMemberReference);
	}
	else
	{
		UObject* SubCategoryObject = nullptr;
		bool bIsWeakPointer = false;
		bool bResult = GetPropertyCategoryInfo(TestProperty, TypeOut.PinCategory, TypeOut.PinSubCategory, SubCategoryObject, bIsWeakPointer);
		TypeOut.bIsWeakPointer = bIsWeakPointer;
		TypeOut.PinSubCategoryObject = SubCategoryObject;
		if (!bResult)
		{
			return false;
		}
	}

	if (TypeOut.PinSubCategory == PSC_Bitmask)
	{
		const FString& BitmaskEnumName = TestProperty->GetMetaData(TEXT("BitmaskEnum"));
		if(!BitmaskEnumName.IsEmpty())
		{
			// @TODO: Potentially replace this with a serialized UEnum reference on the FProperty (e.g. FByteProperty::Enum)
			TypeOut.PinSubCategoryObject = UClass::TryFindTypeSlow<UEnum>(BitmaskEnumName);
		}
	}

	return true;
}

bool UEdGraphSchema_K2::HasWildcardParams(const UFunction* Function)
{
	bool bResult = false;
	for (TFieldIterator<const FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm) && !bResult; ++PropIt)
	{
		const FProperty* FuncParamProperty = *PropIt;

		if (IsWildcardProperty(FuncParamProperty))
		{
			bResult = true;
		}
	}
	return bResult;
}

bool UEdGraphSchema_K2::IsWildcardProperty(const FProperty* Property)
{
	UFunction* Function = Property->GetOwner<UFunction>();

	return Function && ( UK2Node_CallArrayFunction::IsWildcardProperty(Function, Property)
		|| UK2Node_CallFunction::IsStructureWildcardProperty(Function, Property->GetFName())
		|| UK2Node_CallFunction::IsWildcardProperty(Function, Property)
		|| FEdGraphUtilities::IsArrayDependentParam(Function, Property->GetFName()) );
}

FText UEdGraphSchema_K2::TypeToText(const FProperty* const Property)
{
	if (const FStructProperty* Struct = CastField<FStructProperty>(Property))
	{
		if (Struct->Struct)
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = PC_Struct;
			PinType.PinSubCategoryObject = Struct->Struct;
			return TypeToText(PinType);
		}
	}
	else if (const FClassProperty* Class = CastField<FClassProperty>(Property))
	{
		if (Class->MetaClass)
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = PC_Class;
			PinType.PinSubCategoryObject = Class->MetaClass;
			return TypeToText(PinType);
		}
	}
	else if (const FInterfaceProperty* Interface = CastField<FInterfaceProperty>(Property))
	{
		if (Interface->InterfaceClass != nullptr)
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = PC_Interface;
			PinType.PinSubCategoryObject = Interface->InterfaceClass;
			return TypeToText(PinType);
		}
	}
	else if (const FObjectPropertyBase* Obj = CastField<FObjectPropertyBase>(Property))
	{
		if( Obj->PropertyClass )
		{
			FEdGraphPinType PinType;
			PinType.PinCategory = PC_Object;
			PinType.PinSubCategoryObject = Obj->PropertyClass;
			PinType.bIsWeakPointer = Property->IsA(FWeakObjectProperty::StaticClass());
			return TypeToText(PinType);
		}

		return FText::GetEmpty();
	}
	else if (const FArrayProperty* Array = CastField<FArrayProperty>(Property))
	{
		if (Array->Inner)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ArrayType"), TypeToText(Array->Inner));
			return FText::Format(LOCTEXT("ArrayPropertyText", "Array of {ArrayType}"), Args); 
		}
	}
	else if (const FSetProperty* Set = CastField<FSetProperty>(Property))
	{
		if (Set->ElementProp)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("SetType"), TypeToText(Set->ElementProp));
			return FText::Format(LOCTEXT("SetPropertyText", "Set of {SetType}"), Args);
		}
	}
	else if (const FMapProperty* Map = CastField<FMapProperty>(Property))
	{
		if (Map->KeyProp && Map->ValueProp)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("MapKeyType"), TypeToText(Map->KeyProp));
			Args.Add(TEXT("MapValueType"), TypeToText(Map->ValueProp));
			return FText::Format(LOCTEXT("MapPropertyText", "Map of {MapKeyType} to {MapValueType}"), Args);
		}
	}
	
	return FText::FromString(Property->GetClass()->GetName());
}

FText UEdGraphSchema_K2::GetCategoryText(const FName Category, const bool bForMenu)
{
	return GetCategoryText(Category, NAME_None, bForMenu);
}

FText UEdGraphSchema_K2::GetCategoryText(FName Category, FName SubCategory, bool bForMenu)
{
	using namespace UE::EdGraphSchemaK2::Private;

	if (Category.IsNone())
	{
		return FText::GetEmpty();
	}

	static TMap<FName, FText> CategoryDescriptions;
	if (CategoryDescriptions.Num() == 0)
	{
		CategoryDescriptions.Add(PC_Exec, LOCTEXT("Exec", "Exec"));
		CategoryDescriptions.Add(PC_Boolean, LOCTEXT("BoolCategory", "Boolean"));
		CategoryDescriptions.Add(PC_Byte, LOCTEXT("ByteCategory", "Byte"));
		CategoryDescriptions.Add(PC_Class, LOCTEXT("ClassCategory", "Class Reference"));
		CategoryDescriptions.Add(PC_Int, LOCTEXT("IntCategory", "Integer"));
		CategoryDescriptions.Add(PC_Int64, LOCTEXT("Int64Category", "Integer64"));
		CategoryDescriptions.Add(PC_Real, LOCTEXT("RealCategory", "Float"));
		CategoryDescriptions.Add(PC_Float, LOCTEXT("FloatCategory", "Float (single-precision)"));
		CategoryDescriptions.Add(PC_Double, LOCTEXT("DoubleCategory", "Float (double-precision)"));
		CategoryDescriptions.Add(PC_Name, LOCTEXT("NameCategory", "Name"));
		CategoryDescriptions.Add(PC_Delegate, LOCTEXT("DelegateCategory", "Delegate"));
		CategoryDescriptions.Add(PC_MCDelegate, LOCTEXT("MulticastDelegateCategory", "Multicast Delegate"));
		CategoryDescriptions.Add(PC_Object, LOCTEXT("ObjectCategory", "Object Reference"));
		CategoryDescriptions.Add(PC_Interface, LOCTEXT("InterfaceCategory", "Interface"));
		CategoryDescriptions.Add(PC_String, LOCTEXT("StringCategory", "String"));
		CategoryDescriptions.Add(PC_Text, LOCTEXT("TextCategory", "Text"));
		CategoryDescriptions.Add(PC_Struct, LOCTEXT("StructCategory", "Structure"));
		CategoryDescriptions.Add(PC_Wildcard, LOCTEXT("WildcardCategory", "Wildcard"));
		CategoryDescriptions.Add(PC_Enum, LOCTEXT("EnumCategory", "Enum"));
		CategoryDescriptions.Add(PC_SoftObject, LOCTEXT("SoftObjectReferenceCategory", "Soft Object Reference"));
		CategoryDescriptions.Add(PC_SoftClass, LOCTEXT("SoftClassReferenceCategory", "Soft Class Reference"));
		CategoryDescriptions.Add(PC_FieldPath, LOCTEXT("FieldPathReferenceCategory", "Property Reference"));
		CategoryDescriptions.Add(AllObjectTypes, LOCTEXT("AllObjectTypes", "Object Types"));
	}

	if (const FText* TypeDesc = CategoryDescriptions.Find(Category))
	{
		const bool bUseDetailedRealCategory =
			bForMenu &&
			(Category == PC_Real) &&
			((SubCategory == PC_Float) || (SubCategory == PC_Double));

		if (bUseDetailedRealCategory)
		{
			TypeDesc = CategoryDescriptions.Find(SubCategory);
			check(TypeDesc);
		}

		return *TypeDesc;
	}
	else
	{
		return FText::FromName(Category);
	}
}

FText UEdGraphSchema_K2::TerminalTypeToText(const FName Category, const FName SubCategory, UObject* SubCategoryObject, bool bIsWeakPtr)
{
	FText PropertyText;

	if (SubCategory != UEdGraphSchema_K2::PSC_Bitmask && SubCategoryObject != nullptr)
	{
		if (Category == UEdGraphSchema_K2::PC_Byte)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("EnumName"), FText::FromString(SubCategoryObject->GetName()));
			PropertyText = FText::Format(LOCTEXT("EnumAsText", "{EnumName} Enum"), Args);
		}
		else
		{
			FString SubCategoryObjName;
			if (UField* SubCategoryField = Cast<UField>(SubCategoryObject))
			{
				SubCategoryObjName = SubCategoryField->GetDisplayNameText().ToString();
			}
			else
			{
				SubCategoryObjName = SubCategoryObject->GetName();
			}

			if (!bIsWeakPtr)
			{
				UClass* PSCOAsClass = Cast<UClass>(SubCategoryObject);
				const bool bIsInterface = PSCOAsClass && PSCOAsClass->HasAnyClassFlags(CLASS_Interface);

				FFormatNamedArguments Args;
				Args.Add(TEXT("ObjectName"), FText::FromString(FName::NameToDisplayString(SubCategoryObjName, /*bIsBool =*/false)));

				// Don't display the category for "well-known" struct types
				if (Category == UEdGraphSchema_K2::PC_Struct && (SubCategoryObject == UEdGraphSchema_K2::VectorStruct || SubCategoryObject == UEdGraphSchema_K2::Vector3fStruct  || SubCategoryObject == UEdGraphSchema_K2::RotatorStruct || SubCategoryObject == UEdGraphSchema_K2::TransformStruct))
				{
					PropertyText = FText::Format(LOCTEXT("ObjectAsTextWithoutCategory", "{ObjectName}"), Args);
				}
				// If this is a raw UObject reference don't display Object twice
				else if (((Category == UEdGraphSchema_K2::PC_Object) || (Category == UEdGraphSchema_K2::PC_SoftObject)) && (SubCategoryObject == UObject::StaticClass()))
				{
					Args.Add(TEXT("Category"), UEdGraphSchema_K2::GetCategoryText(Category));
					PropertyText = FText::Format(LOCTEXT("ObjectAsJustCategory", "{Category}"), Args);
				}
				else
				{
					Args.Add(TEXT("Category"), (!bIsInterface ? UEdGraphSchema_K2::GetCategoryText(Category) : UEdGraphSchema_K2::GetCategoryText(PC_Interface)));
					PropertyText = FText::Format(LOCTEXT("ObjectAsText", "{ObjectName} {Category}"), Args);
				}
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Category"), FText::FromName(Category));
				Args.Add(TEXT("ObjectName"), FText::FromString(SubCategoryObjName));
				PropertyText = FText::Format(LOCTEXT("WeakPtrAsText", "{ObjectName} Weak {Category}"), Args);
			}
		}
	}
	else if (!SubCategory.IsNone())
	{
		if (Category == UEdGraphSchema_K2::PC_Real)
		{
			PropertyText = UEdGraphSchema_K2::GetCategoryText(Category, SubCategory, true);
		}
		else
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Category"), UEdGraphSchema_K2::GetCategoryText(Category));
			Args.Add(TEXT("ObjectName"), FText::FromString(FName::NameToDisplayString(SubCategory.ToString(), false)));
			PropertyText = FText::Format(LOCTEXT("ObjectAsText", "{ObjectName} {Category}"), Args);
		}
	}
	else
	{
		PropertyText = UEdGraphSchema_K2::GetCategoryText(Category);
	}

	return PropertyText;
}

FText UEdGraphSchema_K2::TypeToText(const FEdGraphPinType& Type)
{
	FText PropertyText = TerminalTypeToText(Type.PinCategory, Type.PinSubCategory, Type.PinSubCategoryObject.Get(), Type.bIsWeakPointer);

	if (Type.IsMap())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("KeyTitle"), PropertyText);
		FText ValueText = TerminalTypeToText(Type.PinValueType.TerminalCategory, Type.PinValueType.TerminalSubCategory, Type.PinValueType.TerminalSubCategoryObject.Get(), Type.PinValueType.bTerminalIsWeakPointer);
		Args.Add(TEXT("ValueTitle"), ValueText);
		PropertyText = FText::Format(LOCTEXT("MapAsText", "Map of {KeyTitle}s to {ValueTitle}s"), Args);
	}
	else if (Type.IsSet())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyTitle"), PropertyText);
		PropertyText = FText::Format(LOCTEXT("SetAsText", "Set of {PropertyTitle}s"), Args);
	}
	else if (Type.IsArray())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyTitle"), PropertyText);
		PropertyText = FText::Format(LOCTEXT("ArrayAsText", "Array of {PropertyTitle}s"), Args);
	}
	else if (Type.bIsReference)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyTitle"), PropertyText);
		PropertyText = FText::Format(LOCTEXT("PropertyByRef", "{PropertyTitle} (by ref)"), Args);
	}

	return PropertyText;
}

void UEdGraphSchema_K2::GetVariableTypeTree(TArray< TSharedPtr<FPinTypeTreeInfo> >& TypeTree, ETypeTreeFilter TypeTreeFilter) const
{
	bool bAllowExec = (TypeTreeFilter & ETypeTreeFilter::AllowExec) == ETypeTreeFilter::AllowExec;
	bool bAllowWildCard = (TypeTreeFilter & ETypeTreeFilter::AllowWildcard) == ETypeTreeFilter::AllowWildcard;
	bool bIndexTypesOnly = (TypeTreeFilter & ETypeTreeFilter::IndexTypesOnly) == ETypeTreeFilter::IndexTypesOnly;
	bool bRootTypesOnly = (TypeTreeFilter & ETypeTreeFilter::RootTypesOnly) == ETypeTreeFilter::RootTypesOnly;

	// Clear the list
	TypeTree.Empty();

	if( bAllowExec )
	{
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Exec, true), PC_Exec, this, LOCTEXT("ExecType", "Execution pin"))));
	}

	TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Boolean, true), PC_Boolean, this, LOCTEXT("BooleanType", "True or false value"))));
	TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Byte, true), PC_Byte, this, LOCTEXT("ByteType", "8 bit number"))));
	TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Int, true), PC_Int, this, LOCTEXT("IntegerType", "Integer number"))));
	TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Int64, true), PC_Int64, this, LOCTEXT("Integer64Type", "64 bit Integer number"))));

	if (!bIndexTypesOnly)
	{
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Real, true), PC_Real, this, LOCTEXT("RealType", "Floating point number"))));
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Name, true), PC_Name, this, LOCTEXT("NameType", "A text name"))));
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_String, true), PC_String, this, LOCTEXT("StringType", "A text string"))));
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Text, true), PC_Text, this, LOCTEXT("TextType", "A localizable text string"))));

		// Add in special first-class struct types
		if (!bRootTypesOnly)
		{
			TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(PC_Struct, TBaseStructure<FVector>::Get(), LOCTEXT("VectorType", "A 3D vector"))));
			TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(PC_Struct, TBaseStructure<FRotator>::Get(), LOCTEXT("RotatorType", "A 3D rotation"))));
			TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(PC_Struct, TBaseStructure<FTransform>::Get(), LOCTEXT("TransformType", "A 3D transformation, including translation, rotation and 3D scale."))));
		}
	}
	// Add wildcard type
	if (bAllowWildCard)
	{
		TypeTree.Add(MakeShareable(new FPinTypeTreeInfo(GetCategoryText(PC_Wildcard, true), PC_Wildcard, this, LOCTEXT("WildcardType", "Wildcard type (unspecified)"))));
	}

	// Add the types that have subtrees
	if (!bIndexTypesOnly)
	{
		TSharedPtr<FPinTypeTreeInfo> Structs = MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_Struct, true), PC_Struct, this, LOCTEXT("StructType", "Struct (value) types"), true);
		if (!bRootTypesOnly)
		{
			GatherPinsImpl::FindStructs(Structs);
		}
		TypeTree.Add(Structs);

		TSharedPtr<FPinTypeTreeInfo> Interfaces = MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_Interface, true), PC_Interface, this, LOCTEXT("InterfaceType", "Interface types"), true);
		TypeTree.Add(Interfaces);

		if (!bRootTypesOnly)
		{
			TSharedPtr<FPinTypeTreeInfo> Objects = MakeShared<FPinTypeTreeInfo>(GetCategoryText(AllObjectTypes, true), AllObjectTypes, this, LOCTEXT("ObjectType", "Object types"), true);
			GatherPinsImpl::FindObjectsAndInterfaces(Objects, Interfaces);
			TypeTree.Add(Objects);
		}
		else
		{
			TypeTree.Add(MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_Object, true), PC_Object, this, LOCTEXT("ObjectTypeHardReference", "Hard reference to an Object"), true));
			TypeTree.Add(MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_Class, true), PC_Class, this, LOCTEXT("ClassType", "Hard reference to a Class"), true));
			TypeTree.Add(MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_SoftObject, true), PC_SoftObject, this, LOCTEXT("SoftObjectType", "Soft reference to an Object"), true));
			TypeTree.Add(MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_SoftClass, true), PC_SoftClass, this, LOCTEXT("SoftClassType", "Soft reference to a Class"), true));
		}
	}
	TSharedPtr<FPinTypeTreeInfo> Enums = MakeShared<FPinTypeTreeInfo>(GetCategoryText(PC_Enum, true), PC_Enum, this, LOCTEXT("EnumType", "Enumeration types."), true);
	if (!bRootTypesOnly)
	{
		GatherPinsImpl::FindEnums(Enums);
	}
	TypeTree.Add(Enums);
}

bool UEdGraphSchema_K2::DoesTypeHaveSubtypes(const FName Category) const
{
	return (Category == PC_Struct) || (Category == PC_Object) || (Category == PC_SoftObject) || (Category == PC_SoftClass) || (Category == PC_Interface) || (Category == PC_Class) || (Category == PC_Enum) || (Category == AllObjectTypes);
}

struct FWildcardArrayPinHelper
{
	static bool CheckArrayCompatibility(const UEdGraphPin* OutputPin, const UEdGraphPin* InputPin, bool bIgnoreArray)
	{
		if (bIgnoreArray)
		{
			return true;
		}

		const UK2Node* OwningNode = InputPin ? Cast<UK2Node>(InputPin->GetOwningNode()) : nullptr;
		const bool bInputWildcardPinAcceptsArray = !OwningNode || OwningNode->DoesInputWildcardPinAcceptArray(InputPin);
		if (bInputWildcardPinAcceptsArray)
		{
			return true;
		}

		const bool bOutputWildcardPinAcceptsContainer = !OwningNode || OwningNode->DoesOutputWildcardPinAcceptContainer(OutputPin);
		if(bOutputWildcardPinAcceptsContainer)
		{
			return true;
		}

		const bool bCheckInputPin = (InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard) && !InputPin->PinType.IsArray();
		const bool bArrayOutputPin = OutputPin && OutputPin->PinType.IsArray();
		return !(bCheckInputPin && bArrayOutputPin);
	}
};

bool UEdGraphSchema_K2::ArePinsCompatible(const UEdGraphPin* PinA, const UEdGraphPin* PinB, const UClass* CallingContext, bool bIgnoreArray /*= false*/) const
{
	if ((PinA->Direction == EGPD_Input) && (PinB->Direction == EGPD_Output))
	{
		return FWildcardArrayPinHelper::CheckArrayCompatibility(PinB, PinA, bIgnoreArray)
			&& ArePinTypesCompatible(PinB->PinType, PinA->PinType, CallingContext, bIgnoreArray);
	}
	else if ((PinB->Direction == EGPD_Input) && (PinA->Direction == EGPD_Output))
	{
		return FWildcardArrayPinHelper::CheckArrayCompatibility(PinA, PinB, bIgnoreArray)
			&& ArePinTypesCompatible(PinA->PinType, PinB->PinType, CallingContext, bIgnoreArray);
	}
	else
	{
		return false;
	}
}

namespace
{
	static UClass* GetOriginalClassToFixCompatibilit(const UClass* InClass)
	{
		const UBlueprint* BP = InClass ? Cast<const UBlueprint>(InClass->ClassGeneratedBy) : nullptr;
		return BP ? BP->OriginalClass : nullptr;
	}

	// During compilation, pins are moved around for node expansion and the Blueprints may still inherit from REINST_ classes
	// which causes problems for IsChildOf. Because we do not want to modify IsChildOf we must use a separate function
	// that can check to see if classes have an AuthoritativeClass that IsChildOf a Target class.
	static bool IsAuthoritativeChildOf(const UStruct* InSourceStruct, const UStruct* InTargetStruct)
	{
		bool bResult = false;
		bool bIsNonNativeClass = false;
		if (const UClass* TargetAsClass = Cast<const UClass>(InTargetStruct))
		{
			InTargetStruct = TargetAsClass->GetAuthoritativeClass();
		}
		if (UClass* SourceAsClass = const_cast<UClass*>(Cast<UClass>(InSourceStruct)))
		{
			if (SourceAsClass->ClassGeneratedBy)
			{
				// We have a non-native (Blueprint) class which means it can exist in a semi-compiled state and inherit from a REINST_ class.
				bIsNonNativeClass = true;
				while (SourceAsClass)
				{
					if (SourceAsClass->GetAuthoritativeClass() == InTargetStruct)
					{
						bResult = true;
						break;
					}
					SourceAsClass = SourceAsClass->GetSuperClass();
				}
			}
		}

		// We have a native (C++) class, do a normal IsChildOf check
		if (!bIsNonNativeClass)
		{
			bResult = InSourceStruct && InSourceStruct->IsChildOf(InTargetStruct);
		}

		return bResult;
	}

	static bool ExtendedIsChildOf(const UClass* Child, const UClass* Parent)
	{
		if (Child && Child->IsChildOf(Parent))
		{
			return true;
		}

		const UClass* OriginalChild = GetOriginalClassToFixCompatibilit(Child);
		if (OriginalChild && OriginalChild->IsChildOf(Parent))
		{
			return true;
		}

		const UClass* OriginalParent = GetOriginalClassToFixCompatibilit(Parent);
		if (OriginalParent && Child && Child->IsChildOf(OriginalParent))
		{
			return true;
		}

		return false;
	}

	static bool ExtendedImplementsInterface(const UClass* Class, const UClass* Interface)
	{
		if (Class->ImplementsInterface(Interface))
		{
			return true;
		}

		const UClass* OriginalClass = GetOriginalClassToFixCompatibilit(Class);
		if (OriginalClass && OriginalClass->ImplementsInterface(Interface))
		{
			return true;
		}

		const UClass* OriginalInterface = GetOriginalClassToFixCompatibilit(Interface);
		if (OriginalInterface && Class->ImplementsInterface(OriginalInterface))
		{
			return true;
		}

		return false;
	}
};


bool UEdGraphSchema_K2::DefaultValueSimpleValidation(const FEdGraphPinType& PinType, const FName PinName, const FString& NewDefaultValue, TObjectPtr<UObject> NewDefaultObject, const FText& InNewDefaultText, FString* OutMsg /*= NULL*/) const
{
#ifdef DVSV_RETURN_MSG
	static_assert(false, "Macro redefinition.");
#endif
#define DVSV_RETURN_MSG(Str) if (OutMsg) { *OutMsg = Str; } return false;

	const FName PinCategory = PinType.PinCategory;
	const FName PinSubCategory = PinType.PinSubCategory;
	const UObject* PinSubCategoryObject = PinType.PinSubCategoryObject.Get();

	if (PinType.IsContainer())
	{
		// containers are validated separately
	}
	//@TODO: FCString::Atoi, FCString::Atof, and appStringToBool will 'accept' any input, but we should probably catch and warn
	// about invalid input (non numeric for int/byte/float, and non 0/1 or yes/no/true/false for bool)

	else if (PinCategory == PC_Boolean)
	{
		// All input is acceptable to some degree
	}
	else if (PinCategory == PC_Byte)
	{
		const UEnum* EnumPtr = Cast<const UEnum>(PinSubCategoryObject);
		if (EnumPtr)
		{
			if ( NewDefaultValue == TEXT("(INVALID)") || EnumPtr->GetIndexByNameString(NewDefaultValue) == INDEX_NONE)
			{
				DVSV_RETURN_MSG(FString::Printf(TEXT("'%s' is not a valid enumerant of '<%s>'"), *NewDefaultValue, *(EnumPtr->GetName())));
			}
		}
		else if (!NewDefaultValue.IsEmpty())
		{
			int32 Value;
			if (!FDefaultValueHelper::ParseInt(NewDefaultValue, Value))
			{
				DVSV_RETURN_MSG(TEXT("Expected a valid unsigned number for a byte property"));
			}
			if ((Value < 0) || (Value > 255))
			{
				DVSV_RETURN_MSG(TEXT("Expected a value between 0 and 255 for a byte property"));
			}
		}
	}
	else if ((PinCategory == PC_Class))
	{
		// Should have an object set but no string
		if (!NewDefaultValue.IsEmpty())
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("String NewDefaultValue '%s' specified on class pin '%s'"), *NewDefaultValue, *PinName.ToString()));
		}

		if (NewDefaultObject == nullptr)
		{
			// Valid self-reference or empty reference
		}
		else
		{
			// Otherwise, we expect to be able to resolve the type at least
			const UClass* DefaultClassType = Cast<const UClass>(NewDefaultObject);
			if (DefaultClassType == nullptr)
			{
				DVSV_RETURN_MSG(FString::Printf(TEXT("Literal on pin %s is not a class."), *PinName.ToString()));
			}
			else
			{
				// @TODO support PinSubCategory == 'self'
				const UClass* PinClassType = Cast<const UClass>(PinSubCategoryObject);
				if (PinClassType == nullptr)
				{
					DVSV_RETURN_MSG(FString::Printf(TEXT("Failed to find class for pin %s"), *PinName.ToString()));
				}
				else
				{
					// Have both types, make sure the specified type is a valid subtype
					if (!IsAuthoritativeChildOf(DefaultClassType, PinClassType))
					{
						DVSV_RETURN_MSG(FString::Printf(TEXT("%s isn't a valid subclass of %s (specified on pin %s)"), *NewDefaultObject->GetPathName(), *PinClassType->GetName(), *PinName.ToString()));
					}
				}
			}
		}
	}
	else if (PinCategory == PC_Real)
	{
		if (!NewDefaultValue.IsEmpty())
		{
			if (!FDefaultValueHelper::IsStringValidFloat(NewDefaultValue))
			{
				DVSV_RETURN_MSG(TEXT("Expected a valid number for a real property"));
			}
		}
	}
	else if (PinCategory == PC_Int)
	{
		if (!NewDefaultValue.IsEmpty())
		{
			if (!FDefaultValueHelper::IsStringValidInteger(NewDefaultValue))
			{
				DVSV_RETURN_MSG(TEXT("Expected a valid number for an integer property"));
			}
		}
	}
	else if (PinCategory == PC_Int64)
	{
		if (!NewDefaultValue.IsEmpty())
		{
			int64 ParsedInt64;
			if (!FDefaultValueHelper::ParseInt64(NewDefaultValue, ParsedInt64))
			{
				DVSV_RETURN_MSG(TEXT("Expected a valid number for an integer64 property"));
			}
		}
	}
	else if (PinCategory == PC_Name)
	{
		// Anything is allowed
	}
	else if ((PinCategory == PC_Object) || (PinCategory == PC_Interface))
	{
		if (PinSubCategoryObject == nullptr && (PinSubCategory != PSC_Self))
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("PinSubCategoryObject on pin '%s' is NULL and PinSubCategory is '%s' not 'self'"), *PinName.ToString(), *PinSubCategory.ToString()));
		}

		if (PinSubCategoryObject != nullptr && !PinSubCategory.IsNone())
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("PinSubCategoryObject on pin '%s' is non-NULL but PinSubCategory is '%s', should be empty"), *PinName.ToString(), *PinSubCategory.ToString()));
		}

		// Should have an object set but no string - 'self' is not a valid NewDefaultValue for PC_Object pins
		if (!NewDefaultValue.IsEmpty())
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("String NewDefaultValue '%s' specified on object pin '%s'"), *NewDefaultValue, *PinName.ToString()));
		}

		// Check that the object that is set is of the correct class
		const UClass* ObjectClass = Cast<const UClass>(PinSubCategoryObject);
		if(ObjectClass)
		{
			ObjectClass = ObjectClass->GetAuthoritativeClass();
		}
		if (NewDefaultObject != nullptr && ObjectClass != nullptr)
		{
			const UClass* AuthoritativeClass = NewDefaultObject.GetClass()->GetAuthoritativeClass();
			if (!AuthoritativeClass || !AuthoritativeClass->IsChildOf(ObjectClass))
			{
				// Not a type of object, but is it an object implementing an interface?
				if(PinCategory != PC_Interface || !NewDefaultObject.GetClass()->ImplementsInterface(ObjectClass))
				{
					DVSV_RETURN_MSG(FString::Printf(TEXT("%s isn't a %s (specified on pin %s)"), *NewDefaultObject->GetPathName(), *ObjectClass->GetName(), *PinName.ToString()));
				}
			}
		}
	}
	else if ((PinCategory == PC_SoftObject) || (PinCategory == PC_SoftClass))
	{
		// Should not have an object set, should be converted to string before getting here
		if (NewDefaultObject)
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("NewDefaultObject '%s' specified on object pin '%s'"), *NewDefaultObject->GetPathName(), *PinName.ToString()));
		}

		if (!NewDefaultValue.IsEmpty())
		{
			FText PathReason;

			if (!FPackageName::IsValidObjectPath(NewDefaultValue, &PathReason))
			{
				DVSV_RETURN_MSG(FString::Printf(TEXT("Soft Reference '%s' is invalid format for object pin '%s': %s"), *NewDefaultValue, *PinName.ToString(), *PathReason.ToString()));
			}

			// Class and IsAsset validation is not foolproof for soft references, skip
		}
	}
	else if (PinCategory == PC_String || PinCategory == PC_FieldPath)
	{
		// All strings are valid
	}
	else if (PinCategory == PC_Text)
	{
		// Neither of these should ever be true 
		if (InNewDefaultText.IsTransient())
		{
			DVSV_RETURN_MSG(TEXT("Invalid text literal, text is transient!"));
		}
	}
	else if (PinCategory == PC_Struct)
	{
		if (!PinSubCategory.IsNone())
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("Invalid PinSubCategory value '%s' (it should be empty)"), *PinSubCategory.ToString()));
		}

		// Only FRotator and FVector properties are currently allowed to have a valid default value
		const UScriptStruct* StructType = Cast<const UScriptStruct>(PinSubCategoryObject);
		if (StructType == nullptr)
		{
			//@TODO: MessageLog.Error(*FString::Printf(TEXT("Failed to find struct named %s (passed thru @@)"), *PinSubCategory), SourceObject);
			DVSV_RETURN_MSG(FString::Printf(TEXT("No struct specified for pin '%s'"), *PinName.ToString()));
		}
		else if (!NewDefaultValue.IsEmpty())
		{
			if ((StructType == VectorStruct) || (StructType == Vector3fStruct))
			{
				if (!FDefaultValueHelper::IsStringValidVector(NewDefaultValue))
				{
					DVSV_RETURN_MSG(TEXT("Invalid value for an FVector"));
				}
			}
			else if (StructType == RotatorStruct)
			{
				if (!FDefaultValueHelper::IsStringValidRotator(NewDefaultValue))
				{
					DVSV_RETURN_MSG(TEXT("Invalid value for an FRotator"));
				}
			}
			else if (StructType == TransformStruct)
			{
				FTransform Transform;
				if (!Transform.InitFromString(NewDefaultValue))
				{
					DVSV_RETURN_MSG(TEXT("Invalid value for an FTransform"));
				}
			}
			else if (StructType == LinearColorStruct)
			{
				FLinearColor Color;
				// Color form: "(R=%f,G=%f,B=%f,A=%f)"
				if (!Color.InitFromString(NewDefaultValue))
				{
					DVSV_RETURN_MSG(TEXT("Invalid value for an FLinearColor"));
				}
			}
			else
			{
				// Structs must pass validation at this point, because we need a FStructProperty to run ImportText
				// They'll be verified in FKCHandler_CallFunction::CreateFunctionCallStatement()
			}
		}
	}
	else if (PinCategory == TEXT("CommentType"))
	{
		// Anything is allowed
	}
	else if (PinCategory == TEXT("Delegate"))
	{
		// Only empty delegates are allowed, support both empty string and the format used in ExportText
		if (!NewDefaultValue.IsEmpty() && NewDefaultValue != TEXT("(null).None"))
		{
			DVSV_RETURN_MSG(FString::Printf(TEXT("Unsupported value %s for delegate pin %s, only empty delegates are supported"), *NewDefaultValue, *PinName.ToString()));
		}
	}
	else
	{
		//@TODO: MessageLog.Error(*FString::Printf(TEXT("Unsupported type %s on @@"), *UEdGraphSchema_K2::TypeToText(Type).ToString()), SourceObject);
		DVSV_RETURN_MSG(FString::Printf(TEXT("Unsupported type %s on pin %s"), *UEdGraphSchema_K2::TypeToText(PinType).ToString(), *PinName.ToString()));
	}

#undef DVSV_RETURN_MSG

	return true;
}


bool UEdGraphSchema_K2::ArePinTypesCompatible(const FEdGraphPinType& Output, const FEdGraphPinType& Input, const UClass* CallingContext, bool bIgnoreArray /*= false*/) const
{
	using namespace UE::Kismet::BlueprintTypeConversions;

	if (!bIgnoreArray && 
		(Output.ContainerType != Input.ContainerType) && 
		(Input.PinCategory != PC_Wildcard || Input.IsContainer()) && 
		(Output.PinCategory != PC_Wildcard || Output.IsContainer()))
	{
		return false;
	}
	else if (Output.PinCategory == Input.PinCategory)
	{
		bool bAreConvertibleStructs = false;
		const UScriptStruct* OutputStruct = Cast<UScriptStruct>(Output.PinSubCategoryObject.Get());
		const UScriptStruct* InputStruct = Cast<UScriptStruct>(Input.PinSubCategoryObject.Get());
		if (OutputStruct != InputStruct)
		{
			bAreConvertibleStructs =
				FStructConversionTable::Get().GetConversionFunction(OutputStruct, InputStruct).IsSet();
		}

		if ((Output.PinSubCategory == Input.PinSubCategory) 
			&& (Output.PinSubCategoryObject == Input.PinSubCategoryObject)
			&& (Output.PinSubCategoryMemberReference == Input.PinSubCategoryMemberReference))
		{
			if(Input.IsMap())
			{
				OutputStruct = Cast<UScriptStruct>(Output.PinValueType.TerminalSubCategoryObject.Get());
				InputStruct = Cast<UScriptStruct>(Input.PinValueType.TerminalSubCategoryObject.Get());
				if (OutputStruct != InputStruct)
				{
					bAreConvertibleStructs =
						FStructConversionTable::Get().GetConversionFunction(OutputStruct, InputStruct).IsSet();
				}

				return 
					Input.PinValueType.TerminalCategory == PC_Wildcard ||
					Output.PinValueType.TerminalCategory == PC_Wildcard ||
					((Input.PinValueType.TerminalCategory == PC_Real) && (Output.PinValueType.TerminalCategory == PC_Real)) ||
					bAreConvertibleStructs ||
					Input.PinValueType == Output.PinValueType;
			}
			return true;
		}
		// Reals, whether they're actually a float or double, are always compatible.
		// We'll insert an implicit conversion in the bytecode where necessary.
		else if (Output.PinCategory == PC_Real)
		{
			return true;
		}
		else if (bAreConvertibleStructs)
		{
			return true;
		}
		else if (Output.PinCategory == PC_Interface)
		{
			UClass const* OutputClass = Cast<UClass const>(Output.PinSubCategoryObject.Get());
			UClass const* InputClass = Cast<UClass const>(Input.PinSubCategoryObject.Get());
			if (!OutputClass || !InputClass 
				|| !OutputClass->IsChildOf(UInterface::StaticClass())
				|| !InputClass->IsChildOf(UInterface::StaticClass()))
			{
				UE_LOGF(LogBlueprint, Error,
					"UEdGraphSchema_K2::ArePinTypesCompatible invalid interface types - OutputClass: %ls, InputClass: %ls, CallingContext: %ls",
					*GetPathNameSafe(OutputClass), *GetPathNameSafe(InputClass), *GetPathNameSafe(CallingContext));
				return false;
			}

			return ExtendedIsChildOf(OutputClass->GetAuthoritativeClass(), InputClass->GetAuthoritativeClass());
		}
		else if (((Output.PinCategory == PC_SoftObject) && (Input.PinCategory == PC_SoftObject))
			|| ((Output.PinCategory == PC_SoftClass) && (Input.PinCategory == PC_SoftClass)))
		{
			const UClass* OutputObject = (Output.PinSubCategory == PSC_Self) ? CallingContext : Cast<const UClass>(Output.PinSubCategoryObject.Get());
			const UClass* InputObject = (Input.PinSubCategory == PSC_Self) ? CallingContext : Cast<const UClass>(Input.PinSubCategoryObject.Get());
			if (OutputObject && InputObject)
			{
				return ExtendedIsChildOf(OutputObject ,InputObject);
			}
		}
		else if ((Output.PinCategory == PC_Object) || (Output.PinCategory == PC_Struct) || (Output.PinCategory == PC_Class))
		{
			// Subcategory mismatch, but the two could be castable
			// Only allow a match if the input is a superclass of the output
			UStruct const* OutputObject = (Output.PinSubCategory == PSC_Self) ? CallingContext : Cast<UStruct>(Output.PinSubCategoryObject.Get());
			UStruct const* InputObject  = (Input.PinSubCategory == PSC_Self)  ? CallingContext : Cast<UStruct>(Input.PinSubCategoryObject.Get());

			if (OutputObject && InputObject)
			{
				if (Output.PinCategory == PC_Struct)
				{
					return OutputObject->IsChildOf(InputObject) && FStructUtils::TheSameLayout(OutputObject, InputObject);
				}

				// Special Case:  Cannot mix interface and non-interface calls, because the pointer size is different under the hood
				const bool bInputIsInterface  = InputObject->IsChildOf(UInterface::StaticClass());
				const bool bOutputIsInterface = OutputObject->IsChildOf(UInterface::StaticClass());

				UClass const* OutputClass = Cast<const UClass>(OutputObject);
				UClass const* InputClass = Cast<const UClass>(InputObject);

				if (bInputIsInterface != bOutputIsInterface) 
				{
					if (bInputIsInterface && (OutputClass != NULL))
					{
						return ExtendedImplementsInterface(OutputClass, InputClass);
					}
					else if (bOutputIsInterface && (InputClass != NULL))
					{
						return ExtendedImplementsInterface(InputClass, OutputClass);
					}
				}				

				return (IsAuthoritativeChildOf(OutputObject, InputObject) || (OutputClass && InputClass && ExtendedIsChildOf(OutputClass, InputClass)))
					&& (bInputIsInterface == bOutputIsInterface);
			}
		}
		else if ((Output.PinCategory == PC_Byte) && (Output.PinSubCategory == Input.PinSubCategory))
		{
			// NOTE: This allows enums to be converted to bytes.  Long-term we don't want to allow that, but we need it
			// for now until we have == for enums in order to be able to compare them.
			if (Input.PinSubCategoryObject == NULL)
			{
				return true;
			}
		}
		else if (PC_Byte == Output.PinCategory || PC_Int == Output.PinCategory)
		{
			// Bitmask integral types are compatible with non-bitmask integral types (of the same word size).
			const FString PSC_Bitmask_Str = PSC_Bitmask.ToString();
			return Output.PinSubCategory.ToString().StartsWith(PSC_Bitmask_Str) || Input.PinSubCategory.ToString().StartsWith(PSC_Bitmask_Str);
		}
		else if (PC_Delegate == Output.PinCategory || PC_MCDelegate == Output.PinCategory)
		{
			auto CanUseFunction = [](const UFunction* Func) -> bool
			{
				return Func && (Func->HasAllFlags(RF_LoadCompleted) || !Func->HasAnyFlags(RF_NeedLoad | RF_WasLoaded));
			};

			const UFunction* OutFunction = FMemberReference::ResolveSimpleMemberReference<UFunction>(Output.PinSubCategoryMemberReference);
			if (!CanUseFunction(OutFunction))
			{
				OutFunction = NULL;
			}
			if (!OutFunction && Output.PinSubCategoryMemberReference.GetMemberParentClass())
			{
				const UClass* ParentClass = Output.PinSubCategoryMemberReference.GetMemberParentClass();
				const UBlueprint* BPOwner = Cast<UBlueprint>(ParentClass->ClassGeneratedBy);
				if (BPOwner && BPOwner->SkeletonGeneratedClass && (BPOwner->SkeletonGeneratedClass != ParentClass))
				{
					OutFunction = BPOwner->SkeletonGeneratedClass->FindFunctionByName(Output.PinSubCategoryMemberReference.MemberName);
				}
			}
			const UFunction* InFunction = FMemberReference::ResolveSimpleMemberReference<UFunction>(Input.PinSubCategoryMemberReference);
			if (!CanUseFunction(InFunction))
			{
				InFunction = NULL;
			}
			if (!InFunction && Input.PinSubCategoryMemberReference.GetMemberParentClass())
			{
				const UClass* ParentClass = Input.PinSubCategoryMemberReference.GetMemberParentClass();
				const UBlueprint* BPOwner = Cast<UBlueprint>(ParentClass->ClassGeneratedBy);
				if (BPOwner && BPOwner->SkeletonGeneratedClass && (BPOwner->SkeletonGeneratedClass != ParentClass))
				{
					InFunction = BPOwner->SkeletonGeneratedClass->FindFunctionByName(Input.PinSubCategoryMemberReference.MemberName);
				}
			}
			return !OutFunction || !InFunction || OutFunction->IsSignatureCompatibleWith(InFunction);
		}
	}
	else if (Output.PinCategory == PC_Wildcard || Input.PinCategory == PC_Wildcard)
	{
		// If this is an Index Wildcard we have to check compatibility for indexing types
		if (Output.PinSubCategory == PSC_Index)
		{
			return IsIndexWildcardCompatible(Input);
		}
		else if (Input.PinSubCategory == PSC_Index)
		{
			return IsIndexWildcardCompatible(Output);
		}

		return true;
	}
	else if ((Output.PinCategory == PC_Object) && (Input.PinCategory == PC_Interface))
	{
		UClass const* OutputClass    = Cast<UClass const>(Output.PinSubCategoryObject.Get());
		UClass const* InterfaceClass = Cast<UClass const>(Input.PinSubCategoryObject.Get());

		if ((OutputClass == nullptr) && (Output.PinSubCategory == PSC_Self))
		{
			OutputClass = CallingContext;
		}

		return OutputClass && (ExtendedImplementsInterface(OutputClass, InterfaceClass) || ExtendedIsChildOf(OutputClass, InterfaceClass));
	}

	return false;
}

bool UEdGraphSchema_K2::ArePinTypesEquivalent(const FEdGraphPinType& PinA, const FEdGraphPinType& PinB) const
{
	// Real pins are effectively equivalent since we implicitly cast where necessary.
	if ((PinA.PinCategory == PC_Real) && (PinB.PinCategory == PC_Real))
	{
		return true;
	}

	return 
		PinA.PinCategory == PinB.PinCategory &&
		PinA.PinSubCategory == PinB.PinSubCategory &&
		PinA.PinSubCategoryObject == PinB.PinSubCategoryObject &&
		PinA.ContainerType == PinB.ContainerType &&
		PinA.bIsWeakPointer == PinB.bIsWeakPointer;
}

void UEdGraphSchema_K2::BreakNodeLinks(UEdGraphNode& TargetNode) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(&TargetNode);
	ensureMsgf(Blueprint != nullptr, TEXT("Node %s does not belong to a blueprint!"), *GetFullNameSafe(&TargetNode));

	Super::BreakNodeLinks(TargetNode);
	
	if (Blueprint != nullptr)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void UEdGraphSchema_K2::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakPinLinks", "Break Pin Links"), UEdGraphSchemaImpl::ShouldActuallyTransact());

	// cache this here, as BreakPinLinks can trigger a node reconstruction invalidating the TargetPin referenceS
	UBlueprint* const Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(TargetPin.GetOwningNode());

	Super::BreakPinLinks(TargetPin, bSendsNodeNotifcation);

	if (Blueprint != nullptr)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void UEdGraphSchema_K2::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "GraphEd_BreakSinglePinLink", "Break Pin Link"), UEdGraphSchemaImpl::ShouldActuallyTransact());

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(TargetPin->GetOwningNode());

	Super::BreakSinglePinLink(SourcePin, TargetPin);

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void UEdGraphSchema_K2::ReconstructNode(UEdGraphNode& TargetNode, bool bIsBatchRequest/*=false*/) const
{
	Super::ReconstructNode(TargetNode, bIsBatchRequest);

	// If the reconstruction is being handled by something doing a batch (i.e. the blueprint autoregenerating itself), defer marking the blueprint as modified to prevent multiple recompiles
	if (!bIsBatchRequest)
	{
		const UK2Node* K2Node = Cast<UK2Node>(&TargetNode);
		if (K2Node && K2Node->NodeCausesStructuralBlueprintChange())
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(&TargetNode);
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
}

bool UEdGraphSchema_K2::CanEncapuslateNode(UEdGraphNode const& TestNode) const
{
	// Can't encapsulate entry points (may relax this restriction in the future, but it makes sense for now)
	return !TestNode.IsA(UK2Node_FunctionTerminator::StaticClass()) && 
			TestNode.GetClass() != UK2Node_Tunnel::StaticClass(); //Tunnel nodes getting sucked into collapsed graphs fails badly, want to allow derived types though(composite node/Macroinstances)
}

void UEdGraphSchema_K2::HandleGraphBeingDeleted(UEdGraph& GraphBeingRemoved) const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&GraphBeingRemoved))
	{
		// Look for collapsed graph nodes that reference this graph
		TArray<UK2Node_Composite*> CompositeNodes;
		FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Composite>(Blueprint, /*out*/ CompositeNodes);

		TSet<UK2Node_Composite*> NodesToDelete;
		for (int32 i = 0; i < CompositeNodes.Num(); ++i)
		{
			UK2Node_Composite* CompositeNode = CompositeNodes[i];
			if (CompositeNode->BoundGraph == &GraphBeingRemoved)
			{
				NodesToDelete.Add(CompositeNode);
			}
		}

		// Delete the node that owns us
		ensure(NodesToDelete.Num() <= 1);
		for (TSet<UK2Node_Composite*>::TIterator It(NodesToDelete); It; ++It)
		{
			UK2Node_Composite* NodeToDelete = *It;

			// Prevent re-entrancy here
			NodeToDelete->BoundGraph = NULL;

			NodeToDelete->Modify();
			NodeToDelete->DestroyNode();
		}
		
		// likely tagged as modified by caller, but make sure:
		Blueprint->Modify();

		// Remove from the list of recently edited documents
		Blueprint->LastEditedDocuments.RemoveAll([&GraphBeingRemoved](const FEditedDocumentInfo& TestDoc) { return TestDoc.EditedObjectPath.ResolveObject() == &GraphBeingRemoved; });

		// Remove any BPs that reference a node in this graph:
		FKismetDebugUtilities::RemoveBreakpointsByPredicate(
			Blueprint,
			[&GraphBeingRemoved](const FBlueprintBreakpoint& Breakpoint)
			{
				return (Breakpoint.GetLocation() && Breakpoint.GetLocation()->IsIn(&GraphBeingRemoved));
			}
		);
	}
}

void UEdGraphSchema_K2::GetPinDefaultValuesFromString(const FEdGraphPinType& PinType, UObject* OwningObject, const FString& NewDefaultValue, FString& UseDefaultValue, TObjectPtr<UObject>& UseDefaultObject, FText& UseDefaultText, bool bPreserveTextIdentity) const
{
	if ((PinType.PinCategory == PC_Object)
		|| (PinType.PinCategory == PC_Class)
		|| (PinType.PinCategory == PC_Interface))
	{
		FString ObjectPathLocal = NewDefaultValue;
		ConstructorHelpers::StripObjectClass(ObjectPathLocal);

		// If this is not a full object path it's a relative path so should be saved as a string
		if (FPackageName::IsValidObjectPath(ObjectPathLocal))
		{
			FSoftObjectPath AssetRef = ObjectPathLocal;
			UseDefaultValue.Empty();
			// @todo: why are we resolving here? We should resolve explicitly 
			// during load or not at all
			if(!GCompilingBlueprint)
			{
				UseDefaultObject = AssetRef.TryLoad();
			}
			else
			{
				UseDefaultObject = AssetRef.ResolveObject();
			}
			UseDefaultText = FText::GetEmpty();
		}
		else
		{
			// "None" should be saved as empty string
			if (ObjectPathLocal == TEXT("None"))
			{
				ObjectPathLocal.Empty();
			}

			UseDefaultValue = MoveTemp(ObjectPathLocal);
			UseDefaultObject = nullptr;
			UseDefaultText = FText::GetEmpty();
		}
	}
	else if (PinType.PinCategory == PC_Text)
	{
		if (bPreserveTextIdentity)
		{
			UseDefaultText = FTextStringHelper::CreateFromBuffer(*NewDefaultValue);
		}
		else
		{
			FString PackageNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
			if (GIsEditor)
			{
				PackageNamespace = TextNamespaceUtil::EnsurePackageNamespace(OwningObject);
			}
#endif // USE_STABLE_LOCALIZATION_KEYS
			UseDefaultText = FTextStringHelper::CreateFromBuffer(*NewDefaultValue, nullptr, *PackageNamespace);
		}
		UseDefaultObject = nullptr;
		UseDefaultValue.Empty();
	}
	else
	{
		UseDefaultValue = NewDefaultValue;
		UseDefaultObject = nullptr;
		UseDefaultText = FText::GetEmpty();

		if (PinType.PinCategory == PC_Byte && UseDefaultValue.IsEmpty())
		{
			UEnum* EnumPtr = Cast<UEnum>(PinType.PinSubCategoryObject.Get());
			if (EnumPtr)
			{
				// Enums are stored as empty string in autogenerated defaults, but should turn into the first value in array 
				UseDefaultValue = EnumPtr->GetNameStringByIndex(0);
			}
		}
		else if ((PinType.PinCategory == PC_SoftObject) || (PinType.PinCategory == PC_SoftClass))
		{
			if (UseDefaultValue == FName(NAME_None).ToString())
			{
				UseDefaultValue.Reset();
			}
			else
			{
				ConstructorHelpers::StripObjectClass(UseDefaultValue);
			}
		}
	}
}

void UEdGraphSchema_K2::TrySetDefaultValue(UEdGraphPin& Pin, const FString& NewDefaultValue, bool bMarkAsModified) const
{
	FString UseDefaultValue;
	TObjectPtr<UObject> UseDefaultObject = nullptr;
	FText UseDefaultText;

	GetPinDefaultValuesFromString(Pin.PinType, Pin.GetOwningNodeUnchecked(), NewDefaultValue, UseDefaultValue, UseDefaultObject, UseDefaultText, /*bPreserveTextIdentity*/false);

	// Check the default value and make it an error if it's bogus
	if (IsPinDefaultValid(&Pin, UseDefaultValue, UseDefaultObject, UseDefaultText).IsEmpty())
	{
		Pin.DefaultObject = UseDefaultObject;
		Pin.DefaultValue = UseDefaultValue;
		Pin.DefaultTextValue = UseDefaultText;

		// Legacy float data will continue to serialize as a single precision float until we explicitly change the default value
		if (Pin.PinType.PinCategory == PC_Real)
		{
			Pin.PinType.bSerializeAsSinglePrecisionFloat = false;
		}

		UEdGraphNode* Node = Pin.GetOwningNode();
		Node->PinDefaultValueChanged(&Pin);

		// If the default value is manually set then treat it as if the value was reset to default and remove the orphaned pin
		if (Pin.bOrphanedPin && Pin.DoesDefaultValueMatchAutogenerated())
		{
			Node->PinConnectionListChanged(&Pin);
		}

		if (bMarkAsModified)
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Node);
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

void UEdGraphSchema_K2::TrySetDefaultObject(UEdGraphPin& Pin, UObject* NewDefaultObject, bool bMarkAsModified) const
{
	FText UseDefaultText;

	if ((Pin.PinType.PinCategory == PC_SoftObject) || (Pin.PinType.PinCategory == PC_SoftClass))
	{
		TrySetDefaultValue(Pin, NewDefaultObject ? NewDefaultObject->GetPathName() : FString());
		return;
	}

	// Check the default value and make it an error if it's bogus
	if (IsPinDefaultValid(&Pin, FString(), NewDefaultObject, UseDefaultText).IsEmpty())
	{
		Pin.DefaultObject = NewDefaultObject;
		Pin.DefaultValue.Empty();
		Pin.DefaultTextValue = UseDefaultText;
	}

	UEdGraphNode* Node = Pin.GetOwningNode();
	Node->PinDefaultValueChanged(&Pin);

	// If the default value is manually set then treat it as if the value was reset to default and remove the orphaned pin
	if (Pin.bOrphanedPin && Pin.DoesDefaultValueMatchAutogenerated())
	{
		Node->PinConnectionListChanged(&Pin);
	}

	if (bMarkAsModified)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Node);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void UEdGraphSchema_K2::TrySetDefaultText(UEdGraphPin& InPin, const FText& InNewDefaultText, bool bMarkAsModified) const
{
	// No reason to set the FText if it is not a PC_Text.
	if(InPin.PinType.PinCategory == PC_Text)
	{
		// Check the default value and make it an error if it's bogus
		if (IsPinDefaultValid(&InPin, FString(), nullptr, InNewDefaultText).IsEmpty())
		{
			InPin.DefaultObject = nullptr;
			InPin.DefaultValue.Empty();
			InPin.DefaultTextValue = InNewDefaultText;
		}

		UEdGraphNode* Node = InPin.GetOwningNode();
		Node->PinDefaultValueChanged(&InPin);

		// If the default value is manually set then treat it as if the value was reset to default and remove the orphaned pin
		if (InPin.bOrphanedPin && InPin.DoesDefaultValueMatchAutogenerated())
		{
			Node->PinConnectionListChanged(&InPin);
		}

		if (bMarkAsModified)
		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(Node);
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

bool UEdGraphSchema_K2::DoesDefaultValueMatchAutogenerated(const UEdGraphPin& InPin) const
{
	if (InPin.PinType.PinCategory == PC_Enum || InPin.PinType.PinCategory == PC_Byte)
	{
		// The autogenerated default value for an enum is left empty in case the default enum value (index 0) changes, in this case we
		// want to validate against the actual value of the 0 index entry
		if (InPin.AutogeneratedDefaultValue.IsEmpty())
		{
			const FString PinDefaultValue = InPin.GetDefaultAsString();
			if (PinDefaultValue.IsEmpty())
			{
				return true;
			}
			else if (UEnum* PinEnumType = Cast<UEnum>(InPin.PinType.PinSubCategoryObject.Get()))
			{
				return InPin.DefaultValue.Equals(PinEnumType->GetNameStringByIndex(0), ESearchCase::IgnoreCase);
			}
			else if (!InPin.bUseBackwardsCompatForEmptyAutogeneratedValue && InPin.PinType.PinCategory == PC_Byte && FCString::Atoi(*PinDefaultValue) == 0)
			{
				return true;
			}
		}
	}
	else if (!InPin.bUseBackwardsCompatForEmptyAutogeneratedValue)
	{
		if (InPin.PinType.PinCategory == PC_Real)
		{
			const double AutogeneratedDouble = FCString::Atod(*InPin.AutogeneratedDefaultValue);
			const double DefaultDouble = FCString::Atod(*InPin.DefaultValue);
			return (AutogeneratedDouble == DefaultDouble);
		}
		else if (InPin.PinType.PinCategory == PC_Struct)
		{
			if ((InPin.PinType.PinSubCategoryObject == VectorStruct) || (InPin.PinType.PinSubCategoryObject == Vector3fStruct))
			{
				FVector AutogeneratedVector = FVector::ZeroVector;
				FVector DefaultVector = FVector::ZeroVector;
				FDefaultValueHelper::ParseVector(InPin.AutogeneratedDefaultValue, AutogeneratedVector);
				FDefaultValueHelper::ParseVector(InPin.DefaultValue, DefaultVector);
				return (AutogeneratedVector == DefaultVector);
			}
			else if (InPin.PinType.PinSubCategoryObject == RotatorStruct)
			{
				FRotator AutogeneratedRotator = FRotator::ZeroRotator;
				FRotator DefaultRotator = FRotator::ZeroRotator;
				FDefaultValueHelper::ParseRotator(InPin.AutogeneratedDefaultValue, AutogeneratedRotator);
				FDefaultValueHelper::ParseRotator(InPin.DefaultValue, DefaultRotator);
				return (AutogeneratedRotator == DefaultRotator);
			}
		}
		else if (InPin.AutogeneratedDefaultValue.IsEmpty())
		{
			if (InPin.IsDefaultAsStringEmpty())
			{
				return true;
			}
			else if (InPin.PinType.PinCategory == PC_Boolean)
			{
				const FString PinDefaultValue = InPin.GetDefaultAsString();
				return (PinDefaultValue == TEXT("false"));
			}
			else if (InPin.PinType.PinCategory == PC_Int)
			{
				const FString PinDefaultValue = InPin.GetDefaultAsString();
				if (FCString::Atoi(*PinDefaultValue) == 0)
				{
					return true;
				}
			}
			else if (InPin.PinType.PinCategory == PC_Int64)
			{
				const FString PinDefaultValue = InPin.GetDefaultAsString();
				if (FCString::Atoi64(*PinDefaultValue) == 0)
				{
					return true;
				}
			}
			else if (InPin.PinType.PinCategory == PC_Name)
			{
				const FString PinDefaultValue = InPin.GetDefaultAsString();
				return (PinDefaultValue == TEXT("None"));
			}
		}
	}

	return Super::DoesDefaultValueMatchAutogenerated(InPin);
}

bool UEdGraphSchema_K2::IsAutoCreateRefTerm(const UEdGraphPin* Pin)
{
	check(Pin);

	bool bIsAutoCreateRefTerm = false;
	if (!Pin->PinName.IsNone())
	{
		if (UK2Node_CallFunction* FuncNode = Cast<UK2Node_CallFunction>(Pin->GetOwningNode()))
		{
			if (UFunction* TargetFunction = FuncNode->GetTargetFunction())
			{
				static TArray<FString> AutoCreateParameterNames;
				GetAutoEmitTermParameters(TargetFunction, AutoCreateParameterNames);
				bIsAutoCreateRefTerm = AutoCreateParameterNames.Contains(Pin->PinName.ToString());
			}
		}
	}

	return bIsAutoCreateRefTerm;
}

bool UEdGraphSchema_K2::ShouldHidePinDefaultValue(UEdGraphPin* Pin) const
{
	check(Pin != NULL);

	if (Pin->bDefaultValueIsIgnored || Pin->PinType.IsContainer() || (Pin->PinName == PN_Self && Pin->LinkedTo.Num() > 0) || (Pin->PinType.PinCategory == PC_Exec) || (Pin->PinType.bIsReference && !IsAutoCreateRefTerm(Pin)))
	{
		return true;
	}

	return false;
}

bool UEdGraphSchema_K2::ShouldShowAssetPickerForPin(UEdGraphPin* Pin) const
{
	bool bShow = true;
	if (Pin->PinType.PinCategory == PC_Object)
	{
		UClass* ObjectClass = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get());
		if (ObjectClass)
		{
			// Don't show literal buttons for component type objects
			bShow = !ObjectClass->IsChildOf(UActorComponent::StaticClass());

			if (bShow && ObjectClass->IsChildOf(AActor::StaticClass()))
			{
				// Only show the picker for Actor classes if the class is placeable and we are in the level script
				bShow = !ObjectClass->HasAllClassFlags(CLASS_NotPlaceable)
							&& FBlueprintEditorUtils::IsLevelScriptBlueprint(FBlueprintEditorUtils::FindBlueprintForNode(Pin->GetOwningNode()));
			}

			if (bShow)
			{
				if (UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(Pin->GetOwningNode()))
				{
					if (UFunction* FunctionRef = CallFunctionNode->GetTargetFunction())
					{
						const UEdGraphPin* WorldContextPin = CallFunctionNode->FindPin(FunctionRef->GetMetaData(FBlueprintMetadata::MD_WorldContext));
						bShow = (WorldContextPin != Pin);

						// Check if we have explictly marked this pin as hiding the asset picker
						FString HideAssetPickerMetaData = FunctionRef->GetMetaData(FBlueprintMetadata::MD_HidePinAssetPicker);
						if (HideAssetPickerMetaData.IsEmpty())
						{
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							HideAssetPickerMetaData = FunctionRef->GetMetaData(FBlueprintMetadata::MD_HideAssetPicker);
							PRAGMA_ENABLE_DEPRECATION_WARNINGS
							if (!HideAssetPickerMetaData.IsEmpty())
							{
								UE_LOGF(LogBlueprint, Warning, "Use of deprecated meta 'HideAssetPicker' for function %ls. Change to HidePinAssetPicker.", *FunctionRef->GetPathName());
							}
						}

						if (!HideAssetPickerMetaData.IsEmpty())
						{
							TArray<FString> PinNames;
							HideAssetPickerMetaData.ParseIntoArray(PinNames, TEXT(","), true);
							const FString PinName = Pin->GetName();
							for (FString& ParamNameToHide : PinNames)
							{
								ParamNameToHide.TrimStartAndEndInline();
								if (ParamNameToHide == PinName)
								{
									bShow = false;
									break;
								}
							}
						}
					}
				}
				else if (Cast<UK2Node_CreateDelegate>(Pin->GetOwningNode())) 
				{
					bShow = false;
				}

				const FString& Meta = bShow ? Pin->GetOwningNode()->GetPinMetaData(Pin->PinName, FBlueprintMetadata::MD_HidePinAssetPicker) : FString();
				if (!Meta.IsEmpty())
				{
					bShow = !(Meta.Compare("true", ESearchCase::IgnoreCase) == 0);
				}
			}
		}
	}
	return bShow;
}

bool UEdGraphSchema_K2::FindFunctionParameterDefaultValue(const UFunction* Function, const FProperty* Param, FString& OutString)
{
	bool bHasAutomaticValue = false;

	const FString& MetadataDefaultValue = Function->GetMetaData(*Param->GetName());
	if (!MetadataDefaultValue.IsEmpty())
	{
		// Specified default value in the metadata
		OutString = MetadataDefaultValue;
		bHasAutomaticValue = true;

		// If the parameter is a class then try and get the full name as the metadata might just be the short name
		if (Param->IsA<FClassProperty>() && !FPackageName::IsValidObjectPath(OutString))
		{
			UE_LOGF(LogBlueprint, Warning, "Short class name \"%ls\" in meta data \"%ls\" for function %ls", *OutString, *Param->GetName(), *Function->GetPathName());
			if (UClass* DefaultClass = FindFirstObject<UClass>(*OutString, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("UEdGraphSchema_K2::FindFunctionParameterDefaultValue")))
			{
				OutString = DefaultClass->GetPathName();
			}
		}
	}
	else
	{
		const FName MetadataCppDefaultValueKey(*(FString(TEXT("CPP_Default_")) + Param->GetName()));
		const FString& MetadataCppDefaultValue = Function->GetMetaData(MetadataCppDefaultValueKey);
		if (!MetadataCppDefaultValue.IsEmpty())
		{
			OutString = MetadataCppDefaultValue;
			bHasAutomaticValue = true;
		}
	}

	return bHasAutomaticValue;
}

void UEdGraphSchema_K2::SetPinAutogeneratedDefaultValue(UEdGraphPin* Pin, const FString& NewValue) const
{
	Pin->AutogeneratedDefaultValue = NewValue;

	ResetPinToAutogeneratedDefaultValue(Pin, false);
}

void UEdGraphSchema_K2::SetPinAutogeneratedDefaultValueBasedOnType(UEdGraphPin* Pin) const
{
	FString NewValue;

	// Create a useful default value based on the pin type
	if(Pin->PinType.IsContainer() )
	{
		NewValue = FString();
	}
	else if (Pin->PinType.PinCategory == PC_Int)
	{
		NewValue = TEXT("0");
	}
	else if (Pin->PinType.PinCategory == PC_Int64)
	{
		NewValue = TEXT("0");
	}
	else if (Pin->PinType.PinCategory == PC_Byte)
	{
		UEnum* EnumPtr = Cast<UEnum>(Pin->PinType.PinSubCategoryObject.Get());
		if(EnumPtr)
		{
			// First element of enum can change. If the enum is { A, B, C } and the default value is A, 
			// the defult value should not change when enum will be changed into { N, A, B, C }
			NewValue = FString();
		}
		else
		{
			NewValue = TEXT("0");
		}
	}
	else if (Pin->PinType.PinCategory == PC_Real)
	{
		NewValue = TEXT("0.0");
	}
	else if (Pin->PinType.PinCategory == PC_Boolean)
	{
		NewValue = TEXT("false");
	}
	else if (Pin->PinType.PinCategory == PC_Name)
	{
		NewValue = TEXT("None");
	}
	else if ((Pin->PinType.PinCategory == PC_Struct) && ((Pin->PinType.PinSubCategoryObject == VectorStruct) || (Pin->PinType.PinSubCategoryObject == Vector3fStruct) || (Pin->PinType.PinSubCategoryObject == RotatorStruct)))
	{
		// This is a slightly different format than is produced by PropertyValueToString, but changing it has backward compatibility issues
		NewValue = TEXT("0, 0, 0");
	}

	// PropertyValueToString also has cases for LinerColor and Transform, LinearColor is identical to export text so is fine, the Transform case is specially handled in the vm

	SetPinAutogeneratedDefaultValue(Pin, NewValue);
}

static void ConformAutogeneratedDefaultValuePackage(
	const FEdGraphPinType& PinType, 
	const UEdGraphNode* OwningNode, 
	FString& AutogeneratedDefaultValue, 
	const FText& DefaultTextValue
)
{
	if (PinType.PinCategory == UEdGraphSchema_K2::PC_Text &&
		!PinType.IsContainer() &&
		!AutogeneratedDefaultValue.IsEmpty())
	{
		// Attempt to find the correct package namespace to use for text within this pin
		// Favor using the node if we have it, as that will be most up-to-date
		FString PackageNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
		if (OwningNode)
		{
			PackageNamespace = TextNamespaceUtil::GetPackageNamespace(OwningNode);
		}
		else if (!DefaultTextValue.IsEmpty())
		{
			PackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(FTextInspector::GetNamespace(DefaultTextValue).Get(FString()));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		if (!PackageNamespace.IsEmpty())
		{
			FText AutogeneratedDefaultTextValue;
			const TCHAR* Success = FTextStringHelper::ReadFromBuffer(*AutogeneratedDefaultValue, AutogeneratedDefaultTextValue);
			check(Success);

			// Does auto-generated default text have a mismatched package ID?
			if (!AutogeneratedDefaultTextValue.IsEmpty() && !AutogeneratedDefaultTextValue.IsCultureInvariant() && !AutogeneratedDefaultTextValue.IsFromStringTable())
			{
				if (const FString CurrentPackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(FTextInspector::GetNamespace(AutogeneratedDefaultTextValue).Get(FString()));
					!CurrentPackageNamespace.IsEmpty() && !CurrentPackageNamespace.Equals(PackageNamespace, ESearchCase::CaseSensitive))
				{
					// Conform the auto-generated default against this package ID, preserving its key to avoid determinism issues
					AutogeneratedDefaultTextValue = TextNamespaceUtil::CopyTextToPackage(AutogeneratedDefaultTextValue, PackageNamespace, TextNamespaceUtil::ETextCopyMethod::PreserveKey);
					// Fix-up the auto-generated default from the conformed value
					AutogeneratedDefaultValue.Reset();
					FTextStringHelper::WriteToBuffer(AutogeneratedDefaultValue, AutogeneratedDefaultTextValue);
				}
			}
		}
	}
}

void UEdGraphSchema_K2::ResetPinToAutogeneratedDefaultValue(UEdGraphPin* Pin, bool bCallModifyCallbacks) const
{
	if (Pin->bOrphanedPin)
	{
		UEdGraphNode* Node = Pin->GetOwningNode();
		Node->PinConnectionListChanged(Pin);
	}
	else
	{
		// Autogenerated value has unreliable package namespace for text value, hack fix it up now:
		ConformAutogeneratedDefaultValuePackage(Pin->PinType, Pin->GetOwningNodeUnchecked(), Pin->AutogeneratedDefaultValue, Pin->DefaultTextValue);

		GetPinDefaultValuesFromString(Pin->PinType, Pin->GetOwningNodeUnchecked(), Pin->AutogeneratedDefaultValue, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue, false);

		if (bCallModifyCallbacks)
		{
			UEdGraphNode* Node = Pin->GetOwningNode();
			Node->PinDefaultValueChanged(Pin);

			if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Node))
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}
}

void UEdGraphSchema_K2::SetPinDefaultValueAtConstruction(UEdGraphPin* Pin, const FString& DefaultValueString) const
{
	GetPinDefaultValuesFromString(Pin->PinType, Pin->GetOwningNodeUnchecked(), DefaultValueString, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue);
}

void UEdGraphSchema_K2::ValidateExistingConnections(UEdGraphPin* Pin)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Pin->GetOwningNodeUnchecked());
	const UClass* CallingContext = Blueprint
		? ((Blueprint->GeneratedClass != nullptr) ? *Blueprint->GeneratedClass : *Blueprint->ParentClass)
		: nullptr;

	// Break any newly invalid links
	TArray<UEdGraphPin*> BrokenLinks;
	for (int32 Index = 0; Index < Pin->LinkedTo.Num();)
	{
		UEdGraphPin* OtherPin = Pin->LinkedTo[Index];
		if (K2Schema->ArePinsCompatible(Pin, OtherPin, CallingContext))
		{
			++Index;
		}
		else
		{
			OtherPin->LinkedTo.Remove(Pin);
			Pin->LinkedTo.RemoveAtSwap(Index);

			BrokenLinks.Add(OtherPin);
		}
	}

	// Cascade the check for changed pin types
	for (TArray<UEdGraphPin*>::TIterator PinIt(BrokenLinks); PinIt; ++PinIt)
	{
		UEdGraphPin* OtherPin = *PinIt;
		OtherPin->GetOwningNode()->PinConnectionListChanged(OtherPin);
	}
}

namespace FSetVariableByNameFunctionNames
{
	static const FName SetIntName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetIntPropertyByName));
	static const FName SetInt64Name(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetInt64PropertyByName));
	static const FName SetByteName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetBytePropertyByName));
	static const FName SetDoubleName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetDoublePropertyByName));
	static const FName SetBoolName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetBoolPropertyByName));
	static const FName SetObjectName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetObjectPropertyByName));
	static const FName SetClassName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetClassPropertyByName));
	static const FName SetInterfaceName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetInterfacePropertyByName));
	static const FName SetStringName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetStringPropertyByName));
	static const FName SetTextName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetTextPropertyByName));
	static const FName SetSoftObjectName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetSoftObjectPropertyByName));
	static const FName SetSoftClassName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetSoftClassPropertyByName));
	static const FName SetNameName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetNamePropertyByName));
	static const FName SetVectorName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetVectorPropertyByName));
	static const FName SetVector3fName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetVector3fPropertyByName));
	static const FName SetRotatorName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetRotatorPropertyByName));
	static const FName SetLinearColorName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetLinearColorPropertyByName));
	static const FName SetColorName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetColorPropertyByName));
	static const FName SetTransformName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetTransformPropertyByName));
	static const FName SetCollisionProfileName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetCollisionProfileNameProperty));
	static const FName SetStructureName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetStructurePropertyByName));
	static const FName SetArrayName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, SetArrayPropertyByName));
	static const FName SetSetName(GET_FUNCTION_NAME_CHECKED(UBlueprintSetLibrary, SetSetPropertyByName));
	static const FName SetMapName(GET_FUNCTION_NAME_CHECKED(UBlueprintMapLibrary, SetMapPropertyByName));
	static const FName SetFieldPathName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetFieldPathPropertyByName));
};

UFunction* UEdGraphSchema_K2::FindSetVariableByNameFunction(const FEdGraphPinType& PinType)
{
	//!!!! Keep this function synced with FExposeOnSpawnValidator::IsSupported and Uht*Property.cs, CanExposeOnSpawn!!!!

	struct FIsCustomStructureParamHelper
	{
		static bool Is(const UObject* Obj)
		{
			const UScriptStruct* Struct = Cast<const UScriptStruct>(Obj);
			return Struct ? Struct->GetBoolMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType) : false;
		}
	};

	UClass* SetFunctionLibraryClass = UKismetSystemLibrary::StaticClass();
	FName SetFunctionName = NAME_None;
	if (PinType.ContainerType == EPinContainerType::Array)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetArrayName;
		SetFunctionLibraryClass = UKismetArrayLibrary::StaticClass();
	}
	else if (PinType.ContainerType == EPinContainerType::Set)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetSetName;
		SetFunctionLibraryClass = UBlueprintSetLibrary::StaticClass();
	}
	else if (PinType.ContainerType == EPinContainerType::Map)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetMapName;
		SetFunctionLibraryClass = UBlueprintMapLibrary::StaticClass();
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetIntName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Int64)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetInt64Name;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetByteName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetDoubleName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetBoolName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetObjectName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetClassName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetInterfaceName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetStringName;
	}
	else if ( PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetTextName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetSoftObjectName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetSoftClassName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetNameName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == VectorStruct)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetVectorName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == Vector3fStruct)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetVector3fName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == RotatorStruct)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetRotatorName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == ColorStruct)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetColorName;
	}
	else if(PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == TransformStruct)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetTransformName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && PinType.PinSubCategoryObject == FCollisionProfileName::StaticStruct())
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetCollisionProfileName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && FIsCustomStructureParamHelper::Is(PinType.PinSubCategoryObject.Get()))
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetStructureName;
	}
	else if (PinType.PinCategory == UEdGraphSchema_K2::PC_FieldPath)
	{
		SetFunctionName = FSetVariableByNameFunctionNames::SetFieldPathName;
	}

	UFunction* Function = nullptr;
	if (!SetFunctionName.IsNone())
	{
		Function = SetFunctionLibraryClass->FindFunctionByName(SetFunctionName);
	}

	return Function;
}

bool UEdGraphSchema_K2::CanPromotePinToVariable( const UEdGraphPin& Pin, const bool bInToMemberVariable ) const
{
	const FEdGraphPinType& PinType = Pin.PinType;
	bool bCanPromote = (PinType.PinCategory != PC_Wildcard && PinType.PinCategory != PC_Exec ) ? true : false;

	const UK2Node* Node = Cast<UK2Node>(Pin.GetOwningNode());
	const UBlueprint* OwningBlueprint = Node->GetBlueprint();
	
	if (Pin.bNotConnectable)
	{
		bCanPromote = false;
	}
	else if (!OwningBlueprint || (OwningBlueprint->BlueprintType == BPTYPE_MacroLibrary) || (bInToMemberVariable && (OwningBlueprint->BlueprintType == BPTYPE_FunctionLibrary || IsStaticFunctionGraph(Node->GetGraph()))))
	{
		// Never allow promotion in macros, because there's not a scope to define them in
		bCanPromote = false;
	}
	else
	{
		if (PinType.PinCategory == PC_Delegate)
		{
			bCanPromote = false;
		}
		else if ((PinType.PinCategory == PC_Object) || (PinType.PinCategory == PC_Interface))
		{
			if (PinType.PinSubCategoryObject != NULL)
			{
				if (UClass* Class = Cast<UClass>(PinType.PinSubCategoryObject.Get()))
				{
					bCanPromote = UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Class);
				}	
			}
		}
		else if ((PinType.PinCategory == PC_Struct) && (PinType.PinSubCategoryObject != NULL))
		{
			if (UScriptStruct* Struct = Cast<UScriptStruct>(PinType.PinSubCategoryObject.Get()))
			{
				bCanPromote = UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Struct);
			}
		}
	}
	
	return bCanPromote;
}

bool UEdGraphSchema_K2::CanSplitStructPin( const UEdGraphPin& Pin ) const
{
	return Pin.GetOwningNode()->CanSplitPin(&Pin) && PinHasSplittableStructType(&Pin);
}

bool UEdGraphSchema_K2::CanRecombineStructPin( const UEdGraphPin& Pin ) const
{
	bool bCanRecombine = (Pin.ParentPin != NULL && Pin.LinkedTo.Num() == 0);
	if (bCanRecombine)
	{
		// Go through all the other subpins and ensure they also are not connected to anything
		TArray<UEdGraphPin*> PinsToExamine = Pin.ParentPin->SubPins;

		int32 PinIndex = 0;
		while (bCanRecombine && PinIndex < PinsToExamine.Num())
		{
			UEdGraphPin* SubPin = PinsToExamine[PinIndex];
			if (SubPin->LinkedTo.Num() > 0)
			{
				bCanRecombine = false;
			}
			else if (SubPin->SubPins.Num() > 0)
			{
				PinsToExamine.Append(SubPin->SubPins);
			}
			++PinIndex;
		}
	}

	return bCanRecombine;
}

void UEdGraphSchema_K2::GetGraphDisplayInformation(const UEdGraph& Graph, /*out*/ FGraphDisplayInfo& DisplayInfo) const
{
	DisplayInfo.DocLink = TEXT("Shared/Editors/BlueprintEditor/GraphTypes");
	DisplayInfo.PlainName = FText::FromString( Graph.GetName() ); // Fallback is graph name

	UFunction* Function = NULL;
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(&Graph);
	if (Blueprint && Blueprint->SkeletonGeneratedClass)
	{
		Function = Blueprint->SkeletonGeneratedClass->FindFunctionByName(Graph.GetFName());
	}

	const EGraphType GraphType = GetGraphType(&Graph);
	if (GraphType == GT_Ubergraph)
	{
		DisplayInfo.DocExcerptName = TEXT("EventGraph");

		if (Graph.GetFName() == GN_EventGraph)
		{
			// localized name for the first event graph
			DisplayInfo.PlainName = LOCTEXT("GraphDisplayName_EventGraph", "EventGraph");
			DisplayInfo.Tooltip = DisplayInfo.PlainName;
		}
		else
		{
			DisplayInfo.Tooltip = FText::FromString(Graph.GetName());
		}
	}
	else if (GraphType == GT_Function)
	{
		if ( Graph.GetFName() == FN_UserConstructionScript )
		{
			DisplayInfo.PlainName = LOCTEXT("GraphDisplayName_ConstructionScript", "ConstructionScript");

			DisplayInfo.Tooltip = LOCTEXT("GraphTooltip_ConstructionScript", "Function executed when Blueprint is placed or modified.");
			DisplayInfo.DocExcerptName = TEXT("ConstructionScript");
		}
		else
		{
			// If we found a function from this graph..
			if (Function)
			{
				DisplayInfo.PlainName = FText::FromString(Function->GetName());
				DisplayInfo.Tooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function)); // grab its tooltip
			}
			else
			{
				DisplayInfo.Tooltip = FText::FromString(Graph.GetName());
			}

			DisplayInfo.DocExcerptName = TEXT("FunctionGraph");
		}
	}
	else if (GraphType == GT_Macro)
	{
		// Show macro description if set
		FKismetUserDeclaredFunctionMetadata* MetaData = UK2Node_MacroInstance::GetAssociatedGraphMetadata(&Graph);
		DisplayInfo.Tooltip = (MetaData && !MetaData->ToolTip.IsEmpty()) ? MetaData->ToolTip : FText::FromString(Graph.GetName());

		DisplayInfo.DocExcerptName = TEXT("MacroGraph");
	}
	else if (GraphType == GT_StateMachine)
	{
		DisplayInfo.Tooltip = FText::FromString(Graph.GetName());
		DisplayInfo.DocExcerptName = TEXT("StateMachine");
	}

	// Add pure/static/const to notes if set
	if (Function)
	{
		if(Function->HasAnyFunctionFlags(FUNC_BlueprintPure))
		{
			DisplayInfo.Notes.Add(TEXT("pure"));
		}

		// since 'static' is implied in a function library, not going to display it (to be consistent with previous behavior)
		if(Function->HasAnyFunctionFlags(FUNC_Static) && Blueprint->BlueprintType != BPTYPE_FunctionLibrary)
		{
			DisplayInfo.Notes.Add(TEXT("static"));
		}
		else if(Function->HasAnyFunctionFlags(FUNC_Const))
		{
			DisplayInfo.Notes.Add(TEXT("const"));
		}

		if (Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction))
		{
			DisplayInfo.Notes.Add(LOCTEXT("FunctionGraphDisplayInfo_Deprecated", "deprecated").ToString());
		}
	}

	// Mark transient graphs as obviously so
	if (Graph.HasAllFlags(RF_Transient))
	{
		DisplayInfo.PlainName = FText::FromString( FString::Printf(TEXT("$$ %s $$"), *DisplayInfo.PlainName.ToString()) );
		DisplayInfo.Notes.Add(TEXT("intermediate build product"));
	}

	// We disallow friendly names so that the Function name appears as entered (unless localized)
	constexpr bool bAllowFriendlyNames = false;
	if (GraphType == GT_Function && Function)
	{
		DisplayInfo.DisplayName = ObjectTools::GetUserFacingFunctionName(Function, bAllowFriendlyNames);
	}
	else
	{
		// This is essentially GetUserFacingFunctionName but without having an actual UFunction to pull metadata from (no localization)
		const bool bShowFriendlyNames = bAllowFriendlyNames ? (GEditor && GetDefault<UEditorStyleSettings>()->bShowFriendlyNames) : false;
		if (bShowFriendlyNames)
		{
			DisplayInfo.DisplayName = FText::FromString(FName::NameToDisplayString(DisplayInfo.PlainName.ToString(), false));
		}
		else
		{
			DisplayInfo.DisplayName = DisplayInfo.PlainName;
		}
	}
}

bool UEdGraphSchema_K2::IsSelfPin(const UEdGraphPin& Pin) const 
{
	return (Pin.PinName == PN_Self);
}

bool UEdGraphSchema_K2::CanShowDataTooltipForPin(const UEdGraphPin& Pin) const
{
	return !IsExecPin(Pin) && !IsDelegateCategory(Pin.PinType.PinCategory);
}

bool UEdGraphSchema_K2::IsDelegateCategory(const FName Category) const
{
	return (Category == PC_Delegate);
}

FVector2D UEdGraphSchema_K2::CalculateAveragePositionBetweenNodes(UEdGraphPin* InputPin, UEdGraphPin* OutputPin)
{
	UEdGraphNode* InputNode = InputPin->GetOwningNode();
	UEdGraphNode* OutputNode = OutputPin->GetOwningNode();
	const FVector2D InputCorner(InputNode->NodePosX, InputNode->NodePosY);
	const FVector2D OutputCorner(OutputNode->NodePosX, OutputNode->NodePosY);
	
	return (InputCorner + OutputCorner) * 0.5f;
}

bool UEdGraphSchema_K2::IsConstructionScript(const UEdGraph* TestEdGraph)
{
	TArray<class UK2Node_FunctionEntry*> EntryNodes;
	TestEdGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);

	bool bIsConstructionScript = false;
	if (EntryNodes.Num() > 0)
	{
		UK2Node_FunctionEntry const* const EntryNode = EntryNodes[0];
		bIsConstructionScript = (EntryNode->FunctionReference.GetMemberName() == FN_UserConstructionScript);
	}
	return bIsConstructionScript;
}

bool UEdGraphSchema_K2::IsCompositeGraph( const UEdGraph* TestEdGraph ) const
{
	check(TestEdGraph);

	const EGraphType GraphType = GetGraphType(TestEdGraph);
	if(GraphType == GT_Function) 
	{
		//Find the Tunnel node for composite graph and see if its output is a composite node
		for (UEdGraphNode* Node : TestEdGraph->Nodes)
		{
			if (UK2Node_Tunnel* Tunnel = Cast<UK2Node_Tunnel>(Node))
			{
				if (UK2Node_Tunnel* OutNode = Tunnel->OutputSourceNode)
				{
					if (OutNode->IsA<UK2Node_Composite>())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

bool UEdGraphSchema_K2::IsConstFunctionGraph( const UEdGraph* TestEdGraph, bool* bOutIsEnforcingConstCorrectness ) const
{
	check(TestEdGraph);

	const EGraphType GraphType = GetGraphType(TestEdGraph);
	if(GraphType == GT_Function) 
	{
		// Find the entry node for the function graph and see if the 'const' flag is set
		for (UEdGraphNode* Node : TestEdGraph->Nodes)
		{
			if(UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(Node))
			{
				if(bOutIsEnforcingConstCorrectness != nullptr)
				{
					*bOutIsEnforcingConstCorrectness = EntryNode->bEnforceConstCorrectness;
				}

				return (EntryNode->GetFunctionFlags() & FUNC_Const) != 0;
			}
		}
	}

	if(bOutIsEnforcingConstCorrectness != nullptr)
	{
		*bOutIsEnforcingConstCorrectness = false;
	}

	return false;
}

bool UEdGraphSchema_K2::IsStaticFunctionGraph( const UEdGraph* TestEdGraph ) const
{
	check(TestEdGraph);

	const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TestEdGraph);
	if (Blueprint && (EBlueprintType::BPTYPE_FunctionLibrary == Blueprint->BlueprintType))
	{
		return true;
	}

	const EGraphType GraphType = GetGraphType(TestEdGraph);
	if(GraphType == GT_Function) 
	{
		// Find the entry node for the function graph and see if the 'static' flag is set
		for (UEdGraphNode* Node : TestEdGraph->Nodes)
		{
			if(UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(Node))
			{
				return (EntryNode->GetFunctionFlags() & FUNC_Static) != 0;
			}
		}
	}

	return false;
}

void UEdGraphSchema_K2::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const 
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Blueprint);

	// only want to spawn event nodes in an event graph
	if (FBlueprintEditorUtils::IsEventGraph(Graph))
	{
		const FBlueprintGraphModule& Module = FModuleManager::LoadModuleChecked<FBlueprintGraphModule>("BlueprintGraph");
		// check all assets to see if we can get some AssetBlueprintGraphActions for it 
		for (const FAssetData& AssetData : Assets)
		{
			// if we can find any actions we want to try to spawn the node - only spawning Input Action event nodes currently
			if (const FAssetBlueprintGraphActions* GraphActions = Module.GetAssetBlueprintGraphActions(AssetData.GetClass()))
			{
				if (!AssetReferenceFilter || AssetReferenceFilter->PassesFilter(AssetData))
				{
					GraphActions->TryCreatingAssetNode(AssetData, Graph, FDeprecateSlateVector2D(GraphPosition), EK2NewNodeFlags::SelectNewNode);
				}
			}
		}
	}

	if ((Blueprint != nullptr) && FBlueprintEditorUtils::IsActorBased(Blueprint))
	{
		float XOffset = 0.0f;
		for(int32 AssetIdx=0; AssetIdx < Assets.Num(); AssetIdx++)
		{
			FVector2f Position = GraphPosition + (AssetIdx * FVector2f(XOffset, 0.0f));

			UObject* Asset = Assets[AssetIdx].GetAsset();
			if (!Asset)
			{
				UE_LOGF(LogBlueprint, Warning, "Failed to load Asset '%ls'.", *Assets[AssetIdx].GetFullName());
				continue;
			}

			UClass* AssetClass = Asset->GetClass();		
			if (UBlueprint* BlueprintAsset = Cast<UBlueprint>(Asset))
			{
				AssetClass = BlueprintAsset->GeneratedClass;
			}

			TSubclassOf<UActorComponent> DestinationComponentType;
			if (AssetClass && AssetClass->IsChildOf(UActorComponent::StaticClass()) && IsAllowableBlueprintVariableType(AssetClass))
			{
				// If it's an actor component subclass that is a BlueprintableComponent, we're good to go
				DestinationComponentType = AssetClass;
			}
			else
			{
				// Otherwise see if we can factory a component from the asset
				DestinationComponentType = FComponentAssetBrokerage::GetPrimaryComponentForAsset(AssetClass);
				if ((DestinationComponentType == nullptr) && AssetClass && AssetClass->IsChildOf(AActor::StaticClass()))
				{
					DestinationComponentType = UChildActorComponent::StaticClass();
				}
			}

			// Make sure we have an asset type that's registered with the component list
			if (DestinationComponentType != nullptr)
			{
				if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(Assets[AssetIdx]))
				{
					continue;
				}

				const FScopedTransaction Transaction(LOCTEXT("CreateAddComponentFromAsset", "Add Component From Asset"), UEdGraphSchemaImpl::ShouldActuallyTransact());

				FComponentTypeEntry ComponentType = { FString(), FString(), DestinationComponentType.GetGCPtr() };
				if (const UBlueprintComponentNodeSpawner* Spawner = UBlueprintComponentNodeSpawner::Create(ComponentType))
				{
					IBlueprintNodeBinder::FBindingSet Bindings;
					Bindings.Add(Asset);
					Spawner->Invoke(Graph, Bindings, FDeprecateSlateVector2D(GraphPosition));
				}
				else
				{
					UE_LOGF(LogBlueprint, Warning, "Component Node could not be created for dropped Asset '%ls'.", *Asset->GetName());
				}
			}
		}
	}
}

void UEdGraphSchema_K2::DroppedAssetsOnNode(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphNode* Node) const
{
	// @TODO: Should dropping on component node change the component?
}

void UEdGraphSchema_K2::DroppedAssetsOnPin(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraphPin* Pin) const
{
	// If dropping onto an 'object' pin, try and set the literal
	if ((Pin->PinType.PinCategory == PC_Object) || (Pin->PinType.PinCategory == PC_Interface))
	{
		UClass* PinClass = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get());
		if(PinClass != nullptr)
		{
			// Find first asset of type of the pin
			FAssetData AssetData = FAssetData::GetFirstAssetDataOfClass(Assets, PinClass);
			if (AssetData.IsValid())
			{
				TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(FBlueprintEditorUtils::FindBlueprintForNode(Pin->GetOwningNode()));
				if (!AssetReferenceFilter || AssetReferenceFilter->PassesFilter(AssetData))
				{
					UObject* Asset = AssetData.GetAsset();
					if (Asset != nullptr)
					{
						TrySetDefaultObject(*Pin, Asset);
					}
				}
			}
		}
	}
}

void UEdGraphSchema_K2::GetAssetsPinHoverMessage(const TArray<FAssetData>& Assets, const UEdGraphPin* HoverPin, FString& OutTooltipText, bool& OutOkIcon) const
{ 
	OutTooltipText.Reset();
	OutOkIcon = false;

	// If dropping onto an 'object' pin, try and set the literal
	if ((HoverPin->PinType.PinCategory == PC_Object) || (HoverPin->PinType.PinCategory == PC_Interface))
	{
		UClass* PinClass = Cast<UClass>(HoverPin->PinType.PinSubCategoryObject.Get());
		if(PinClass != nullptr)
		{
			// Find first asset of type of the pin
			FAssetData AssetData = FAssetData::GetFirstAssetDataOfClass(Assets, PinClass);
			if(AssetData.IsValid())
			{
				TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(FBlueprintEditorUtils::FindBlueprintForNode(HoverPin->GetOwningNode()));
				FText FailureReason;
				if (AssetReferenceFilter.IsValid() && !AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
				{
					OutTooltipText = FailureReason.ToString();
				}
				else
				{
					OutOkIcon = true;
					OutTooltipText = FText::FormatOrdered(LOCTEXT("SupportedAssetTypeForPinDragDrop", "Assign {0} to this pin"), FText::FromName(AssetData.AssetName)).ToString();
				}
			}
			else
			{
				OutTooltipText = LOCTEXT("UnsupportedAssetTypeForPinDragDrop", "Not compatible with this pin").ToString();
			}
		}
	}
}


void UEdGraphSchema_K2::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutTooltipText.Reset();
	OutOkIcon = false;

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(HoverGraph);
	if ((Blueprint != nullptr) && FBlueprintEditorUtils::IsActorBased(Blueprint))
	{
		TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter = MakeAssetReferenceFilter(Blueprint);

		const FBlueprintGraphModule& Module = FModuleManager::LoadModuleChecked<FBlueprintGraphModule>("BlueprintGraph");
		for (const FAssetData& AssetData : Assets)
		{
			// check asset to see if we can get some AssetBlueprintGraphActions for it
			if (const FAssetBlueprintGraphActions* GraphActions = Module.GetAssetBlueprintGraphActions(AssetData.GetClass()))
			{
				if (AssetReferenceFilter)
				{
					FText FailureReason;
					if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
					{
						if (OutTooltipText.IsEmpty())
						{
							OutTooltipText = FailureReason.ToString();
						}
						continue;
					}
				}

				// want to make sure the hover message properly represents that Input Action nodes can only be dragged onto event graphs
				if (FBlueprintEditorUtils::IsEventGraph(HoverGraph))
				{
					OutTooltipText = GraphActions->GetGraphHoverMessage(AssetData, HoverGraph).ToString();
					OutOkIcon = true;
					return;
				}
				else
				{
					OutTooltipText = LOCTEXT("UnsupportedAssetTypeForGraphDragDropEventGraph", "Cannot create a node from this type of asset in this graph").ToString();
				}
			}

			if (UObject* Asset = AssetData.GetAsset())
			{
				UClass* AssetClass = Asset->GetClass();
				if (UBlueprint* BlueprintAsset = Cast<UBlueprint>(Asset))
				{
					AssetClass = BlueprintAsset->GeneratedClass;
				}

				TSubclassOf<UActorComponent> DestinationComponentType;
				if (AssetClass && AssetClass->IsChildOf(UActorComponent::StaticClass()) && IsAllowableBlueprintVariableType(AssetClass))
				{
					// If it's an actor component subclass that is a BlueprintableComponent, we're good to go
					DestinationComponentType = AssetClass;
				}
				else
				{
					// Otherwise, see if we have a way to make a component out of the specified asset
					DestinationComponentType = FComponentAssetBrokerage::GetPrimaryComponentForAsset(AssetClass);
					if ((DestinationComponentType == nullptr) && AssetClass && AssetClass->IsChildOf(AActor::StaticClass()))
					{
						DestinationComponentType = UChildActorComponent::StaticClass();
					}
				}

				if (DestinationComponentType != nullptr)
				{
					if (AssetReferenceFilter)
					{
						FText FailureReason;
						if (!AssetReferenceFilter->PassesFilter(AssetData, &FailureReason))
						{
							if (OutTooltipText.IsEmpty())
							{
								OutTooltipText = FailureReason.ToString();
							}
							continue;
						}
					}

					OutOkIcon = true;
					OutTooltipText.Reset();
					return;
				}
			}
		}

		if (OutTooltipText.IsEmpty())
		{
			OutTooltipText = LOCTEXT("UnsupportedAssetTypeForGraphDragDrop", "Cannot create a node from this type of asset").ToString();
		}
	}
	else
	{
		OutTooltipText = LOCTEXT("CannotCreateComponentsInNonActorBlueprints", "Cannot create components from assets in a non-Actor blueprint").ToString();
	}
}

bool UEdGraphSchema_K2::FadeNodeWhenDraggingOffPin(const UEdGraphNode* Node, const UEdGraphPin* Pin) const
{
	if(Node && Pin && (PC_Delegate == Pin->PinType.PinCategory) && (EGPD_Input == Pin->Direction))
	{
		//When dragging off a delegate pin, we should duck the alpha of all nodes except the Custom Event nodes that are compatible with the delegate signature
		//This would help reinforce the connection between delegates and their matching events, and make it easier to see at a glance what could be matched up.
		if(const UK2Node_Event* EventNode = Cast<const UK2Node_Event>(Node))
		{
			const UEdGraphPin* DelegateOutPin = EventNode->FindPin(UK2Node_Event::DelegateOutputName);
			if ((NULL != DelegateOutPin) && 
				(ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW != CanCreateConnection(DelegateOutPin, Pin).Response))
			{
				return false;
			}
		}

		if(const UK2Node_CreateDelegate* CreateDelegateNode = Cast<const UK2Node_CreateDelegate>(Node))
		{
			const UEdGraphPin* DelegateOutPin = CreateDelegateNode->GetDelegateOutPin();
			if ((NULL != DelegateOutPin) && 
				(ECanCreateConnectionResponse::CONNECT_RESPONSE_DISALLOW != CanCreateConnection(DelegateOutPin, Pin).Response))
			{
				return false;
			}
		}
		
		return true;
	}
	return false;
}

struct FBackwardCompatibilityConversionHelper
{
	// Re-add orphaned pins to deal with any links that were lost during converstion
	static bool RestoreOrphanLinks(UEdGraphPin* OldPin, UEdGraphPin* NewPin, UEdGraphNode* NewNode, const TArray<UEdGraphPin*>& OldLinks)
	{
		// See if there are any links that didn't get copied, including to orphan pins or if the newpin is null
		TArray<UEdGraphPin*> OrphanedLinks;

		for (UEdGraphPin* OldLink : OldLinks)
		{
			if (!NewPin || (!NewPin->LinkedTo.Contains(OldLink) && NewNode->GetSchema()->CanCreateConnection(NewPin, OldLink).Response != CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE))
			{
				OrphanedLinks.Add(OldLink);
			}
		}

		if (OrphanedLinks.Num() > 0)
		{
			// The link may have been to a pin on a node that was *also* recreated, so now that
			// we're recreated, lets make one last ditch effort to reconnect the link:
			const auto IsUnorphanedPinByNameAndDirection = [](const UEdGraphPin* PinInQuestion, const UEdGraphPin* PinToMatch) 
			{ 
				return !PinInQuestion->bOrphanedPin && 
					PinInQuestion->PinName == PinToMatch->PinName && 
					PinInQuestion->Direction == PinToMatch->Direction;  
			};

			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
			TArray<TPair<UEdGraphPin*, UEdGraphPin*>> PinsToConnect;
			TArray<UEdGraphPin*> RelinkedOrphanedLinks;
			for(UEdGraphPin* LinkedPin : OrphanedLinks)
			{
				if(!LinkedPin->bOrphanedPin)
				{
					continue;
				}

				UEdGraphNode* Node = LinkedPin->GetOwningNode();
				UEdGraphPin* NewPinToLink = Node->FindPinByPredicate(
					[&IsUnorphanedPinByNameAndDirection, LinkedPin]
					(const UEdGraphPin* NodePin) 
					{
						return IsUnorphanedPinByNameAndDirection(NodePin, LinkedPin);
					}
				);

				if( NewPinToLink &&
					Schema->TryCreateConnection(NewPinToLink, NewPin) )
				{
					RelinkedOrphanedLinks.AddUnique(LinkedPin); // NewPinToLink is now linked to NewPin
				}
			}
			
			// remove pins with no connections if we successfully grabbed all of their connections:
			for(UEdGraphPin* DirtyPin : RelinkedOrphanedLinks)
			{
				if(DirtyPin->LinkedTo.Num() == 0)
				{
					DirtyPin->GetOwningNode()->RemovePin(DirtyPin);
					OrphanedLinks.Remove(DirtyPin);
				}
			}
			
			if (OrphanedLinks.Num() > 0)
			{
				// reconnection failed so add an orphan pin so warning/connections are not silently lost:
				UEdGraphPin* OrphanPin = NewNode->CreatePin(OldPin->Direction, OldPin->PinType, OldPin->PinName);

				OrphanPin->bOrphanedPin = true;
				OrphanPin->bNotConnectable = true;

				for (UEdGraphPin* OldLink : OrphanedLinks)
				{
					OrphanPin->MakeLinkTo(OldLink);
				}

				return true;
			}
		}

		return false;
	}

	static bool ConvertNode(
		UK2Node* OldNode,
		const FString& BlueprintPinName,
		UK2Node* NewNode,
		const FString& ClassPinName,
		const UEdGraphSchema_K2& Schema,
		bool bOnlyWithDefaultBlueprint)
	{
		check(OldNode && NewNode);
		const UBlueprint* Blueprint = OldNode->GetBlueprint();

		UEdGraph* Graph = OldNode->GetGraph();
		if (!Graph)
		{
			UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error bp: '%ls' node: '%ls'. No graph containing the node.",
				Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
				*OldNode->GetName());
			return false;
		}

		UEdGraphPin* OldBlueprintPin = OldNode->FindPin(BlueprintPinName);
		if (!OldBlueprintPin)
		{
			UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error bp: '%ls' node: '%ls'. No bp pin found '%ls'",
				Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
				*OldNode->GetName(),
				*BlueprintPinName);
			return false;
		}

		const bool bNondefaultBPConnected = (OldBlueprintPin->LinkedTo.Num() > 0);
		const bool bTryConvert = !bNondefaultBPConnected || !bOnlyWithDefaultBlueprint;
		if (bTryConvert)
		{
			// CREATE NEW NODE
			NewNode->SetFlags(RF_Transactional);
			Graph->AddNode(NewNode, false, false);
			NewNode->CreateNewGuid();
			NewNode->PostPlacedNewNode();
			NewNode->AllocateDefaultPins();
			NewNode->NodePosX = OldNode->NodePosX;
			NewNode->NodePosY = OldNode->NodePosY;

			UEdGraphPin* ClassPin = NewNode->FindPin(ClassPinName);
			if (!ClassPin)
			{
				UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error bp: '%ls' node: '%ls'. No class pin found '%ls'",
					Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
					*NewNode->GetName(),
					*ClassPinName);
				return false;
			}
			UClass* TargetClass = Cast<UClass>(ClassPin->PinType.PinSubCategoryObject.Get());
			if (!TargetClass)
			{
				UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error bp: '%ls' node: '%ls'. No class found '%ls'",
					Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
					*NewNode->GetName(),
					*ClassPinName);
				return false;
			}

			// REPLACE BLUEPRINT WITH CLASS
			if (!bNondefaultBPConnected)
			{
				// DEFAULT VALUE
				const UBlueprint* UsedBlueprint = Cast<UBlueprint>(OldBlueprintPin->DefaultObject);
				ensure(!OldBlueprintPin->DefaultObject || UsedBlueprint);
				ensure(!UsedBlueprint || *UsedBlueprint->GeneratedClass);
				UClass* UsedClass = UsedBlueprint ? *UsedBlueprint->GeneratedClass : NULL;
				Schema.TrySetDefaultObject(*ClassPin, UsedClass);
				if (ClassPin->DefaultObject != UsedClass)
				{
					FString ErrorStr = Schema.IsPinDefaultValid(ClassPin, FString(), UsedClass, FText());
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot set class' in blueprint: %ls node: '%ls' actor bp: %ls, reason: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*OldNode->GetName(),
						UsedBlueprint ? *UsedBlueprint->GetName() : TEXT("Unknown"),
						ErrorStr.IsEmpty() ? TEXT("Unknown") : *ErrorStr);
					return false;
				}
			}
			else
			{
				// LINK
				UK2Node_ClassDynamicCast* CastNode = NewObject<UK2Node_ClassDynamicCast>(Graph);
				CastNode->SetFlags(RF_Transactional);
				CastNode->TargetType = TargetClass;
				Graph->AddNode(CastNode, false, false);
				CastNode->CreateNewGuid();
				CastNode->PostPlacedNewNode();
				CastNode->AllocateDefaultPins();
				const int32 OffsetOnGraph = 200;
				CastNode->NodePosX = OldNode->NodePosX - OffsetOnGraph;
				CastNode->NodePosY = OldNode->NodePosY;

				UEdGraphPin* ExecPin = OldNode->GetExecPin();
				UEdGraphPin* ExecCastPin = CastNode->GetExecPin();
				check(ExecCastPin);
				if (!ExecPin || !Schema.MovePinLinks(*ExecPin, *ExecCastPin, false, true).CanSafeConnect())
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*ExecCastPin->PinName.ToString());
					return false;
				}

				UEdGraphPin* ValidCastPin = CastNode->GetValidCastPin();
				check(ValidCastPin);
				if (!Schema.TryCreateConnection(ValidCastPin, ExecPin))
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*ValidCastPin->PinName.ToString());
					return false;
				}

				UEdGraphPin* InValidCastPin = CastNode->GetInvalidCastPin();
				check(InValidCastPin);
				if (!Schema.TryCreateConnection(InValidCastPin, ExecPin))
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*InValidCastPin->PinName.ToString());
					return false;
				}

				UEdGraphPin* CastSourcePin = CastNode->GetCastSourcePin();
				check(CastSourcePin);
				if (!Schema.MovePinLinks(*OldBlueprintPin, *CastSourcePin, false, true).CanSafeConnect())
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*CastSourcePin->PinName.ToString());
					return false;
				}

				UEdGraphPin* CastResultPin = CastNode->GetCastResultPin();
				check(CastResultPin);
				if (!Schema.TryCreateConnection(CastResultPin, ClassPin))
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						*CastResultPin->PinName.ToString());
					return false;
				}
			}

			// MOVE OTHER PINS
			TArray<UEdGraphPin*> OldPins;
			OldPins.Add(OldBlueprintPin);
			for (UEdGraphPin* Pin : NewNode->Pins)
			{
				check(Pin);
				if (ClassPin != Pin)
				{
					UEdGraphPin* OldPin = OldNode->FindPin(Pin->PinName);
					if (OldPin)
					{
						TArray<UEdGraphPin*> OldLinks = OldPin->LinkedTo;
						OldPins.Add(OldPin);

						if (!Schema.MovePinLinks(*OldPin, *Pin, false, true).CanSafeConnect())
						{
							UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot connect' in blueprint: %ls, pin: %ls",
								Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
								*Pin->PinName.ToString());
						}

						FBackwardCompatibilityConversionHelper::RestoreOrphanLinks(OldPin, Pin, NewNode, OldLinks);
					}
					else
					{
						UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'missing old pin' in blueprint: %ls, pin: %ls",
							Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
							Pin ? *Pin->PinName.ToString() : TEXT("Unknown"));
					}
				}
			}
			OldNode->BreakAllNodeLinks();
			for (UEdGraphPin* Pin : OldNode->Pins)
			{
				if (!OldPins.Contains(Pin))
				{
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'missing new pin' in blueprint: %ls, pin: %ls",
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"),
						Pin ? *Pin->PinName.ToString() : TEXT("Unknown"));
				}
			}
			Graph->RemoveNode(OldNode);
			return true;
		}
		return false;
	}

	struct FFunctionCallParams
	{
		const FName OldFuncName;
		const FName NewFuncName;
		const FString& BlueprintPinName;
		const FString& ClassPinName;
		const UClass* FuncScope;

		FFunctionCallParams(FName InOldFunc, FName InNewFunc, const FString& InBlueprintPinName, const FString& InClassPinName, const UClass* InFuncScope)
			: OldFuncName(InOldFunc), NewFuncName(InNewFunc), BlueprintPinName(InBlueprintPinName), ClassPinName(InClassPinName), FuncScope(InFuncScope)
		{
			check(FuncScope);
		}

		FFunctionCallParams(const FBlueprintCallableFunctionRedirect& FunctionRedirect)
			: OldFuncName(*FunctionRedirect.OldFunctionName)
			, NewFuncName(*FunctionRedirect.NewFunctionName)
			, BlueprintPinName(FunctionRedirect.BlueprintParamName)
			, ClassPinName(FunctionRedirect.ClassParamName)
			, FuncScope(NULL)
		{
			FuncScope = FindFirstObject<UClass>(*FunctionRedirect.ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Fatal, TEXT("looking for FunctionRedirect.ClassName"));			
		}

	};

	static void ConvertFunctionCallNodes(const FFunctionCallParams& ConversionParams, TArray<UK2Node_CallFunction*>& Nodes, UEdGraph* Graph, const UEdGraphSchema_K2& Schema, bool bOnlyWithDefaultBlueprint)
	{
		if (ConversionParams.FuncScope)
		{
			const UFunction* NewFunc = ConversionParams.FuncScope->FindFunctionByName(ConversionParams.NewFuncName);
			if (ensureMsgf(NewFunc, TEXT("Can't find conversion function %s on %s!"), *ConversionParams.NewFuncName.ToString(), *ConversionParams.FuncScope->GetName()))
			{
				for (UK2Node_CallFunction* Node : Nodes)
				{
					// Check to see if the class scope and name are the same, we can't depend on the UFunction still existing
					UClass* MemberParent = Node->FunctionReference.GetMemberParentClass(Node->GetBlueprintClassFromNode());

					if (MemberParent == ConversionParams.FuncScope && Node->FunctionReference.GetMemberName() == ConversionParams.OldFuncName)
					{
						UK2Node_CallFunction* NewNode = NewObject<UK2Node_CallFunction>(Graph);
						NewNode->SetFromFunction(NewFunc);
						ConvertNode(Node, ConversionParams.BlueprintPinName, NewNode,
							ConversionParams.ClassPinName, Schema, bOnlyWithDefaultBlueprint);
					}
				}
			}
		}
	}
};

bool UEdGraphSchema_K2::ReplaceOldNodeWithNew(UEdGraphNode* OldNode, UEdGraphNode* NewNode, const TMap<FName, FName>& OldPinToNewPinMap) const
{
	if (!ensure(NewNode->GetGraph() == OldNode->GetGraph()))
	{
		return false;
	}
	const UEdGraphSchema* Schema = NewNode->GetSchema();
	const UObject* NodeOuter = NewNode->GetGraph() ? NewNode->GetGraph()->GetOuter() : nullptr;

	NewNode->NodePosX = OldNode->NodePosX;
	NewNode->NodePosY = OldNode->NodePosY;

	bool bFailedToFindPin = false;
	TArray<UEdGraphPin*> NewPinArray;

	for (int32 PinIdx = 0; PinIdx < OldNode->Pins.Num(); ++PinIdx)
	{
		UEdGraphPin* OldPin = OldNode->Pins[PinIdx];
		UEdGraphPin* NewPin = nullptr;

		const FName* NewPinNamePtr = OldPinToNewPinMap.Find(OldPin->PinName);
		if (NewPinNamePtr && NewPinNamePtr->IsNone())
		{
			// if they added an remapping for this pin, but left it empty, then it's assumed that they didn't want us to port any of the connections
			NewPinArray.Add(nullptr);
			continue;
		}
		else
		{
			const FName NewPinName = NewPinNamePtr ? *NewPinNamePtr : OldPin->PinName;
			NewPin = NewNode->FindPin(NewPinName);

			if (!NewPin && OldPin->ParentPin)
			{
				int32 ParentIndex = INDEX_NONE;
				if (OldNode->Pins.Find(OldPin->ParentPin, ParentIndex))
				{
					if (ensure(ParentIndex < PinIdx))
					{
						UEdGraphPin* OldParent = OldNode->Pins[ParentIndex];
						UEdGraphPin* NewParent = NewPinArray[ParentIndex];

						if (NewParent->SubPins.Num() == 0)
						{
							if (NewParent->PinType.PinCategory == PC_Wildcard)
							{
								NewParent->PinType = OldParent->PinType;
							}
							SplitPin(NewParent);
						}

						if (NewParent->SubPins.Num() > 0)
						{
							FString OldPinName = OldPin->PinName.ToString();
							OldPinName.RemoveFromStart(OldParent->PinName.ToString());

							const FString NewParentNameStr = NewParent->PinName.ToString();

							for (UEdGraphPin* SubPin : NewParent->SubPins)
							{
								FString SubPinName = SubPin->PinName.ToString();
								SubPinName.RemoveFromStart(NewParentNameStr);

								if (SubPinName == OldPinName)
								{
									NewPin = SubPin;
									break;
								}
							}
						}
					}
				}
			}
		}

		if (NewPin == nullptr)
		{
			bFailedToFindPin = true;

			UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot find pin %ls in node %ls' in: %ls",
				*OldPin->PinName.ToString(),
				*NewNode->GetNodeTitle(ENodeTitleType::FullTitle).ToString(),
				NodeOuter ? *NodeOuter->GetName() : TEXT("Unknown"));

			break;
		}
		else
		{
			NewPinArray.Add(NewPin);
		}
	}

	if (!bFailedToFindPin)
	{
		for (int32 PinIdx = 0; PinIdx < OldNode->Pins.Num(); ++PinIdx)
		{
			UEdGraphPin* OldPin = OldNode->Pins[PinIdx];
			UEdGraphPin* NewPin = NewPinArray[PinIdx];
			TArray<UEdGraphPin*> OldLinks = OldPin->LinkedTo;

			// could be null, meaning they didn't want to map this OldPin to anything
			if (NewPin == nullptr)
			{
				continue;
			}
			else if (!Schema->MovePinLinks(*OldPin, *NewPin, false, true).CanSafeConnect())
			{
				UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot safely move pin %ls to %ls' in: %ls",
					*OldPin->PinName.ToString(),
					*NewPin->PinName.ToString(),
					NodeOuter ? *NodeOuter->GetName() : TEXT("Unknown"));
			}
			else if(UK2Node* K2Node = Cast<UK2Node>(NewNode))
			{
				// for wildcard pins, which may have to react to being connected with
				K2Node->NotifyPinConnectionListChanged(NewPin);
			}

			FBackwardCompatibilityConversionHelper::RestoreOrphanLinks(OldPin, NewPin, NewNode, OldLinks);
		}

		NewNode->NodeComment = OldNode->NodeComment;
		NewNode->bCommentBubblePinned = OldNode->bCommentBubblePinned;
		NewNode->bCommentBubbleVisible = OldNode->bCommentBubbleVisible;
		
		FLinkerLoad::InvalidateExport(OldNode);
		OldNode->DestroyNode();
	}
	return !bFailedToFindPin;
}

UK2Node* UEdGraphSchema_K2::ConvertDeprecatedNodeToFunctionCall(UK2Node* OldNode, UFunction* NewFunction, TMap<FName, FName>& OldPinToNewPinMap, UEdGraph* Graph) const
{
	UK2Node_CallFunction* CallFunctionNode = NewObject<UK2Node_CallFunction>(Graph);
	check(CallFunctionNode);
	CallFunctionNode->SetFlags(RF_Transactional);
	Graph->AddNode(CallFunctionNode, false, false);
	CallFunctionNode->SetFromFunction(NewFunction);
	CallFunctionNode->CreateNewGuid();
	CallFunctionNode->PostPlacedNewNode();
	CallFunctionNode->AllocateDefaultPins();

	if (!ReplaceOldNodeWithNew(OldNode, CallFunctionNode, OldPinToNewPinMap))
	{
		// Failed, destroy node
		CallFunctionNode->DestroyNode();
		return nullptr;
	}
	return CallFunctionNode;
}

void UEdGraphSchema_K2::BackwardCompatibilityNodeConversion(UEdGraph* Graph, bool bOnlySafeChanges) const 
{
	if (Graph)
	{
		{
			static const FString BlueprintPinName(TEXT("Blueprint"));
			static const FString ClassPinName(TEXT("Class"));
			TArray<UK2Node_SpawnActor*> SpawnActorNodes;
			Graph->GetNodesOfClass(SpawnActorNodes);
			for (UK2Node_SpawnActor* SpawnActorNode : SpawnActorNodes)
			{
				FBackwardCompatibilityConversionHelper::ConvertNode(
					SpawnActorNode, BlueprintPinName, NewObject<UK2Node_SpawnActorFromClass>(Graph),
					ClassPinName, *this, bOnlySafeChanges);
			}
		}

		{
			UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
			if (Blueprint && *Blueprint->SkeletonGeneratedClass)
			{
				TArray<UK2Node_CallFunction*> Nodes;
				Graph->GetNodesOfClass(Nodes);
				for (const FBlueprintCallableFunctionRedirect& FunctionRedirect : EditoronlyBPFunctionRedirects)
				{
					FBackwardCompatibilityConversionHelper::ConvertFunctionCallNodes(
						FBackwardCompatibilityConversionHelper::FFunctionCallParams(FunctionRedirect),
						Nodes, Graph, *this, bOnlySafeChanges);
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Log, "BackwardCompatibilityNodeConversion: Blueprint '%ls' cannot be fully converted. It has no skeleton class!",
					Blueprint ? *Blueprint->GetName() : TEXT("Unknown"));
			}
		}

		// Call per-node deprecation functions
		TArray<UK2Node*> PossiblyDeprecatedNodes;
		Graph->GetNodesOfClass<UK2Node>(PossiblyDeprecatedNodes);

		for (UK2Node* Node : PossiblyDeprecatedNodes)
		{
			Node->ConvertDeprecatedNode(Graph, bOnlySafeChanges);
		}
	}
}

UEdGraphNode* UEdGraphSchema_K2::CreateSubstituteNode(UEdGraphNode* Node, const UEdGraph* Graph, FObjectInstancingGraph* InstanceGraph, TSet<FName>& InOutExtraNames) const
{
	// If this is an event node, create a unique custom event node as a substitute
	UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node);
	if(EventNode)
	{
		if(!Graph)
		{
			// Use the node's graph (outer) if an explicit graph was not specified
			Graph = Node->GetGraph();
		}

		// Can only place events in ubergraphs
		if (GetGraphType(Graph) != EGraphType::GT_Ubergraph)
		{
			return NULL;
		}

		// Find the Blueprint that owns the graph
		UBlueprint* Blueprint = Graph ? FBlueprintEditorUtils::FindBlueprintForGraph(Graph) : NULL;
		if(Blueprint && Blueprint->SkeletonGeneratedClass)
		{
			// Gather all names in use by the Blueprint class
			TSet<FName> ExistingNamesInUse = InOutExtraNames;
			FBlueprintEditorUtils::GetFunctionNameList(Blueprint, ExistingNamesInUse);
			FBlueprintEditorUtils::GetClassVariableList(Blueprint, ExistingNamesInUse);

			const ERenameFlags RenameFlags = (Blueprint->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0);

			// Allow the old object name to be used in the graph
			FName ObjName = EventNode->GetFName();
			UObject* Found = FindObject<UObject>(EventNode->GetOuter(), *ObjName.ToString());
			if(Found)
			{
				Found->Rename(NULL, NULL, REN_DontCreateRedirectors | RenameFlags);
			}

			// Create a custom event node to replace the original event node imported from text
			UK2Node_CustomEvent* CustomEventNode = NewObject<UK2Node_CustomEvent>(EventNode->GetOuter(), ObjName, EventNode->GetFlags(), nullptr, true, InstanceGraph);

			// Ensure that it is editable
			CustomEventNode->bIsEditable = true;

			// Set grid position to match that of the target node
			CustomEventNode->NodePosX = EventNode->NodePosX;
			CustomEventNode->NodePosY = EventNode->NodePosY;

			// Reuse the same GUID as the replaced node
			CustomEventNode->NodeGuid = EventNode->NodeGuid;

			// Build a function name that is appropriate for the event we're replacing
			FString FunctionName;
			const UK2Node_ActorBoundEvent* ActorBoundEventNode = Cast<const UK2Node_ActorBoundEvent>(EventNode);
			const UK2Node_ComponentBoundEvent* CompBoundEventNode = Cast<const UK2Node_ComponentBoundEvent>(EventNode);

			const UEdGraphNode* PreExistingNode = nullptr;
			
			if (InstanceGraph)
			{
				// Use a generic name for the new custom event
				FunctionName = TEXT("CustomEvent");
			}
			else
			{
				// Create a name for the custom event based off the original function
				if (ActorBoundEventNode)
				{
					FString TargetName = TEXT("None");
					if (ActorBoundEventNode->EventOwner)
					{
						TargetName = ActorBoundEventNode->EventOwner->GetActorLabel();
					}

					FunctionName = FString::Printf(TEXT("%s_%s"), *ActorBoundEventNode->DelegatePropertyName.ToString(), *TargetName);
					PreExistingNode = FKismetEditorUtilities::FindBoundEventForActor(ActorBoundEventNode->GetReferencedLevelActor(), ActorBoundEventNode->DelegatePropertyName);
				}
				else if (CompBoundEventNode)
				{
					FunctionName = FString::Printf(TEXT("%s_%s"), *CompBoundEventNode->DelegatePropertyName.ToString(), *CompBoundEventNode->ComponentPropertyName.ToString());
					PreExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, CompBoundEventNode->DelegatePropertyName, CompBoundEventNode->ComponentPropertyName);
				}
				else if (EventNode->CustomFunctionName != NAME_None)
				{
					FunctionName = EventNode->CustomFunctionName.ToString();
				}
				else if (EventNode->bOverrideFunction)
				{
					FunctionName = EventNode->EventReference.GetMemberName().ToString();
				}
				else
				{
					FunctionName = CustomEventNode->GetName().Replace(TEXT("K2Node_"), TEXT(""), ESearchCase::CaseSensitive);
				}
			}

			// Ensure the name does not overlap with other names
			CustomEventNode->CustomFunctionName = FName(*FunctionName, FNAME_Find);
			if (CustomEventNode->CustomFunctionName != NAME_None
				&& ExistingNamesInUse.Contains(CustomEventNode->CustomFunctionName))
			{
				int32 i = 0;
				FString TempFuncName;

				do
				{
					TempFuncName = FString::Printf(TEXT("%s_%d"), *FunctionName, ++i);
					CustomEventNode->CustomFunctionName = FName(*TempFuncName, FNAME_Find);
				} while (CustomEventNode->CustomFunctionName != NAME_None
					&& ExistingNamesInUse.Contains(CustomEventNode->CustomFunctionName));

				FunctionName = TempFuncName;
			}

			if (ActorBoundEventNode)
			{
				PreExistingNode = FKismetEditorUtilities::FindBoundEventForActor(ActorBoundEventNode->GetReferencedLevelActor(), ActorBoundEventNode->DelegatePropertyName);
			}
			else if (CompBoundEventNode)
			{
				PreExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(Blueprint, CompBoundEventNode->DelegatePropertyName, CompBoundEventNode->ComponentPropertyName);
			}
			else
			{
				if (Cast<UK2Node_CustomEvent>(EventNode))
				{
					PreExistingNode = FBlueprintEditorUtils::FindCustomEventNode(Blueprint, EventNode->CustomFunctionName);
				}
				else if (UFunction* EventSignature = EventNode->FindEventSignatureFunction())
				{
					// Note: EventNode::FindEventSignatureFunction will return null if it is deleted (for instance, someone declared a 
					// BlueprintImplementableEvent, and some blueprint implements it, but then the declaration is deleted). It also
					// returns null if the pasted node was sourced from another asset that's not included in the destination project.
					// This is acceptable since we've already created a substitute anyway; this is just looking to see if we actually
					// have a valid pre-existing node that was in conflict, in which case we will emit a warning to the message log.
					UClass* ClassOwner = EventSignature->GetOwnerClass();
					if (ensureMsgf(ClassOwner, TEXT("Wrong class owner of signature %s in node %s"), *GetPathNameSafe(EventSignature), *GetPathNameSafe(EventNode)))
					{
						PreExistingNode = FBlueprintEditorUtils::FindOverrideForFunction(Blueprint, ClassOwner->GetAuthoritativeClass(), EventSignature->GetFName());
					}
				}
			}

			// Should be a unique name now, go ahead and assign it
			CustomEventNode->CustomFunctionName = FName(*FunctionName);
			InOutExtraNames.Add(CustomEventNode->CustomFunctionName);

			// Copy the pins from the old node to the new one that's replacing it
			CustomEventNode->Pins = EventNode->Pins;
			CustomEventNode->UserDefinedPins = EventNode->UserDefinedPins;

			// Clear out the pins from the old node so that links aren't broken later when it's destroyed
			EventNode->Pins.Empty();
			EventNode->UserDefinedPins.Empty();

			bool bOriginalWasCustomEvent = Cast<UK2Node_CustomEvent>(Node) != nullptr;

			// Fixup pins
			for(int32 PinIndex = 0; PinIndex < CustomEventNode->Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = CustomEventNode->Pins[PinIndex];
				check(Pin);

				// Reparent the pin to the new custom event node
				Pin->SetOwningNode(CustomEventNode);

				// Don't include execution or delegate output pins as user-defined pins
				if(!bOriginalWasCustomEvent && !IsExecPin(*Pin) && !IsDelegateCategory(Pin->PinType.PinCategory))
				{
					// Check to see if this pin already exists as a user-defined pin on the custom event node
					bool bFoundUserDefinedPin = false;
					for(int32 UserDefinedPinIndex = 0; UserDefinedPinIndex < CustomEventNode->UserDefinedPins.Num() && !bFoundUserDefinedPin; ++UserDefinedPinIndex)
					{
						const FUserPinInfo& UserDefinedPinInfo = *CustomEventNode->UserDefinedPins[UserDefinedPinIndex].Get();
						bFoundUserDefinedPin = Pin->PinName == UserDefinedPinInfo.PinName && Pin->PinType == UserDefinedPinInfo.PinType;
					}

					if(!bFoundUserDefinedPin)
					{
						// Add a new entry into the user-defined pin array for the custom event node
						TSharedPtr<FUserPinInfo> UserPinInfo = MakeShareable(new FUserPinInfo());
						UserPinInfo->PinName = Pin->PinName;
						UserPinInfo->PinType = Pin->PinType;
						CustomEventNode->UserDefinedPins.Add(UserPinInfo);
					}
				}
			}

			if (PreExistingNode)
			{
				if (!Blueprint->PreCompileLog.IsValid())
				{
					Blueprint->PreCompileLog = TSharedPtr<FCompilerResultsLog>(new FCompilerResultsLog(false));
					Blueprint->PreCompileLog->bSilentMode = false;
					Blueprint->PreCompileLog->bAnnotateMentionedNodes = false;
					Blueprint->PreCompileLog->SetSourcePath(Blueprint->GetPathName());
				}

				// Append a warning to the node and to the logs
				CustomEventNode->bHasCompilerMessage = true;
				CustomEventNode->ErrorType = EMessageSeverity::Warning;
				FFormatNamedArguments Args;
				Args.Add(TEXT("NodeName"), CustomEventNode->GetNodeTitle(ENodeTitleType::ListView));
				Args.Add(TEXT("OriginalNodeName"), FText::FromString(PreExistingNode->GetName()));
				CustomEventNode->ErrorMsg = FText::Format(LOCTEXT("ReverseUpgradeWarning", "Conflicted with {OriginalNodeName} and was replaced as a Custom Event!"), Args).ToString();
				Blueprint->PreCompileLog->Warning(*LOCTEXT("ReverseUpgradeWarning_Log", "Pasted node @@  conflicted with @@ and was replaced as a Custom Event!").ToString(), CustomEventNode, PreExistingNode);
			}
			// Return the new custom event node that we just created as a substitute for the original event node
			return CustomEventNode;
		}
	}

	// Use the default logic in all other cases
	return UEdGraphSchema::CreateSubstituteNode(Node, Graph, InstanceGraph, InOutExtraNames);
}

int32 UEdGraphSchema_K2::GetNodeSelectionCount(const UEdGraph* Graph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	int32 SelectionCount = 0;
	
	if( Blueprint )
	{
		SelectionCount = FKismetEditorUtilities::GetNumberOfSelectedNodes(Blueprint);
	}
	return SelectionCount;
}

TSharedPtr<FEdGraphSchemaAction> UEdGraphSchema_K2::GetCreateCommentAction() const
{
	return TSharedPtr<FEdGraphSchemaAction>(static_cast<FEdGraphSchemaAction*>(new FEdGraphSchemaAction_K2AddComment));
}

bool UEdGraphSchema_K2::CanDuplicateGraph(UEdGraph* InSourceGraph) const
{
	EGraphType GraphType = GetGraphType(InSourceGraph);
	return GraphType == GT_Function || GraphType == GT_Macro;
}

UEdGraph* UEdGraphSchema_K2::DuplicateGraph(UEdGraph* GraphToDuplicate) const
{
	UEdGraph* NewGraph = NULL;

	if (CanDuplicateGraph(GraphToDuplicate))
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(GraphToDuplicate);

		NewGraph = FEdGraphUtilities::CloneGraph(GraphToDuplicate, Blueprint);

		if (NewGraph)
		{
			bool bIsOverrideGraph = false;
			if (Blueprint->BlueprintType == BPTYPE_Interface)
			{
				bIsOverrideGraph = true;
			}
			else if (FBlueprintEditorUtils::FindFunctionInImplementedInterfaces(Blueprint, GraphToDuplicate->GetFName()))
			{
				bIsOverrideGraph = true;
			}
			else if (FindUField<UFunction>(Blueprint->ParentClass, GraphToDuplicate->GetFName()))
			{
				bIsOverrideGraph = true;
			}

			// When duplicating an override function, we must put the graph through some extra work to properly own the data being duplicated, instead of expecting pin information will come from a parent
			if (bIsOverrideGraph)
			{
				FBlueprintEditorUtils::PromoteGraphFromInterfaceOverride(Blueprint, NewGraph);
				
				// Remove all calls to the parent function, fix any exec pin links to pass through
				TArray< UK2Node_CallParentFunction* > ParentFunctionCalls;
				NewGraph->GetNodesOfClass(ParentFunctionCalls);

				for (UK2Node_CallParentFunction* ParentFunctionCall : ParentFunctionCalls)
				{
					UEdGraphPin* ExecPin = ParentFunctionCall->GetExecPin();
					UEdGraphPin* ThenPin = ParentFunctionCall->GetThenPin();
					if (ExecPin->LinkedTo.Num() && ThenPin->LinkedTo.Num())
					{
						MovePinLinks(*ExecPin, *ThenPin->LinkedTo[0]);
					}
					NewGraph->RemoveNode(ParentFunctionCall);
				}
			}

			// can't have two graphs with the same guid... that'd be silly!
			NewGraph->GraphGuid = FGuid::NewGuid();

			//Rename the entry node or any further renames will not update the entry node, also fixes a duplicate node issue on compile
			for (int32 NodeIndex = 0; NodeIndex < NewGraph->Nodes.Num(); ++NodeIndex)
			{
				UEdGraphNode* Node = NewGraph->Nodes[NodeIndex];
				if (UK2Node_FunctionTerminator* TerminatorNode = Cast<UK2Node_FunctionTerminator>(Node))
				{
					if (TerminatorNode->FunctionReference.GetMemberName() == GraphToDuplicate->GetFName())
					{
						TerminatorNode->Modify();
						
						// We're duplicating the graph, so fully reset the member reference (including the GUID!)
						FMemberReference NewRef;
						NewRef.SetMemberName(NewGraph->GetFName());
						TerminatorNode->FunctionReference = NewRef;
					}
				}
				// Rename any custom events to be unique
				else if (Node->GetClass()->GetFName() ==  TEXT("K2Node_CustomEvent"))
				{
					UK2Node_CustomEvent* CustomEvent = Cast<UK2Node_CustomEvent>(Node);
					CustomEvent->RenameCustomEventCloseToName();
				}
			}

			// Potentially adjust variable names for any child blueprints
			FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, NewGraph->GetFName());
		}
	}
	return NewGraph;
}

/**
 * Attempts to best-guess the height of the node. This is necessary because we don't know the actual
 * size of the node until the next Slate tick
 *
 * @param Node The node to guess the height of
 * @return The estimated height of the specified node
 */
float UEdGraphSchema_K2::EstimateNodeHeight( UEdGraphNode* Node )
{
	float HeightEstimate = 0.0f;

	if ( Node != NULL )
	{
		float BaseNodeHeight = 48.0f;
		bool bConsiderNodePins = false;
		float HeightPerPin = 18.0f;

		if ( Node->IsA( UK2Node_CallFunction::StaticClass() ) )
		{
			BaseNodeHeight = 80.0f;
			bConsiderNodePins = true;
			HeightPerPin = 18.0f;
		}
		else if ( Node->IsA( UK2Node_Event::StaticClass() ) )
		{
			BaseNodeHeight = 48.0f;
			bConsiderNodePins = true;
			HeightPerPin = 16.0f;
		}

		HeightEstimate = BaseNodeHeight;

		if ( bConsiderNodePins )
		{
			int32 NumInputPins = 0;
			int32 NumOutputPins = 0;

			for ( int32 PinIndex = 0; PinIndex < Node->Pins.Num(); PinIndex++ )
			{
				UEdGraphPin* CurrentPin = Node->Pins[PinIndex];
				if ( CurrentPin != NULL && !CurrentPin->bHidden )
				{
					switch ( CurrentPin->Direction )
					{
						case EGPD_Input:
							{
								NumInputPins++;
							}
							break;
						case EGPD_Output:
							{
								NumOutputPins++;
							}
							break;
					}
				}
			}

			float MaxNumPins = float(FMath::Max<int32>( NumInputPins, NumOutputPins ));

			HeightEstimate += MaxNumPins * HeightPerPin;
		}
	}

	return HeightEstimate;
}

/**
 * Attempts to best-guess the width of the node. This is necessary because we don't know the actual
 * size of the node until the next Slate tick
 *
 * @param Node The node to guess the width of
 * @return The estimated width of the specified node
 */
float UEdGraphSchema_K2::EstimateNodeWidth( const UEdGraphNode* Node )
{
	float WidthEstimate = 0.0f;

	if ( Node != NULL )
	{
		const float BaseNodeWidth = Node->IsA( UK2Node_CallFunction::StaticClass() ) ? 250.0f : 200.0f;

		FString NodeTitle = Node->GetNodeTitle( ENodeTitleType::FullTitle ).ToString();

		if ( const UFont* Font = GetDefault<UEditorEngine>()->EditorFont )
		{
			WidthEstimate = FMath::Max( BaseNodeWidth, (float)Font->GetStringSize( *NodeTitle ) );
		}
		else
		{
			static const float EstimatedCharWidth = 6.0f;
			WidthEstimate = FMath::Max( BaseNodeWidth, NodeTitle.Len() * EstimatedCharWidth );
		}
	}

	return WidthEstimate;
}


bool UEdGraphSchema_K2::CollapseGatewayNode(UK2Node* InNode, UEdGraphNode* InEntryNode, UEdGraphNode* InResultNode, FKismetCompilerContext* CompilerContext, TSet<UEdGraphNode*>* OutExpandedNodes) const
{
	bool bSuccessful = true;

	// Handle any split pin cleanup in either the Entry or Result node first
	auto HandleSplitPins = [CompilerContext, OutExpandedNodes](UK2Node* Node)
	{
		if (Node)
		{
			for (int32 PinIdx = Node->Pins.Num() - 1; PinIdx >= 0; --PinIdx)
			{
				UEdGraphPin* const Pin = Node->Pins[PinIdx];

				// Expand any gateway pins as needed
				if (Pin->SubPins.Num() > 0)
				{
					if (UK2Node* ExpandedNode = Node->ExpandSplitPin(CompilerContext, Node->GetGraph(), Pin))
					{
						if (OutExpandedNodes)
						{
							OutExpandedNodes->Add(ExpandedNode);
						}
					}
				}
			}
		}
	};
	HandleSplitPins(Cast<UK2Node>(InEntryNode));
	HandleSplitPins(Cast<UK2Node>(InResultNode));

	// We iterate the array in reverse so we can both remove the subpins safely after we've read them and
	// so we have split nested structs we combine them back together in the right order
	for (int32 BoundaryPinIndex = InNode->Pins.Num() - 1; BoundaryPinIndex >= 0; --BoundaryPinIndex)
	{
		UEdGraphPin* const BoundaryPin = InNode->Pins[BoundaryPinIndex];

		bool bFunctionNode = InNode->IsA(UK2Node_CallFunction::StaticClass());

		// For each pin in the gateway node, find the associated pin in the entry or result node.
		UEdGraphNode* const GatewayNode = (BoundaryPin->Direction == EGPD_Input) ? InEntryNode : InResultNode;
		UEdGraphPin* GatewayPin = nullptr;
		if (GatewayNode)
		{
			// First handle struct combining if necessary
			if (BoundaryPin->SubPins.Num() > 0)
			{
				if (UK2Node* ExpandedNode = InNode->ExpandSplitPin(CompilerContext, InNode->GetGraph(), BoundaryPin))
				{
					if (OutExpandedNodes)
					{
						OutExpandedNodes->Add(ExpandedNode);
					}
				}
			}

			for (int32 PinIdx = GatewayNode->Pins.Num() - 1; PinIdx >= 0; --PinIdx)
			{
				UEdGraphPin* const Pin = GatewayNode->Pins[PinIdx];

				// Function graphs have a single exec path through them, so only one exec pin for input and another for output. In this fashion, they must not be handled by name.
				if(InNode->GetClass() == UK2Node_CallFunction::StaticClass() && Pin->PinType.PinCategory == PC_Exec && BoundaryPin->PinType.PinCategory == PC_Exec && (Pin->Direction != BoundaryPin->Direction))
				{
					GatewayPin = Pin;
					break;
				}
				else if ((Pin->PinName == BoundaryPin->PinName) && (Pin->Direction != BoundaryPin->Direction))
				{
					GatewayPin = Pin;
					break;
				}
			}
		}

		if (GatewayPin)
		{
			CombineTwoPinNetsAndRemoveOldPins(BoundaryPin, GatewayPin);
		}
		else
		{
			if (BoundaryPin->LinkedTo.Num() > 0 && BoundaryPin->ParentPin == nullptr)
			{
				UBlueprint* OwningBP = InNode->GetBlueprint();
				if( OwningBP )
				{
					// We had an input/output with a connection that wasn't twinned
					bSuccessful = false;
					OwningBP->Message_Warn(
						FText::Format(
							NSLOCTEXT("K2Node", "PinOnBoundryNode_WarningFmt", "Warning: Pin '{0}' on boundary node '{1}' could not be found in the composite node '{2}'"),
							FText::FromString(BoundaryPin->PinName.ToString()),
							GatewayNode ? FText::FromString(GatewayNode->GetName()) : NSLOCTEXT("K2Node", "PinOnBoundryNode_WarningNoNode", "(null)"),
							FText::FromString(GetName())
						).ToString()
					);
				}
				else
				{
					UE_LOGF(
						LogBlueprint,
						Warning,
						"%ls",
						*FText::Format(
							NSLOCTEXT("K2Node", "PinOnBoundryNode_WarningFmt", "Warning: Pin '{0}' on boundary node '{1}' could not be found in the composite node '{2}'"),
							FText::FromString(BoundaryPin->PinName.ToString()),
							GatewayNode ? FText::FromString(GatewayNode->GetName()) : NSLOCTEXT("K2Node", "PinOnBoundryNode_WarningNoNode", "(null)"),
							FText::FromString(GetName())
						).ToString()
					);
				}
			}
			else
			{
				// Associated pin was not found but there were no links on this side either, so no harm no foul
			}
		}
	}

	return bSuccessful;
}

void UEdGraphSchema_K2::CombineTwoPinNetsAndRemoveOldPins(UEdGraphPin* InPinA, UEdGraphPin* InPinB) const
{
	check(InPinA != NULL);
	check(InPinB != NULL);
	ensure(InPinA->Direction != InPinB->Direction);

	if ((InPinA->LinkedTo.Num() == 0) && (InPinA->Direction == EGPD_Input))
	{
		// Push the literal value of A to InPinB's connections
		for (int32 IndexB = 0; IndexB < InPinB->LinkedTo.Num(); ++IndexB)
		{
			UEdGraphPin* FarB = InPinB->LinkedTo[IndexB];
			// TODO: Michael N. says this if check should be unnecessary once the underlying issue is fixed.
			// (Probably should use a check() instead once it's removed though.  See additional cases below.
			if (FarB != nullptr)
			{
				FarB->DefaultValue = InPinA->DefaultValue;
				FarB->DefaultObject = InPinA->DefaultObject;
				FarB->DefaultTextValue = InPinA->DefaultTextValue;
			}
		}
	}
	else if ((InPinB->LinkedTo.Num() == 0) && (InPinB->Direction == EGPD_Input))
	{
		// Push the literal value of B to InPinA's connections
		for (int32 IndexA = 0; IndexA < InPinA->LinkedTo.Num(); ++IndexA)
		{
			UEdGraphPin* FarA = InPinA->LinkedTo[IndexA];
			// TODO: Michael N. says this if check should be unnecessary once the underlying issue is fixed.
			// (Probably should use a check() instead once it's removed though.  See additional cases above and below.
			if (FarA != nullptr)
			{
				FarA->DefaultValue = InPinB->DefaultValue;
				FarA->DefaultObject = InPinB->DefaultObject;
				FarA->DefaultTextValue = InPinB->DefaultTextValue;
			}
		}
	}
	else
	{
		// Make direct connections between the things that connect to A or B, removing A and B from the picture
		for (int32 IndexA = 0; IndexA < InPinA->LinkedTo.Num(); ++IndexA)
		{
			UEdGraphPin* FarA = InPinA->LinkedTo[IndexA];
			// TODO: Michael N. says this if check should be unnecessary once the underlying issue is fixed.
			// (Probably should use a check() instead once it's removed though.  See additional cases above.
			if (FarA != nullptr)
			{
				for (int32 IndexB = 0; IndexB < InPinB->LinkedTo.Num(); ++IndexB)
				{
					UEdGraphPin* FarB = InPinB->LinkedTo[IndexB];

					if (FarB != nullptr)
					{
						FarA->Modify();
						FarB->Modify();
						FarA->MakeLinkTo(FarB);
					}
					
				}
			}
		}
	}

	InPinA->BreakAllPinLinks();
	InPinB->BreakAllPinLinks();
}

UK2Node* UEdGraphSchema_K2::CreateSplitPinNode(UEdGraphPin* Pin, const FCreateSplitPinNodeParams& Params) const
{
	ensure((Params.bTransient == false) || ((Params.CompilerContext == nullptr) && (Params.SourceGraph == nullptr)));

	UEdGraphNode* GraphNode = Pin->GetOwningNode();
	UEdGraph* Graph = GraphNode->GetGraph();
	UScriptStruct* StructType = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
	if (!StructType)
	{
		if (Params.CompilerContext)
		{
			Params.CompilerContext->MessageLog.Error(TEXT("No structure in SubCategoryObject in pin @@"), Pin);
		}
		StructType = GetFallbackStruct();
	}

	UK2Node* SplitPinNode = nullptr;

	if (Pin->Direction == EGPD_Input)
	{
		if (UK2Node_MakeStruct::CanBeMade(StructType))
		{
			UK2Node_MakeStruct* MakeStructNode;

			if (Params.bTransient || Params.CompilerContext)
			{
				MakeStructNode = (Params.bTransient ? NewObject<UK2Node_MakeStruct>(Graph) : Params.CompilerContext->SpawnIntermediateNode<UK2Node_MakeStruct>(GraphNode, Params.SourceGraph));
				MakeStructNode->StructType = StructType;
				MakeStructNode->bMadeAfterOverridePinRemoval = true;
				MakeStructNode->AllocateDefaultPins();
			}
			else
			{
				FGraphNodeCreator<UK2Node_MakeStruct> MakeStructCreator(*Graph);
				MakeStructNode = MakeStructCreator.CreateNode(false);
				MakeStructNode->StructType = StructType;
				MakeStructNode->bMadeAfterOverridePinRemoval = true;
				MakeStructCreator.Finalize();
			}

			if (Pin->DefaultValue.Len() > 0)
			{
				FStructOnScope StructOnScope(StructType);
				StructType->ImportText(*Pin->DefaultValue, StructOnScope.GetStructMemory(), nullptr, PPF_None, GLog, StructType->GetName());

				for (TFieldIterator<FProperty> PropIt(StructType); PropIt; ++PropIt)
				{
					if (UEdGraphPin* MakeStructPin = MakeStructNode->FindPin(PropIt->GetFName(), EGPD_Input))
					{
						MakeStructPin->DefaultValue.Reset();
						FBlueprintEditorUtils::PropertyValueToString(*PropIt, StructOnScope.GetStructMemory(), MakeStructPin->DefaultValue);
						MakeStructPin->AutogeneratedDefaultValue = MakeStructPin->DefaultValue;
					}
				}
			}

			SplitPinNode = MakeStructNode;
		}
		else
		{
			const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeMakeFunction);
			const UFunction* Function = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);

			UK2Node_CallFunction* CallFunctionNode;

			if (Params.bTransient || Params.CompilerContext)
			{
				CallFunctionNode = (Params.bTransient ? NewObject<UK2Node_CallFunction>(Graph) : Params.CompilerContext->SpawnIntermediateNode<UK2Node_CallFunction>(GraphNode, Params.SourceGraph));
				CallFunctionNode->SetFromFunction(Function);
				CallFunctionNode->AllocateDefaultPins();
			}
			else
			{
				FGraphNodeCreator<UK2Node_CallFunction> MakeStructCreator(*Graph);
				CallFunctionNode = MakeStructCreator.CreateNode(false);
				CallFunctionNode->SetFromFunction(Function);
				MakeStructCreator.Finalize();
			}

			SplitPinNode = CallFunctionNode;
		}
	}
	else
	{
		if (UK2Node_BreakStruct::CanBeBroken(StructType))
		{
			UK2Node_BreakStruct* BreakStructNode;

			if (Params.bTransient || Params.CompilerContext)
			{
				BreakStructNode = (Params.bTransient ? NewObject<UK2Node_BreakStruct>(Graph) : Params.CompilerContext->SpawnIntermediateNode<UK2Node_BreakStruct>(GraphNode, Params.SourceGraph));
				BreakStructNode->StructType = StructType;
				BreakStructNode->bMadeAfterOverridePinRemoval = true;
				BreakStructNode->AllocateDefaultPins();
			}
			else
			{
				FGraphNodeCreator<UK2Node_BreakStruct> MakeStructCreator(*Graph);
				BreakStructNode = MakeStructCreator.CreateNode(false);
				BreakStructNode->StructType = StructType;
				BreakStructNode->bMadeAfterOverridePinRemoval = true;
				MakeStructCreator.Finalize();
			}

			SplitPinNode = BreakStructNode;
		}
		else
		{
			const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeBreakFunction);
			const UFunction* Function = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);

			UK2Node_CallFunction* CallFunctionNode;

			if (Params.bTransient || Params.CompilerContext)
			{
				CallFunctionNode = (Params.bTransient ? NewObject<UK2Node_CallFunction>(Graph) : Params.CompilerContext->SpawnIntermediateNode<UK2Node_CallFunction>(GraphNode, Params.SourceGraph));
				CallFunctionNode->SetFromFunction(Function);
				CallFunctionNode->AllocateDefaultPins();
			}
			else
			{
				FGraphNodeCreator<UK2Node_CallFunction> MakeStructCreator(*Graph);
				CallFunctionNode = MakeStructCreator.CreateNode(false);
				CallFunctionNode->SetFromFunction(Function);
				MakeStructCreator.Finalize();
			}

			SplitPinNode = CallFunctionNode;
		}
	}

	SplitPinNode->NodePosX = GraphNode->NodePosX - SplitPinNode->NodeWidth - 10;
	SplitPinNode->NodePosY = GraphNode->NodePosY;

	return SplitPinNode;
}

void UEdGraphSchema_K2::SplitPin(UEdGraphPin* Pin, const bool bNotify) const
{
	// Under some circumstances we can get here when PinSubCategoryObject is not set, so we just can't split the pin in that case
	UScriptStruct* StructType = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
	const bool bCanSplit =
		(StructType != nullptr) &&
		(!Pin->PinType.IsContainer())
	;

	if (!bCanSplit)
	{
		return;
	}

	UEdGraphNode* GraphNode = Pin->GetOwningNode();
	UK2Node* K2Node = Cast<UK2Node>(GraphNode);
	UEdGraph* Graph = CastChecked<UEdGraph>(GraphNode->GetOuter());

	GraphNode->Modify();
	Pin->Modify();

	Pin->bHidden = true;

	UK2Node* ProtoExpandNode = CreateSplitPinNode(Pin, FCreateSplitPinNodeParams(/*bTransient*/true));
			
	for (UEdGraphPin* ProtoPin : ProtoExpandNode->Pins)
	{
		if (ProtoPin->Direction == Pin->Direction && !ProtoPin->bHidden)
		{
			const FName PinName = *FString::Printf(TEXT("%s_%s"), *Pin->PinName.ToString(), *ProtoPin->PinName.ToString());
			const FEdGraphPinType& ProtoPinType = ProtoPin->PinType;
			UEdGraphNode::FCreatePinParams PinParams;
			PinParams.ContainerType = ProtoPinType.ContainerType;
			PinParams.ValueTerminalType = ProtoPinType.PinValueType;
			UEdGraphPin* SubPin = GraphNode->CreatePin(Pin->Direction, ProtoPinType.PinCategory, ProtoPinType.PinSubCategory, ProtoPinType.PinSubCategoryObject.Get(), PinName, PinParams);
			check(SubPin);

			// Delegate pins will also need a signature copied over. CreatePin doesn't handle this.
			if (const UFunction* PinSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(ProtoPinType.PinSubCategoryMemberReference))
			{
				FMemberReference::FillSimpleMemberReference(PinSignature, SubPin->PinType.PinSubCategoryMemberReference);
			}

			if (K2Node != nullptr && K2Node->ShouldDrawCompact() && !Pin->ParentPin)
			{
				SubPin->PinFriendlyName = ProtoPin->GetDisplayName();
			}
			else
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("PinDisplayName"), Pin->GetDisplayName());
				Arguments.Add(TEXT("ProtoPinDisplayName"), ProtoPin->GetDisplayName());
				SubPin->PinFriendlyName = FText::Format(LOCTEXT("SplitPinFriendlyNameFormat", "{PinDisplayName} {ProtoPinDisplayName}"), Arguments);
			}

			SubPin->DefaultValue = MoveTemp(ProtoPin->DefaultValue);
			SubPin->AutogeneratedDefaultValue = MoveTemp(ProtoPin->AutogeneratedDefaultValue);
			SubPin->bDeprecated = ProtoPin->bDeprecated;
			SubPin->DeprecationMessage = ProtoPin->DeprecationMessage;

			SubPin->ParentPin = Pin;

			// CreatePin puts the Pin in the array, but we are going to insert it later, so pop it back out
			GraphNode->Pins.Pop(EAllowShrinking::No);

			Pin->SubPins.Add(SubPin);
		}
	}

	ProtoExpandNode->DestroyNode();

	if (Pin->Direction == EGPD_Input)
	{
		TArray<FString> OriginalDefaults;
		if (   StructType == TBaseStructure<FVector>::Get()
			|| StructType == TBaseStructure<FRotator>::Get())
		{
			Pin->DefaultValue.ParseIntoArray(OriginalDefaults, TEXT(","), false);
			for (FString& Default : OriginalDefaults)
			{
				Default = FString::SanitizeFloat(FCString::Atof(*Default));
			}
			// In some cases (particularly wildcards) the default value may not accurately reflect the normal component elements
			while (OriginalDefaults.Num() < 3)
			{
				OriginalDefaults.Add(TEXT("0.0"));
			}
			
			// Rotator OriginalDefaults are in the form of Y,Z,X but our pins are in the form of X,Y,Z
			// so we have to change the OriginalDefaults order here to match our pins
			if (StructType == TBaseStructure<FRotator>::Get())
			{
				OriginalDefaults.Swap(0, 2);
				OriginalDefaults.Swap(1, 2);
			}
		}
		else if (StructType == TBaseStructure<FVector2D>::Get())
		{
			FVector2D V2D;
			V2D.InitFromString(Pin->DefaultValue);

			OriginalDefaults.Add(FString::SanitizeFloat(V2D.X));
			OriginalDefaults.Add(FString::SanitizeFloat(V2D.Y));
		}
		else if (StructType == TBaseStructure<FLinearColor>::Get())
		{
			FLinearColor LC;
			LC.InitFromString(Pin->DefaultValue);

			OriginalDefaults.Add(FString::SanitizeFloat(LC.R));
			OriginalDefaults.Add(FString::SanitizeFloat(LC.G));
			OriginalDefaults.Add(FString::SanitizeFloat(LC.B));
			OriginalDefaults.Add(FString::SanitizeFloat(LC.A));
		}

		check(OriginalDefaults.Num() == 0 || OriginalDefaults.Num() == Pin->SubPins.Num());

		for (int32 SubPinIndex = 0; SubPinIndex < OriginalDefaults.Num(); ++SubPinIndex)
		{
			UEdGraphPin* SubPin = Pin->SubPins[SubPinIndex];
			SubPin->DefaultValue = OriginalDefaults[SubPinIndex];
		}
	}

	GraphNode->Pins.Insert(Pin->SubPins, GraphNode->Pins.Find(Pin) + 1);

	if (bNotify)
	{
		Graph->NotifyGraphChanged();

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void UEdGraphSchema_K2::RecombinePin(UEdGraphPin* Pin) const
{
	UEdGraphPin* ParentPin = Pin->ParentPin;

	if (ParentPin == nullptr)
	{
		if (Pin->SubPins.Num() > 0)
		{
			RecombinePin(Pin->SubPins[0]);
		}

		return;
	}

	UEdGraphNode* GraphNode = Pin->GetOwningNode();

	GraphNode->Modify();
	ParentPin->Modify();

	ParentPin->bHidden = false;

	UEdGraph* Graph = CastChecked<UEdGraph>(GraphNode->GetOuter());
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);

	for (int32 SubPinIndex = 0; SubPinIndex < ParentPin->SubPins.Num(); ++SubPinIndex)
	{
		UEdGraphPin* SubPin = ParentPin->SubPins[SubPinIndex];

		if (SubPin->SubPins.Num() > 0)
		{
			RecombinePin(SubPin->SubPins[0]);
		}

		GraphNode->Pins.Remove(SubPin);
		FKismetDebugUtilities::RemovePinWatch(Blueprint, SubPin);
	}

	if (Pin->Direction == EGPD_Input)
	{
		if (UScriptStruct* StructType = Cast<UScriptStruct>(ParentPin->PinType.PinSubCategoryObject.Get()))
		{
			if (StructType == TBaseStructure<FVector>::Get())
			{
				ParentPin->DefaultValue = ParentPin->SubPins[0]->DefaultValue + TEXT(",") 
										+ ParentPin->SubPins[1]->DefaultValue + TEXT(",")
										+ ParentPin->SubPins[2]->DefaultValue;
			}
			else if (StructType == TBaseStructure<FRotator>::Get())
			{
				// Our pins are in the form X,Y,Z but the Rotator pin type expects the form Y,Z,X
				// so we need to make sure they are added in that order here
				ParentPin->DefaultValue = ParentPin->SubPins[1]->DefaultValue + TEXT(",")
										+ ParentPin->SubPins[2]->DefaultValue + TEXT(",")
										+ ParentPin->SubPins[0]->DefaultValue;
			}
			else if (StructType == TBaseStructure<FVector2D>::Get())
			{
				FVector2D V2D;
				V2D.X = FCString::Atof(*ParentPin->SubPins[0]->DefaultValue);
				V2D.Y = FCString::Atof(*ParentPin->SubPins[1]->DefaultValue);
			
				ParentPin->DefaultValue = V2D.ToString();
			}
			else if (StructType == TBaseStructure<FLinearColor>::Get())
			{
				FLinearColor LC;
				LC.R = FCString::Atof(*ParentPin->SubPins[0]->DefaultValue);
				LC.G = FCString::Atof(*ParentPin->SubPins[1]->DefaultValue);
				LC.B = FCString::Atof(*ParentPin->SubPins[2]->DefaultValue);
				LC.A = FCString::Atof(*ParentPin->SubPins[3]->DefaultValue);

				ParentPin->DefaultValue = LC.ToString();
			}
		}
	}

	// Clear out subpins:
	TArray<UEdGraphPin*>& ParentSubPins = ParentPin->SubPins;
	while (ParentSubPins.Num())
	{
		// To ensure that MarkPendingKill does not mutate ParentSubPins, we null out the ParentPin
		// if we assume that MarkPendingKill *will* mutate ParentSubPins we could introduce an infinite
		// loop. No known case of this being possible, but it would be trivial to write bad node logic
		// that introduces this problem:
		ParentSubPins.Last()->ParentPin = nullptr; 
		ParentSubPins.Last()->MarkAsGarbage();
		ParentSubPins.RemoveAt(ParentSubPins.Num()-1);
	}

	Graph->NotifyGraphChanged();
	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
}

void UEdGraphSchema_K2::OnPinConnectionDoubleCicked(UEdGraphPin* PinA, UEdGraphPin* PinB, const FVector2f& GraphPosition) const
{
	const FScopedTransaction Transaction(LOCTEXT("CreateRerouteNodeOnWire", "Create Reroute Node"), UEdGraphSchemaImpl::ShouldActuallyTransact());

	//@TODO: This constant is duplicated from inside of SGraphNodeKnot
	const FVector2f NodeSpacerSize(42.0f, 24.0f);
	const FVector2f KnotTopLeft = GraphPosition - (NodeSpacerSize * 0.5f);

	// Create a new knot
	UEdGraph* ParentGraph = PinA->GetOwningNode()->GetGraph();
	if (!FBlueprintEditorUtils::IsGraphReadOnly(ParentGraph))
	{
		UK2Node_Knot* NewKnot = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_Knot>(ParentGraph, FDeprecateSlateVector2D(KnotTopLeft), EK2NewNodeFlags::SelectNewNode);

		// Move the connections across (only notifying the knot, as the other two didn't really change)
		PinA->BreakLinkTo(PinB);
		PinA->MakeLinkTo((PinA->Direction == EGPD_Output) ? NewKnot->GetInputPin() : NewKnot->GetOutputPin());
		PinB->MakeLinkTo((PinB->Direction == EGPD_Output) ? NewKnot->GetInputPin() : NewKnot->GetOutputPin());
		NewKnot->PostReconstructNode();

		// Dirty the blueprint
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

void UEdGraphSchema_K2::ConfigureVarNode(UK2Node_Variable* InVarNode, FName InVariableName, UStruct* InVariableSource, UBlueprint* InTargetBlueprint)
{
	// See if this is a 'self context' (ie. blueprint class is owner (or child of owner) of dropped var class)
	if ((InVariableSource == NULL) || (InTargetBlueprint->SkeletonGeneratedClass && InTargetBlueprint->SkeletonGeneratedClass->IsChildOf(InVariableSource)))
	{
		FGuid Guid = FBlueprintEditorUtils::FindMemberVariableGuidByName(InTargetBlueprint, InVariableName);
		InVarNode->VariableReference.SetSelfMember(InVariableName, Guid);
	}
	else if (InVariableSource->IsA(UClass::StaticClass()))
	{
		FGuid Guid;
		if (UBlueprint* VariableOwnerBP = Cast<UBlueprint>(Cast<UClass>(InVariableSource)->ClassGeneratedBy))
		{
			Guid = FBlueprintEditorUtils::FindMemberVariableGuidByName(VariableOwnerBP, InVariableName);
		}

		InVarNode->VariableReference.SetExternalMember(InVariableName, CastChecked<UClass>(InVariableSource), Guid);
	}
	else
	{
		FGuid LocalVarGuid = FBlueprintEditorUtils::FindLocalVariableGuidByName(InTargetBlueprint, InVariableSource, InVariableName);
		if (LocalVarGuid.IsValid())
		{
			InVarNode->VariableReference.SetLocalMember(InVariableName, InVariableSource, LocalVarGuid);
		}
	}
}

UK2Node_VariableGet* UEdGraphSchema_K2::SpawnVariableGetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(ParentGraph);

	return FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_VariableGet>(
		ParentGraph,
		GraphPosition,
		EK2NewNodeFlags::SelectNewNode,
		[VariableName, Source, Blueprint](UK2Node_VariableGet* NewInstance)
		{
			UEdGraphSchema_K2::ConfigureVarNode(NewInstance, VariableName, Source, Blueprint);
		}
	);
}

UK2Node_VariableSet* UEdGraphSchema_K2::SpawnVariableSetNode(const FVector2D GraphPosition, class UEdGraph* ParentGraph, FName VariableName, UStruct* Source) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(ParentGraph);

	return FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_VariableSet>(
		ParentGraph,
		GraphPosition,
		EK2NewNodeFlags::SelectNewNode,
		[VariableName, Source, Blueprint](UK2Node_VariableSet* NewInstance)
		{
			UEdGraphSchema_K2::ConfigureVarNode(NewInstance, VariableName, Source, Blueprint);
		}
	);
}

UEdGraphPin* UEdGraphSchema_K2::DropPinOnNode(UEdGraphNode* InTargetNode, const FName& InSourcePinName, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection) const
{
	UEdGraphPin* ResultPin = nullptr;
	if (UK2Node_EditablePinBase* EditablePinNode = Cast<UK2Node_EditablePinBase>(InTargetNode))
	{
		TArray<UK2Node_EditablePinBase*> EditablePinNodes;
		EditablePinNode->Modify();

		FEdGraphPinType SourcePinType = InSourcePinType;

		// When a user creates a new float pin, we need to ensure that it's a double.
		// Otherwise, we're unintentionally creating a loophole where users can create single-precision floats.
		if (SourcePinType.PinCategory == PC_Real)
		{
			SourcePinType.PinSubCategory = PC_Double;
		}

		if (InSourcePinDirection == EGPD_Output && Cast<UK2Node_FunctionEntry>(InTargetNode))
		{
			if (UK2Node_FunctionResult* ResultNode = FBlueprintEditorUtils::FindOrCreateFunctionResultNode(EditablePinNode))
			{
				EditablePinNodes.Add(ResultNode);
			}
			else
			{
				// If we did not successfully find or create a result node, just fail out
				return nullptr;
			}
		}
		else if (InSourcePinDirection == EGPD_Input && Cast<UK2Node_FunctionResult>(InTargetNode))
		{
			TArray<UK2Node_FunctionEntry*> FunctionEntryNode;
			InTargetNode->GetGraph()->GetNodesOfClass(FunctionEntryNode);

			if (FunctionEntryNode.Num() == 1)
			{
				EditablePinNodes.Add(FunctionEntryNode[0]);
			}
			else
			{
				// If we did not successfully find the entry node, just fail out
				return nullptr;
			}
		}
		else
		{
			if (UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(EditablePinNode))
			{
				EditablePinNodes.Append(ResultNode->GetAllResultNodes());
			}
			else
			{
				EditablePinNodes.Add(EditablePinNode);
			}
		}

		for (UK2Node_EditablePinBase* CurrentEditablePinNode : EditablePinNodes)
		{
			CurrentEditablePinNode->Modify();
			UEdGraphPin* CreatedPin = CurrentEditablePinNode->CreateUserDefinedPin(InSourcePinName, SourcePinType, (InSourcePinDirection == EGPD_Input) ? EGPD_Output : EGPD_Input);

			// The final ResultPin is from the node the user dragged and dropped to
			if (EditablePinNode == CurrentEditablePinNode)
			{
				ResultPin = CreatedPin;
			}
		}

		HandleParameterDefaultValueChanged(EditablePinNode);
	}
	return ResultPin;
}

void UEdGraphSchema_K2::HandleParameterDefaultValueChanged(UK2Node* InTargetNode) const
{
	if (UK2Node_EditablePinBase* EditablePinNode = Cast<UK2Node_EditablePinBase>(InTargetNode))
	{
		// If this is happening during a save, it's not safe to trigger a compilation
		if (UE::IsSavingPackage())
		{
			return;
		}

		// Make sure we have a valid Blueprint owner before proceeding, so we don't crash the editor attempting to modify it below.
		UBlueprint* TargetBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(EditablePinNode);
		if (!ensureMsgf(TargetBlueprint, TEXT("UEdGraphSchema_K2::HandleParameterDefaultValueChanged(): Failed to find a Blueprint for target node: %s"), *GetPathNameSafe(EditablePinNode)))
		{
			return;
		}

		// Make sure we have a valid outer graph object before proceeding, so we don't crash the editor attempting to modify it below.
		UEdGraph* EditablePinNodeGraph = EditablePinNode->GetGraph();
		if (!ensureMsgf(EditablePinNodeGraph, TEXT("UEdGraphSchema_K2::HandleParameterDefaultValueChanged(): Failed to find a valid graph for target node: %s"), *GetPathNameSafe(EditablePinNode)))
		{
			return;
		}

		FParamsChangedHelper ParamsChangedHelper;
		ParamsChangedHelper.ModifiedBlueprints.Add(TargetBlueprint);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(TargetBlueprint);

		ParamsChangedHelper.Broadcast(TargetBlueprint, EditablePinNode, EditablePinNodeGraph);

		for (UEdGraph* ModifiedGraph : ParamsChangedHelper.ModifiedGraphs)
		{
			if (ModifiedGraph)
			{
				ModifiedGraph->NotifyGraphChanged();
			}
		}

		// Now update all the blueprints that got modified
		for (UBlueprint* Blueprint : ParamsChangedHelper.ModifiedBlueprints)
		{
			if (Blueprint)
			{
				Blueprint->BroadcastChanged();
			}
		}
	}
}

bool UEdGraphSchema_K2::SupportsDropPinOnNode(UEdGraphNode* InTargetNode, const FEdGraphPinType& InSourcePinType, EEdGraphPinDirection InSourcePinDirection, FText& OutErrorMessage) const
{
	bool bIsSupported = false;
	if (UK2Node_EditablePinBase* EditablePinNode = Cast<UK2Node_EditablePinBase>(InTargetNode))
	{
		if (InSourcePinDirection == EGPD_Output && Cast<UK2Node_FunctionEntry>(InTargetNode))
		{
			// Just check with the Function Entry and see if it's legal, we'll create/use a result node if the user drops
			bIsSupported = EditablePinNode->CanCreateUserDefinedPin(InSourcePinType, InSourcePinDirection, OutErrorMessage);

			if (bIsSupported)
			{
				OutErrorMessage = LOCTEXT("AddConnectResultNode", "Add Pin to Result Node");
			}
		}
		else if (InSourcePinDirection == EGPD_Input && Cast<UK2Node_FunctionResult>(InTargetNode))
		{
			// Just check with the Function Result and see if it's legal, we'll create/use a result node if the user drops
			bIsSupported = EditablePinNode->CanCreateUserDefinedPin(InSourcePinType, InSourcePinDirection, OutErrorMessage);

			if (bIsSupported)
			{
				OutErrorMessage = LOCTEXT("AddPinEntryNode", "Add Pin to Entry Node");
			}
		}
		else
		{
			bIsSupported = EditablePinNode->CanCreateUserDefinedPin(InSourcePinType, (InSourcePinDirection == EGPD_Input)? EGPD_Output : EGPD_Input, OutErrorMessage);
			if (bIsSupported)
			{
				OutErrorMessage = LOCTEXT("AddPinToNode", "Add Pin to Node");
			}
		}
	}
	return bIsSupported;
}

bool UEdGraphSchema_K2::IsCacheVisualizationOutOfDate(int32 InVisualizationCacheID) const
{
	return CurrentCacheRefreshID != InVisualizationCacheID;
}

int32 UEdGraphSchema_K2::GetCurrentVisualizationCacheID() const
{
	return CurrentCacheRefreshID;
}

void UEdGraphSchema_K2::ForceVisualizationCacheClear() const
{
	++CurrentCacheRefreshID;
}


bool UEdGraphSchema_K2::SafeDeleteNodeFromGraph(UEdGraph* Graph, UEdGraphNode* NodeToDelete) const 
{
	UK2Node* Node = Cast<UK2Node>(NodeToDelete);
	if (Node == nullptr || Graph == nullptr || NodeToDelete->GetGraph() != Graph)
	{
		return false;
	}

	UBlueprint* OwnerBlueprint = Node->GetBlueprint();
	Graph->Modify();

	FBlueprintEditorUtils::RemoveNode(OwnerBlueprint, Node, /*bDontRecompile=*/ true);
	FBlueprintEditorUtils::MarkBlueprintAsModified(OwnerBlueprint);
	return true;
}

TSharedPtr<IAssetReferenceFilter> UEdGraphSchema_K2::MakeAssetReferenceFilter(const UBlueprint* Blueprint)
{
	if (GEditor && Blueprint)
	{
		FAssetReferenceFilterContext AssetReferenceFilterContext;
		AssetReferenceFilterContext.AddReferencingAsset(Blueprint);
		return GEditor->MakeAssetReferenceFilter(AssetReferenceFilterContext);
	}

	return {};
}

#if WITH_EDITORONLY_DATA

//////////////////////////////////////////////////////////////////////////
/** CVars for tweaking how the blueprint context menu search picks the best match */
namespace BPContextMenuConsoleVariables
{
	/** Increasing this weight will give a bonus to shorter matching words */
	static float ShorterWeight = 10.0f;
	static FAutoConsoleVariableRef CVarShorterWeight(
		TEXT("BP.ContextMenu.ShorterWeight"), ShorterWeight,
		TEXT("Increasing this weight will make shorter words preferred"),
		ECVF_Default);

	/** When calculating shorter weight, this is the maximum length to make it relative to */
	static int32 MaxWordLength = 30;
	static FAutoConsoleVariableRef CVarMaxWordLength(
		TEXT("BP.ContextMenu.MaxWordLength"), MaxWordLength,
		TEXT("Maximum length to count while awarding short word weight"),
		ECVF_Default);

	/** Increasing this will prefer whole percentage matches when comparing the keyword to what the user has typed in */
	static float PercentageMatchWeightMultiplier = 1.0f;
	static FAutoConsoleVariableRef CVarPercentageMatchWeightMultiplier(
		TEXT("BP.ContextMenu.PercentageMatchWeightMultiplier"), PercentageMatchWeightMultiplier,
		TEXT("A multiplier for how much weight to give something based on the percentage match it is"),
		ECVF_Default);

	/** How much weight the description of actions have */
	static float DescriptionWeight = 10.0f;
	static FAutoConsoleVariableRef CVarDescriptionWeight(
		TEXT("BP.ContextMenu.DescriptionWeight"), DescriptionWeight,
		TEXT("The amount of weight placed on search items description"),
		ECVF_Default);

	/** Weight used to prefer categories that are the same as the node that was dragged off of */
	static float MatchingFromPinCategory = 500.0f;
	static FAutoConsoleVariableRef CVarMatchingFromPinCategory(
		TEXT("BP.ContextMenu.MatchingFromPinCategory"), MatchingFromPinCategory,
		TEXT("The amount of weight placed on actions with the same category as the node being dragged off of"),
		ECVF_Default);

	/** Weight that a match to a category search has */
	static float CategoryWeight = 4.0f;
	static FAutoConsoleVariableRef CVarCategoryWeight(
		TEXT("BP.ContextMenu.CategoryWeight"), CategoryWeight,
		TEXT("The amount of weight placed on categories that match what the user has typed in"),
		ECVF_Default);

	/** How much weight the node's title has */
	static float NodeTitleWeight = 10.0f;
	static FAutoConsoleVariableRef CVarNodeTitleWeight(
		TEXT("BP.ContextMenu.NodeTitleWeight"), NodeTitleWeight,
		TEXT("The amount of weight placed on the search items title"),
		ECVF_Default);

	/** Weight used to prefer keywords of actions  */
	static float KeywordWeight = 30.0f;
	static FAutoConsoleVariableRef CVarKeywordWeight(
		TEXT("BP.ContextMenu.KeywordWeight"), KeywordWeight,
		TEXT("The amount of weight placed on search items keyword"),
		ECVF_Default);

	/** The multiplier given if the keyword starts with a term the user typed in */
	static float StartsWithBonusWeightMultiplier = 4.0f;
	static FAutoConsoleVariableRef CVarStartsWithBonusWeightMultiplier(
		TEXT("BP.ContextMenu.StartsWithBonusWeightMultiplier"), StartsWithBonusWeightMultiplier,
		TEXT("The multiplier given if the keyword starts with a term the user typed in"),
		ECVF_Default);

	/** The multiplier given if the keyword contains a term the user typed in */
	static float WordContainsLetterWeightMultiplier = 0.5f;
	static FAutoConsoleVariableRef CVarWordContainsLetterWeightMultiplier(
		TEXT("BP.ContextMenu.WordContainsLetterWeightMultiplier"), WordContainsLetterWeightMultiplier,
		TEXT("The multiplier given if the keyword only contains a term the user typed in"),
		ECVF_Default);

	/** The bonus given if node is a favorite */
	static float FavoriteBonus = 1000.0f;
	static FAutoConsoleVariableRef CVarWordContainsLetterFavoriteBonus(
		TEXT("BP.ContextMenu.FavoriteBonus"), FavoriteBonus,
		TEXT("The bonus given if node is a favorite"),
		ECVF_Default);

	/** The bonus given if an action has the same container type as the dragged from pin */
	static float ContainerBonus = 1000.0f;
	static FAutoConsoleVariableRef CVarContainerBonus(
		TEXT("BP.ContextMenu.ContainerBonus"), ContainerBonus,
		TEXT("The bonus given if the dragged from pin matches the same container type of the action"),
		ECVF_Default);
};	// namespace BPContextMenuConsoleVariables

FGraphSchemaSearchWeightModifiers UEdGraphSchema_K2::GetSearchWeightModifiers() const
{
	FGraphSchemaSearchWeightModifiers Modifiers;
	Modifiers.NodeTitleWeight = BPContextMenuConsoleVariables::NodeTitleWeight;
	Modifiers.KeywordWeight = BPContextMenuConsoleVariables::KeywordWeight;
	Modifiers.DescriptionWeight = BPContextMenuConsoleVariables::DescriptionWeight;
	Modifiers.CategoryWeight = BPContextMenuConsoleVariables::DescriptionWeight;
	Modifiers.WholeMatchLocalizedWeightMultiplier = BPContextMenuConsoleVariables::WordContainsLetterWeightMultiplier;
	Modifiers.WholeMatchWeightMultiplier = BPContextMenuConsoleVariables::WordContainsLetterWeightMultiplier;
	Modifiers.StartsWithBonusWeightMultiplier = BPContextMenuConsoleVariables::StartsWithBonusWeightMultiplier;
	Modifiers.PercentageMatchWeightMultiplier = BPContextMenuConsoleVariables::PercentageMatchWeightMultiplier;
	Modifiers.ShorterMatchWeight = BPContextMenuConsoleVariables::ShorterWeight;
	return Modifiers;
}

/**
* Debug Info about how the preferred context menu action is chosen
* @see SGraphActionMenu::GetActionFilteredWeight
*/
struct FBPContextMenuWeightDebugInfo : public FGraphSchemaSearchTextDebugInfo
{
	float FavoriteBonusWeight = 0.0f;
	float CategoryBonusWeight = 0.0f;

	/**
	* Print out the debug info about this weight info to the console
	*/
	virtual void Print(const TArray<FString>& SearchForKeywords, const FEdGraphSchemaAction& Action) const override
	{
		// Combine the actions string, separate with \n so terms don't run into each other, and remove the spaces (incase the user is searching for a variable)
		// In the case of groups containing multiple actions, they will have been created and added at the same place in the code, using the same description
		// and keywords, so we only need to use the first one for filtering.
		const FString& SearchText = Action.GetFullSearchText();

		UE_LOG(LogTemp, Warning, TEXT("[Weight for %s] \
TotalWeight: %-8.2f | PercentageMatchWeight: %-8.2f | PercMatch: %-8.2f | ShorterWeight: %-8.2f | CategoryBonusWeight: %-8.2f | KeywordArrayWeight: %-8.2f | DescriptionWeight: %-8.2f | NodeTitleWeight: %-8.2f | CategoryWeight: %-8.2f | Fav. Bonus:%-8.2f\n"),
			*SearchText, TotalWeight, PercentMatchWeight, PercentMatch, ShorterMatchWeight, CategoryBonusWeight, KeywordWeight, DescriptionWeight, NodeTitleWeight, CategoryWeight, FavoriteBonusWeight);
	}
};

float UEdGraphSchema_K2::GetActionFilteredWeight(const FEdGraphSchemaAction& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	// The overall 'weight' of this action 
	float TotalWeight = 0.0f;

	// Setup an array of arrays so we can do a weighted search			
	TArray< FGraphSchemaSearchTextWeightInfo > WeightedArrayList;
	FBPContextMenuWeightDebugInfo OutDebugInfo;

	const bool bIsFromDrag = (DraggedFromPins.Num() > 0);

	FGraphSchemaSearchWeightModifiers WeightModifiers = GetSearchWeightModifiers();
	// If there are no keywords, bump the weight on description to compensate
	const TArray<FString>& LocKeywords = InCurrentAction.GetLocalizedSearchKeywordsArray();
	WeightModifiers.DescriptionWeight = LocKeywords.Num() > 0 ? WeightModifiers.DescriptionWeight : WeightModifiers.DescriptionWeight * 2.0f;

	CollectSearchTextWeightInfo(InCurrentAction, WeightModifiers, WeightedArrayList, &OutDebugInfo);

	// Give a weight bonus to actions whose category matches what was dragged off of
	if (bIsFromDrag)
	{
		const TArray<FString>& InActionCategories = InCurrentAction.GetCategoryChain();
		bool bAddMatchBonus = false;

		/** Get a string reference for an EPinContainerType */
		auto GetContainerTypeString = [](const EPinContainerType Type) -> const FString&
		{
			static const FString ArrayName = TEXT("Array");
			static const FString MapName = TEXT("Map");
			static const FString SetName = TEXT("Set");
			static const FString InvalidName = TEXT("INVALID");

			switch (Type)
			{
				case EPinContainerType::Array:
					return ArrayName;
				case EPinContainerType::Map:
					return MapName;
				case EPinContainerType::Set:
					return SetName;
				default:
					return InvalidName;
			}
		};

		bool bAddedContainerPreferenceBonus = false;

		for (const FString& InActionCategory : InActionCategories)
		{
			for (UEdGraphPin* const FromPin : DraggedFromPins)
			{
				check(FromPin != nullptr);

				// For containers, add a preference for functions that are marked in their category
				if (!bAddedContainerPreferenceBonus && FromPin->PinType.IsContainer() && InActionCategory == GetContainerTypeString(FromPin->PinType.ContainerType))
				{
					TotalWeight += BPContextMenuConsoleVariables::ContainerBonus;
					bAddedContainerPreferenceBonus = true;
				}

				// Check the subcategory of the object to cover more more complex struct types (LinearColor, date time, etc)
				if (UObject* const SubCatObj = FromPin->PinType.PinSubCategoryObject.Get())
				{
					const FString& SubCatObjName = SubCatObj->GetPathName();
					// The pin SubObjectCategory names don't have any spaces, so split up the category
					TArray<FString> DelimitedArray;
					InActionCategory.ParseIntoArray(DelimitedArray, TEXT(" "), true);
					for (const FString& DelimetedCat : DelimitedArray)
					{
						if (SubCatObjName.Contains(DelimetedCat))
						{
							bAddMatchBonus = true;
							break;
						}
					}
				}
				// Check the category of the pin, this works for basic math types (int, float, byte, etc)
				else if (InActionCategory.Contains(FromPin->PinType.PinCategory.ToString()))
				{
					bAddMatchBonus = true;
				}

				// If we found match in any cases above then add the weight bonus and stop looking
				if (bAddMatchBonus)
				{
					TotalWeight += BPContextMenuConsoleVariables::MatchingFromPinCategory;
					OutDebugInfo.CategoryBonusWeight += BPContextMenuConsoleVariables::MatchingFromPinCategory;

					// Break out of the loop so that we don't give any extra bonuses
					break;
				}
			}
		}
	}

	// If the user has favorite this action, then give it a hefty bonus
	const UEditorPerProjectUserSettings& EditorSettings = *GetDefault<UEditorPerProjectUserSettings>();
	if (UBlueprintPaletteFavorites* BlueprintFavorites = EditorSettings.BlueprintFavorites)
	{
		if (BlueprintFavorites->IsFavorited(InCurrentAction))
		{
			TotalWeight += BPContextMenuConsoleVariables::FavoriteBonus;
			OutDebugInfo.FavoriteBonusWeight += BPContextMenuConsoleVariables::FavoriteBonus;
		}
	}

	// Now iterate through all the filter terms and calculate a 'weight' using the values and multipliers
	const FString* EachTerm = nullptr;
	const FString* EachTermSanitized = nullptr;

	// For every filter item the user has typed in (the text in the search bar, separated by spaces)
	for (int32 FilterIndex = 0; FilterIndex < InFilterTerms.Num(); ++FilterIndex)
	{
		EachTerm = &InFilterTerms[FilterIndex];
		EachTermSanitized = &InSanitizedFilterTerms[FilterIndex];
		int32 TermLen = EachTerm->Len();

		// Now check the weighted lists	(We could further improve the hit weight by checking consecutive word matches)
		for (int32 iFindCount = 0; iFindCount < WeightedArrayList.Num(); ++iFindCount)
		{
			const TArray<FString>& KeywordArray = *WeightedArrayList[iFindCount].Array;
			float WeightPerList = 0.0f;
			float KeywordArrayWeight = WeightedArrayList[iFindCount].WeightModifier;

			// Count of how many words in this keyword array contain a filter(letter) that the user has typed in
			int32 WordMatchCount = 0;

			// The number of characters in the best matching word
			int32 BestMatchCharLength = 0;

			// Loop through every word that the user could be looking for
			for (int32 iEachWord = 0; iEachWord < KeywordArray.Num(); ++iEachWord)
			{
				float WeightPerWord = 0.0f;

				// If a word contains the letter that the user has typed in, than increment the whole match count					
				if (KeywordArray[iEachWord].Contains(*EachTermSanitized, ESearchCase::CaseSensitive) || KeywordArray[iEachWord].Contains(*EachTerm, ESearchCase::CaseSensitive))
				{
					++WordMatchCount;
					WeightPerWord += KeywordArrayWeight * BPContextMenuConsoleVariables::WordContainsLetterWeightMultiplier;

					// If the word starts with the letter, give it a little extra boost of weight
					if (KeywordArray[iEachWord].StartsWith(*EachTermSanitized, ESearchCase::CaseSensitive) || KeywordArray[iEachWord].StartsWith(*EachTerm, ESearchCase::CaseSensitive))
					{
						WeightPerWord += KeywordArrayWeight * BPContextMenuConsoleVariables::StartsWithBonusWeightMultiplier;
					}

					if (WeightPerWord > WeightPerList)
					{
						// Use the best word match weight, we don't want to double-count redundant keywords like add and addmap here
						WeightPerList = WeightPerWord;
						BestMatchCharLength = KeywordArray[iEachWord].Len();
					}
				}
			}

			// If the user has dragged off of a pin then do not prefer shorter things, because that will result
			// in the matching of "Add" for a container instead of "+" for numeric types
			// We only care about length penalty if something actually matched
			if (BestMatchCharLength > 0 && WeightPerList > 0)
			{
				// How many words that we are checking had partial matches compared to what the user typed in?
				float PercMatch = static_cast<float>(WordMatchCount) / static_cast<float>(KeywordArray.Num());

				float PercentageBonus = (WeightPerList * PercMatch * BPContextMenuConsoleVariables::PercentageMatchWeightMultiplier);
				WeightPerList += PercentageBonus;

				// The shorter the matching word, the larger bonus it gets
				float ShortFactor = static_cast<float>(BPContextMenuConsoleVariables::MaxWordLength - FMath::Min(BestMatchCharLength, BPContextMenuConsoleVariables::MaxWordLength));
				float ShortWeight = ShortFactor * BPContextMenuConsoleVariables::ShorterWeight * (bIsFromDrag ? 0.25f : 1.0f);
				WeightPerList += ShortWeight;

				OutDebugInfo.PercentMatch += PercMatch;
				OutDebugInfo.ShorterMatchWeight += ShortWeight;
				OutDebugInfo.PercentMatchWeight += PercentageBonus;
			}

			TotalWeight += WeightPerList;
			if (WeightedArrayList[iFindCount].DebugWeight)
			{
				// Each weight is used twice so add them
				*WeightedArrayList[iFindCount].DebugWeight += WeightPerList;
			}
		}
	}
	OutDebugInfo.TotalWeight = TotalWeight;

	PrintSearchTextDebugInfo(InFilterTerms, InCurrentAction, &OutDebugInfo);

	return TotalWeight;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
float UEdGraphSchema_K2::GetActionFilteredWeight(const FGraphActionListBuilderBase::ActionGroup& InCurrentAction, const TArray<FString>& InFilterTerms, const TArray<FString>& InSanitizedFilterTerms, const TArray<UEdGraphPin*>& DraggedFromPins) const
{
	int32 Action = 0;
	if (InCurrentAction.Actions[Action].IsValid() == true)
	{
		return GetActionFilteredWeight(*InCurrentAction.Actions[Action], InFilterTerms, InSanitizedFilterTerms, DraggedFromPins);
	}

	return 0.f;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITORONLY_DATA

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
