// Copyright Epic Games, Inc. All Rights Reserved.

#include "EdGraphSchema_K2_Actions.h"

#include "ComponentAssetBroker.h"
#include "Components/ActorComponent.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "K2Node.h"
#include "K2Node_AddComponent.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "K2Node_Literal.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Layout/SlateRect.h"
#include "Math/UnrealMathSSE.h"
#include "ScopedTransaction.h"
#include "Settings/EditorStyleSettings.h"
#include "UObject/Field.h"
#include "UObject/ObjectHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphSchema_K2_Actions)


namespace 
{
	// Maximum distance a drag can be off a node edge to require 'push off' from node
	const int32 NodeDistance = 60;

	// The amount to offset a literal reference (to an actor) from the function node it is being connected to
	const float FunctionNodeLiteralReferencesXOffset = 224.0f;

	// The height of a literal reference node
	const float NodeLiteralHeight = 48.0f;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_BlueprintVariableBase

void FEdGraphSchemaAction_BlueprintVariableBase::MovePersistentItemToCategory(const FText& NewCategoryName)
{
	FBlueprintEditorUtils::SetBlueprintVariableCategory(GetSourceBlueprint(), VarName, Cast<UStruct>(GetVariableScope()), NewCategoryName);
}

int32 FEdGraphSchemaAction_BlueprintVariableBase::GetReorderIndexInContainer() const
{
	if (UBlueprint* SourceBlueprint = GetSourceBlueprint())
	{
		return FBlueprintEditorUtils::FindNewVariableIndex(SourceBlueprint, VarName);
	}

	return INDEX_NONE;
}

bool FEdGraphSchemaAction_BlueprintVariableBase::ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction)
{
	if (OtherAction->GetPersistentItemDefiningObject() == GetPersistentItemDefiningObject())
	{
		FEdGraphSchemaAction_BlueprintVariableBase* VarAction = (FEdGraphSchemaAction_BlueprintVariableBase*)&OtherAction.Get();

		// Only let you drag and drop if variables are from same BP class, and not onto itself
		UBlueprint* BP = GetSourceBlueprint();
		FName TargetVarName = VarAction->GetVariableName();
		if ((BP != nullptr) && (VarName != TargetVarName) && (VariableSource == VarAction->GetVariableScope()))
		{
			if (FBlueprintEditorUtils::MoveVariableBeforeVariable(BP, Cast<UStruct>(VarAction->GetVariableScope()), VarName, TargetVarName, true))
			{
				// Change category of var to match the one we dragged on to as well
				FText TargetVarCategory = FBlueprintEditorUtils::GetBlueprintVariableCategory(BP, TargetVarName, Cast<UStruct>(GetVariableScope()));
				MovePersistentItemToCategory(TargetVarCategory);

				// Update Blueprint after changes so they reflect in My Blueprint tab.
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);

				return true;
			}
		}
	}

	return false;
}


FEdGraphSchemaActionDefiningObject FEdGraphSchemaAction_BlueprintVariableBase::GetPersistentItemDefiningObject() const
{
	UObject* DefiningObject = GetSourceBlueprint();
	if (FProperty* Prop = GetProperty())
	{
		DefiningObject = Prop->GetOwnerStruct();
	}
	return FEdGraphSchemaActionDefiningObject(DefiningObject);
}

UBlueprint* FEdGraphSchemaAction_BlueprintVariableBase::GetSourceBlueprint() const
{
	UClass* ClassToCheck = GetVariableClass();
	if (ClassToCheck == nullptr)
	{
		if (UFunction* Function = Cast<UFunction>(GetVariableScope()))
		{
			ClassToCheck = Function->GetOuterUClass();
		}
	}
	return UBlueprint::GetBlueprintFromClass(ClassToCheck);
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2LocalVar

int32 FEdGraphSchemaAction_K2LocalVar::GetReorderIndexInContainer() const
{
	return FBlueprintEditorUtils::FindLocalVariableIndex(GetSourceBlueprint(), Cast<UStruct>(GetVariableScope()), GetVariableName());
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2Graph

void FEdGraphSchemaAction_K2Graph::MovePersistentItemToCategory(const FText& NewCategoryName)
{
	if ((GraphType == EEdGraphSchemaAction_K2Graph::Function) || (GraphType == EEdGraphSchemaAction_K2Graph::Macro))
	{
		if(EdGraph->GetSchema()->GetClass()->GetFName() == TEXT("AnimationGraphSchema"))
		{
			FBlueprintEditorUtils::SetAnimationGraphLayerGroup(EdGraph, NewCategoryName);
		}
		else
		{
			FBlueprintEditorUtils::SetBlueprintFunctionOrMacroCategory(EdGraph, NewCategoryName);
		}
	}
}

int32 FEdGraphSchemaAction_K2Graph::GetReorderIndexInContainer() const
{
	return FBlueprintEditorUtils::FindIndexOfGraphInParent(EdGraph);
}

bool FEdGraphSchemaAction_K2Graph::ReorderToBeforeAction(TSharedRef<FEdGraphSchemaAction> OtherAction)
{
	if (OtherAction->GetTypeId() == GetTypeId())
	{
		const int32 OldIndex = GetReorderIndexInContainer();
		const int32 NewIndexToGoBefore = OtherAction->GetReorderIndexInContainer();

		if ((OldIndex != INDEX_NONE) && (OldIndex != NewIndexToGoBefore))
		{
			if (FBlueprintEditorUtils::MoveGraphBeforeOtherGraph(EdGraph, NewIndexToGoBefore, true))
			{
				// Change category to match the one we dragged on to as well
				MovePersistentItemToCategory(OtherAction->GetCategory());

				return true;
			}
		}
	}

	return false;
}

FEdGraphSchemaActionDefiningObject FEdGraphSchemaAction_K2Graph::GetPersistentItemDefiningObject() const
{
	UObject* DefiningObject = GetSourceBlueprint();
	if (UFunction* Func = GetFunction())
	{
		// Use the class where the function was initially introduced as the defining object
		while (Func->GetSuperFunction())
		{
			Func = Func->GetSuperFunction();
		}
		DefiningObject = Func->GetOuterUClassUnchecked();
	}

	return FEdGraphSchemaActionDefiningObject(DefiningObject, (void*)GraphType);
}

UBlueprint* FEdGraphSchemaAction_K2Graph::GetSourceBlueprint() const
{
	return FBlueprintEditorUtils::FindBlueprintForGraph(EdGraph);
}

UFunction* FEdGraphSchemaAction_K2Graph::GetFunction() const
{
	if (GraphType == EEdGraphSchemaAction_K2Graph::Function)
	{
		if (UBlueprint* SourceBlueprint = GetSourceBlueprint())
		{
			if (FuncName != NAME_None)
			{
				return FindUField<UFunction>(SourceBlueprint->SkeletonGeneratedClass, FuncName);
			}
		}
	}

	return nullptr;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2Delegate

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2ViewNode

UEdGraphNode* FEdGraphSchemaAction_K2NewNode::CreateNode(
	UEdGraph* ParentGraph,
	TArrayView<UEdGraphPin*> FromPins,
	const FVector2D Location,
	UK2Node* NodeTemplate,
	EK2NewNodeFlags Options)
{
	if (NodeTemplate)
	{
		return CreateNode(
			ParentGraph,
			FromPins,
			Location,
			[NodeTemplate](UEdGraph* InParentGraph)->UK2Node*
			{
				return DuplicateObject<UK2Node>(NodeTemplate, InParentGraph);
			},
			[](UK2Node*) {},
			Options
			);
	}
	return nullptr;
}

UEdGraphNode* FEdGraphSchemaAction_K2NewNode::CreateNode(
	UEdGraph* ParentGraph,
	TArrayView<UEdGraphPin*> FromPins,
	const FVector2D Location,
	TFunctionRef<UK2Node*(UEdGraph*)> ConstructionFn,
	TFunctionRef<void(UK2Node*)> InitializerFn,
	EK2NewNodeFlags Options)
{
	const bool bSelectNewNode = (Options & EK2NewNodeFlags::SelectNewNode) != EK2NewNodeFlags::None;
	const bool bGotoNode = (Options & EK2NewNodeFlags::GotoNewNode) != EK2NewNodeFlags::None;

	const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "K2_AddNode", "Add Node"), !IsInAsyncLoadingThread());
	ParentGraph->Modify();
	for(UEdGraphPin* FromPin : FromPins)
	{
		if (FromPin != nullptr)
		{
			FromPin->Modify();
		}
	}

	// Smart pointer that handles fixup after potential node reconstruction
	FWeakGraphPinPtr FromPinPtr = FromPins.Num() > 0 ? FromPins[0] : nullptr;

	// Duplicate template node to create new node
	UK2Node* ResultNode = ConstructionFn(ParentGraph);
	InitializerFn(ResultNode);
	if (ParentGraph->HasAnyFlags(RF_Transactional))
	{
		ResultNode->SetFlags(RF_Transactional);
	}

	ParentGraph->AddNode(ResultNode, true, bSelectNewNode);

	ResultNode->CreateNewGuid();
	ResultNode->PostPlacedNewNode();
	ResultNode->AllocateDefaultPins();

	// For input pins, new node will generally overlap node being dragged off
	// Work out if we want to visually push away from connected node
	int32 XLocation = static_cast<int32>(Location.X);
	if (FromPinPtr.IsValid() && FromPinPtr->Direction == EGPD_Input)
	{
		UEdGraphNode* PinNode = FromPinPtr->GetOwningNode();
		const double XDelta = FMath::Abs(PinNode->NodePosX - Location.X);

		if (XDelta < NodeDistance)
		{
			// Set location to edge of current node minus the max move distance
			// to force node to push off from connect node enough to give selection handle
			XLocation = PinNode->NodePosX - NodeDistance;
		}
	}
	ResultNode->NodePosX = XLocation;
	ResultNode->NodePosY = static_cast<int32>(Location.Y);
	ResultNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);

	// make sure to auto-wire after we position the new node (in case the 
	// auto-wire creates a conversion node to put between them)
	for (UEdGraphPin* FromPin : FromPins)
	{
		ResultNode->AutowireNewNode(FromPin);
	}

	// Update Analytics for the new nodes
	FBlueprintEditorUtils::AnalyticsTrackNewNode(ResultNode);
	// NOTE: At this point the node may have been reconstructed, depending on node type!

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);

	// See if we need to recompile skeleton after adding this node, or just mark dirty
	if (ResultNode->NodeCausesStructuralBlueprintChange())
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	else
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}

	// Clear any error messages resulting from placing a node.  They'll be flagged on the next compile
	ResultNode->ErrorMsg.Empty();
	ResultNode->bHasCompilerMessage = false;

	if (bGotoNode)
	{
		// Select existing node
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ResultNode);
	}

	return ResultNode;
}

UEdGraphNode* FEdGraphSchemaAction_K2NewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	EK2NewNodeFlags Options = EK2NewNodeFlags::None;
	if (bSelectNewNode)
	{
		Options |= EK2NewNodeFlags::SelectNewNode;
	}
	if (bGotoNode)
	{
		Options |= EK2NewNodeFlags::GotoNewNode;
	}
	return CreateNode(ParentGraph, MakeArrayView(&FromPin, 1), FDeprecateSlateVector2D(Location), NodeTemplate, Options);
}

UEdGraphNode* FEdGraphSchemaAction_K2NewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/) 
{
	EK2NewNodeFlags Options = EK2NewNodeFlags::None;
	if (bSelectNewNode)
	{
		Options |= EK2NewNodeFlags::SelectNewNode;
	}
	if (bGotoNode)
	{
		Options |= EK2NewNodeFlags::GotoNewNode;
	}
	return CreateNode(ParentGraph, FromPins, FDeprecateSlateVector2D(Location), NodeTemplate, Options);
}

void FEdGraphSchemaAction_K2NewNode::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( NodeTemplate );
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2ViewNode

UEdGraphNode* FEdGraphSchemaAction_K2ViewNode::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// If the node is valid, select it
	if (NodePtr)
	{
		// Select existing node
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NodePtr);
	}

	return nullptr;
}

UEdGraphNode* FEdGraphSchemaAction_K2ViewNode::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/) 
{
	PerformAction(ParentGraph, nullptr, Location, bSelectNewNode);
	return nullptr;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AssignDelegate

UEdGraphNode* FEdGraphSchemaAction_K2AssignDelegate::AssignDelegate(class UK2Node* NodeTemplate, class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2D Location, bool bSelectNewNode)
{
	UK2Node_AddDelegate* BindNode = nullptr;
	if (UK2Node_AddDelegate* AddDelegateTemplate = Cast<UK2Node_AddDelegate>(NodeTemplate))
	{
		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_AddNode", "Add Node") );
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		BindNode = Cast<UK2Node_AddDelegate>(CreateNode(ParentGraph, MakeArrayView(&FromPin, 1), Location, NodeTemplate, bSelectNewNode ? EK2NewNodeFlags::SelectNewNode : EK2NewNodeFlags::None));
		FMulticastDelegateProperty* DelegateProperty = BindNode ? CastField<FMulticastDelegateProperty>(BindNode->GetProperty()) : nullptr;
		if(DelegateProperty)
		{
			const FString FunctionName = FString::Printf(TEXT("%s_Event"), *DelegateProperty->GetName());
			UK2Node_CustomEvent* EventNode = UK2Node_CustomEvent::CreateFromFunction(
				FVector2D(Location.X - 150, Location.Y + 150), 
				ParentGraph, FunctionName, DelegateProperty->SignatureFunction, bSelectNewNode);
			if(EventNode)
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
				UEdGraphPin* OutDelegatePin = EventNode->FindPinChecked(UK2Node_CustomEvent::DelegateOutputName);
				UEdGraphPin* InDelegatePin = BindNode->GetDelegatePin();
				K2Schema->TryCreateConnection(OutDelegatePin, InDelegatePin);
			}
		}

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	return BindNode;
}

UEdGraphNode* FEdGraphSchemaAction_K2AssignDelegate::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	return AssignDelegate(NodeTemplate, ParentGraph, FromPin, FDeprecateSlateVector2D(Location), bSelectNewNode);
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_EventFromFunction

UEdGraphNode* FEdGraphSchemaAction_EventFromFunction::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UK2Node_CustomEvent* EventNode = nullptr;
	if (SignatureFunction)
	{
		if (FromPin)
		{
			// Make sure, that function is latest, so the names of parameters are proper.
			UK2Node_BaseMCDelegate* MCDelegateNode = Cast<UK2Node_BaseMCDelegate>(FromPin->GetOwningNode());
			UEdGraphPin* InputDelegatePin = MCDelegateNode ? MCDelegateNode->GetDelegatePin() : nullptr;
			UFunction* OriginalFunction = MCDelegateNode ? MCDelegateNode->GetDelegateSignature() : nullptr;
			if (OriginalFunction && 
				(OriginalFunction != SignatureFunction) && 
				(FromPin == InputDelegatePin) &&
				SignatureFunction->IsSignatureCompatibleWith(OriginalFunction))
			{
				SignatureFunction = OriginalFunction;
			}
		}

		const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_AddNode", "Add Node") );
		ParentGraph->Modify();
		if (FromPin)
		{
			FromPin->Modify();
		}

		EventNode = UK2Node_CustomEvent::CreateFromFunction(FDeprecateSlateVector2D(Location), ParentGraph, SignatureFunction->GetName() + TEXT("_Event"), SignatureFunction, bSelectNewNode);
		EventNode->AutowireNewNode(FromPin);

		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	return EventNode;
}

UEdGraphNode* FEdGraphSchemaAction_EventFromFunction::PerformAction(class UEdGraph* ParentGraph, TArray<UEdGraphPin*>& FromPins, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode* ResultNode = nullptr;

	if (FromPins.Num() > 0)
	{
		ResultNode = PerformAction(ParentGraph, FromPins[0], Location, bSelectNewNode);

		// Try autowiring the rest of the pins
		for (int32 Index = 1; Index < FromPins.Num(); ++Index)
		{
			ResultNode->AutowireNewNode(FromPins[Index]);
		}
	}
	else
	{
		ResultNode = PerformAction(ParentGraph, nullptr, Location, bSelectNewNode);
	}

	return ResultNode;
}

void FEdGraphSchemaAction_EventFromFunction::AddReferencedObjects(FReferenceCollector& Collector)
{
	FEdGraphSchemaAction::AddReferencedObjects(Collector);

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject(SignatureFunction);
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AddComponent

UEdGraphNode* FEdGraphSchemaAction_K2AddComponent::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	if (ComponentClass == nullptr)
	{
		return nullptr;
	}

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);
	UEdGraphNode* NewNode = FEdGraphSchemaAction_K2NewNode::PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);
	if ((NewNode != nullptr) && (Blueprint != nullptr))
	{
		UK2Node_AddComponent* AddCompNode = CastChecked<UK2Node_AddComponent>(NewNode);

		ensure(nullptr != Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass));
		// Then create a new template object, and add to array in
		UActorComponent* NewTemplate = NewObject<UActorComponent>(Blueprint->GeneratedClass, ComponentClass, NAME_None, RF_ArchetypeObject | RF_Public);
		Blueprint->ComponentTemplates.Add(NewTemplate);

		// Set the name of the template as the default for the TemplateName param
		UEdGraphPin* TemplateNamePin = AddCompNode->GetTemplateNamePinChecked();
		if (TemplateNamePin)
		{
			TemplateNamePin->DefaultValue = NewTemplate->GetName();
		}

		// Set the return type to be the type of the template
		UEdGraphPin* ReturnPin = AddCompNode->GetReturnValuePin();
		if (ReturnPin)
		{
			ReturnPin->PinType.PinSubCategoryObject = *ComponentClass;
		}

		// Set the asset
		if(ComponentAsset != nullptr)
		{
			FComponentAssetBrokerage::AssignAssetToComponent(NewTemplate, ComponentAsset);
		}

		AddCompNode->ReconstructNode();
	}

	FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);

	return NewNode;
}

void FEdGraphSchemaAction_K2AddComponent::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction_K2NewNode::AddReferencedObjects( Collector );

	// These don't get saved to disk, but we want to make sure the objects don't get GC'd while the action array is around
	Collector.AddReferencedObject( ComponentAsset );
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AddEvent

UEdGraphNode* FEdGraphSchemaAction_K2AddEvent::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	UEdGraphNode* NewNode = nullptr;
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_Event", "Add Event") );	

	UBlueprint const* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);

	UK2Node_Event const* ExistingEvent = nullptr;
	if (EventHasAlreadyBeenPlaced(Blueprint, &ExistingEvent))
	{
		check(ExistingEvent != nullptr);
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(ExistingEvent);
	}
	else
	{
		NewNode = FEdGraphSchemaAction_K2NewNode::PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);
	}

	return NewNode;
}

bool FEdGraphSchemaAction_K2AddEvent::EventHasAlreadyBeenPlaced(UBlueprint const* Blueprint, UK2Node_Event const** FoundEventOut /*= nullptr*/) const
{
	UK2Node_Event* ExistingEvent = nullptr;

	if (Blueprint != nullptr)
	{
		UK2Node_Event const* EventTemplate = Cast<UK2Node_Event const>(NodeTemplate);
		ExistingEvent = FBlueprintEditorUtils::FindOverrideForFunction(Blueprint, EventTemplate->EventReference.GetMemberParentClass(EventTemplate->GetBlueprintClassFromNode()), EventTemplate->EventReference.GetMemberName());
	}

	if (FoundEventOut != nullptr)
	{
		*FoundEventOut = ExistingEvent;
	}

	return (ExistingEvent != nullptr);
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AddCustomEvent

UEdGraphNode* FEdGraphSchemaAction_K2AddCustomEvent::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	const FScopedTransaction Transaction( NSLOCTEXT("UnrealEd", "K2_CustomEvent", "Add Custom Event") );
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(ParentGraph);

	UEdGraphNode* NewNode = FEdGraphSchemaAction_K2NewNode::PerformAction(ParentGraph, FromPin, Location, bSelectNewNode);

	// Set the name for the template to be a unique custom event name, so the generated node will have a default name that is already validated
	UK2Node_CustomEvent* CustomEventNode = CastChecked<UK2Node_CustomEvent>(NewNode);
	CustomEventNode->CustomFunctionName = FBlueprintEditorUtils::FindUniqueCustomEventName(Blueprint);

	return NewNode;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AddCallOnActor

UEdGraphNode* FEdGraphSchemaAction_K2AddCallOnActor::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// Snap the node placement location to the grid, ensures calculations later match up better
	const uint32 GridSnapSize = GetDefault<UEditorStyleSettings>()->GridSnapSize;
	FVector2f LocalLocation;
	LocalLocation.X = FMath::GridSnap(Location.X, GridSnapSize);
	LocalLocation.Y = FMath::GridSnap(Location.Y, GridSnapSize);

	// First use the base functionality to spawn the 'call function' node
	UEdGraphNode* CallNode = FEdGraphSchemaAction_K2NewNode::PerformAction(ParentGraph, FromPin, LocalLocation);
	const float FunctionNodeHeightUnsnapped = UEdGraphSchema_K2::EstimateNodeHeight( CallNode );

	// this is the guesstimate of the function node's height, snapped to grid units
	const float FunctionNodeHeight = FMath::GridSnap( FunctionNodeHeightUnsnapped, GridSnapSize );
	// this is roughly the middle of the function node height
	const float FunctionNodeMidY = static_cast<float>(LocalLocation.Y + FunctionNodeHeight * 0.5f);
	// this is the offset up from the mid point at which we start placing nodes 
	const float StartYOffset = (static_cast<float>((LevelActors.Num() > 0) ? LevelActors.Num() - 1 : 0) * -NodeLiteralHeight) * 0.5f;
	// The Y location we start placing nodes from
	const float ReferencedNodesPlacementYLocation = FunctionNodeMidY + StartYOffset;

	// Now we need to create the actor literal to wire up
	for ( int32 ActorIndex = 0; ActorIndex < LevelActors.Num(); ActorIndex++ )
	{
		AActor* LevelActor = LevelActors[ActorIndex];

		if(LevelActor != nullptr)
		{
			UK2Node_Literal* LiteralNode =  NewObject<UK2Node_Literal>(ParentGraph);
			ParentGraph->AddNode(LiteralNode, false, bSelectNewNode);
			LiteralNode->SetFlags(RF_Transactional);

			LiteralNode->SetObjectRef(LevelActor);
			LiteralNode->AllocateDefaultPins();
			LiteralNode->NodePosX = static_cast<int32>(LocalLocation.X - FunctionNodeLiteralReferencesXOffset);

			// this is the current offset down from the Y start location to place the next node at
			float CurrentNodeOffset = NodeLiteralHeight * static_cast<float>(ActorIndex);
			LiteralNode->NodePosY = static_cast<int32>(ReferencedNodesPlacementYLocation + CurrentNodeOffset);

			LiteralNode->SnapToGrid(GridSnapSize);

			// Connect the literal out to the self of the call
			UEdGraphPin* LiteralOutput = LiteralNode->GetValuePin();
			UEdGraphPin* CallSelfInput = CallNode->FindPin(UEdGraphSchema_K2::PN_Self);
			if(LiteralOutput != nullptr && CallSelfInput != nullptr)
			{
				LiteralOutput->MakeLinkTo(CallSelfInput);
			}
		}
	}

	return CallNode;
}

void FEdGraphSchemaAction_K2AddCallOnActor::AddReferencedObjects( FReferenceCollector& Collector )
{
	FEdGraphSchemaAction_K2NewNode::AddReferencedObjects( Collector );

	Collector.AddReferencedObjects(LevelActors);
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2AddComment

UEdGraphNode* FEdGraphSchemaAction_K2AddComment::PerformAction(class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/)
{
	// Add menu item for creating comment boxes
	UEdGraphNode_Comment* CommentTemplate = NewObject<UEdGraphNode_Comment>();

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(ParentGraph);

	FVector2f SpawnLocation = Location;

	FSlateRect Bounds;
	if ((Blueprint != nullptr) && FKismetEditorUtilities::GetBoundsForSelectedNodes(Blueprint, Bounds, 50.0f))
	{
		CommentTemplate->SetBounds(Bounds);
		SpawnLocation.X = CommentTemplate->NodePosX;
		SpawnLocation.Y = CommentTemplate->NodePosY;
	}

	UEdGraphNode* NewNode = FEdGraphSchemaAction_NewNode::SpawnNodeFromTemplate<UEdGraphNode_Comment>(ParentGraph, CommentTemplate, SpawnLocation, bSelectNewNode);

	// Update Analytics for these nodes
	FBlueprintEditorUtils::AnalyticsTrackNewNode( NewNode );

	// Mark Blueprint as structurally modified since
	// UK2Node_Comment::NodeCausesStructuralBlueprintChange used to return true
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);

	return NewNode;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2TargetNode

UEdGraphNode* FEdGraphSchemaAction_K2TargetNode::PerformAction( class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/ ) 
{
	FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NodeTemplate);
	return nullptr;
}

/////////////////////////////////////////////////////
// FEdGraphSchemaAction_K2PasteHere

UEdGraphNode* FEdGraphSchemaAction_K2PasteHere::PerformAction( class UEdGraph* ParentGraph, UEdGraphPin* FromPin, const FVector2f& Location, bool bSelectNewNode/* = true*/ ) 
{
	FKismetEditorUtilities::PasteNodesHere(ParentGraph, FDeprecateSlateVector2D(Location));
	return nullptr;
}


