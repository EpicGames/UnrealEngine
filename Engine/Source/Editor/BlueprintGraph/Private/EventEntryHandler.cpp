// Copyright Epic Games, Inc. All Rights Reserved.

#include "EventEntryHandler.h"

#include "BPTerminal.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformMath.h"
#include "K2Node_Event.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "UObject/NameTypes.h"

class UEdGraphNode;
struct FBlueprintCompiledStatement;

void FKCHandler_EventEntry::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	const bool bIsDelegateOutput = (NULL != Net) && (UK2Node_Event::DelegateOutputName == Net->PinName);
	if(!bIsDelegateOutput)
	{
		if (!Context.IsEventGraph())
		{
			Context.MessageLog.Error(TEXT("Event node @@ registers net @@ in a non-event graph."), Net ? Net->GetOwningNodeUnchecked() : nullptr, Net);
			return;
		}

		// This net is an event parameter; push to be a private class member variable
		FBPTerminal* Term = Context.CreateLocalTerminal(ETerminalSpecification::TS_ForcedShared);
		Term->CopyFromPin(Net, Context.NetNameMap->MakeValidName(Net));

		Context.NetMap.Add(Net, Term);
	}
}

void FKCHandler_EventEntry::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	// Generate the output impulse from this node
	FBlueprintCompiledStatement& Statement = GenerateSimpleThenGoto(Context, *Node);
}
