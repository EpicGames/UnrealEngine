// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node.h"
#include "BlueprintCompilationManager.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UnrealType.h"
#include "UObject/CoreRedirects.h"
#include "EdGraph/EdGraphPin.h"
#include "UObject/Interface.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GraphEditorSettings.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MacroInstance.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Editor/EditorEngine.h"
#include "Misc/OutputDeviceNull.h"

#include "Kismet2/Breakpoint.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "KismetCompiler.h"
#include "PropertyCustomizationHelpers.h"

#include "ObjectEditorUtils.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/FrameworkObjectVersion.h"

#include "Kismet2/WatchedPin.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node)

#define LOCTEXT_NAMESPACE "K2Node"

// File-Scoped Globals
static const uint32 MaxArrayPinTooltipLineCount = 10;

namespace UK2Node_Private
{
	struct FPinRenamedAnnotation
	{
		mutable FOnUserDefinedPinRenamed PinRenamedEvent;
		bool bIsDefault = true;

		bool IsDefault() const
		{
			return bIsDefault;
		}
	};
	FUObjectAnnotationSparse<FPinRenamedAnnotation, true> GOnUserDefinedPinRenamedAnnotation;
}

/////////////////////////////////////////////////////
// UK2Node

UK2Node::UK2Node(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAllButExec;
}

void UK2Node::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	const FName PropertyName = PropertyThatWillChange ? PropertyThatWillChange->GetFName() : FName();
	if (PropertyName == FName() ||
		// don't dirty on cosmetic property changes:
		(	PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodePosX) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodePosY) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodeWidth) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, NodeHeight) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, bCommentBubblePinned) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, bCommentBubbleVisible) &&
			PropertyName != GET_MEMBER_NAME_CHECKED(UEdGraphNode, bCommentBubbleMakeVisible) ))
	{
		// we call 'preeditchange' on uk2nodes in the transaction buffer, and they
		// may be outered to the transient package:
		if (HasValidBlueprint())
		{
			UBlueprint* BP = GetBlueprint();
			BP->Status = BS_Dirty;
		}
		
	}
}

void UK2Node::PostLoad()
{
#if WITH_EDITORONLY_DATA
	// Clean up win watches for any deprecated pins we are about to remove in Super::PostLoad
	if (DeprecatedPins.Num() && HasValidBlueprint())
	{
		UBlueprint* BP = GetBlueprint();
		check(BP);

		// patch DeprecatedPinWatches to WatchedPins:
		for (int32 WatchIdx = BP->DeprecatedPinWatches.Num() - 1; WatchIdx >= 0; --WatchIdx)
		{
			UEdGraphPin_Deprecated* WatchedPin = BP->DeprecatedPinWatches[WatchIdx];
			if (DeprecatedPins.Contains(WatchedPin))
			{
				if (UEdGraphPin* NewPin = UEdGraphPin::FindPinCreatedFromDeprecatedPin(WatchedPin))
				{
					FKismetDebugUtilities::AddPinWatch(BP, NewPin);
				}

				BP->DeprecatedPinWatches.RemoveAt(WatchIdx);
			}
		}
		
	}
#endif // WITH_EDITORONLY_DATA

	Super::PostLoad();
}

void UK2Node::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.IsSaving())
	{
		if (Ar.IsObjectReferenceCollector() || Ar.Tell() < 0)
		{
			// When this is a reference collector/modifier, serialize some pins as structs
			FixupPinStringDataReferences(&Ar);
		}
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading() && ((Ar.GetPortFlags() & PPF_Duplicate) == 0))
	{
		// Fix up pin default values, must be done before post load
		FixupPinDefaultValues();

		if (GIsEditor)
		{
			// We need to serialize string data references on load in editor builds so the cooker knows about them
			FixupPinStringDataReferences(nullptr);
		}
	}
}

bool UK2Node::HasNonEditorOnlyReferences() const
{
	return true;
}

void UK2Node::FixupPinDefaultValues()
{
	const FPackageFileVersion LinkerUEVersion = GetLinkerUEVersion();
	const int32 LinkerFrameworkVersion = GetLinkerCustomVersion(FFrameworkObjectVersion::GUID);
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Swap "new" default error tolerance value with zero on vector/rotator equality nodes, in order to preserve current behavior in existing blueprints.
	if(LinkerUEVersion < VER_UE4_BP_MATH_VECTOR_EQUALITY_USES_EPSILON)
	{
		static const FString VectorsEqualFunctionEpsilonPinName = TEXT("KismetMathLibrary.EqualEqual_VectorVector.ErrorTolerance");
		static const FString VectorsNotEqualFunctionEpsilonPinName = TEXT("KismetMathLibrary.NotEqual_VectorVector.ErrorTolerance");
		static const FString RotatorsEqualFunctionEpsilonPinName = TEXT("KismetMathLibrary.EqualEqual_RotatorRotator.ErrorTolerance");
		static const FString RotatorsNotEqualFunctionEpsilonPinName = TEXT("KismetMathLibrary.NotEqual_RotatorRotator.ErrorTolerance");

		bool bFoundPin = false;
		for(int32 i = 0; i < Pins.Num() && !bFoundPin; ++i)
		{
			UEdGraphPin* Pin = Pins[i];
			check(Pin);

			TArray<FString> RedirectPinNames;
			GetRedirectPinNames(*Pin, RedirectPinNames);

			for (const FString& PinName : RedirectPinNames)
			{
				if((Pin->DefaultValue == Pin->AutogeneratedDefaultValue)
					&& (PinName == VectorsEqualFunctionEpsilonPinName
						|| PinName == VectorsNotEqualFunctionEpsilonPinName
						|| PinName == RotatorsEqualFunctionEpsilonPinName
						|| PinName == RotatorsNotEqualFunctionEpsilonPinName))
				{
					bFoundPin = true;
					K2Schema->TrySetDefaultValue(*Pin, TEXT("0.0"));
					break;
				}
			}
		}
	}

	// Fix soft object ptr pins
	if (LinkerFrameworkVersion < FFrameworkObjectVersion::ChangeAssetPinsToString)
	{
		for (int32 i = 0; i < Pins.Num(); ++i)
		{
			UEdGraphPin* Pin = Pins[i];
			if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
			{
				// Fix old assetptr pins
				if (Pin->DefaultObject && Pin->DefaultValue.IsEmpty())
				{
					Pin->DefaultValue = Pin->DefaultObject->GetPathName();
					Pin->DefaultObject = nullptr;
				}
			}
		}
	}
}

void UK2Node::FixupPinStringDataReferences(FArchive* SavingArchive)
{
	// This code expands some pin types into their real representation, optionally serializes them, and then writes them out as strings again
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	FLinkerLoad* LinkerLoad = GetLinker();

	// Can't do any fixups without an archive of some sort
	if (!SavingArchive && !LinkerLoad)
	{
		return;
	}

	for (UEdGraphPin* Pin : Pins)
	{
		if (!Pin->bDefaultValueIsIgnored && !Pin->DefaultValue.IsEmpty())
		{
			// We skip this for structs like FVector that have a custom format for default values
			if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && Pin->PinType.PinSubCategoryObject.IsValid() && !K2Schema->PinHasCustomDefaultFormat(*Pin))
			{
				UScriptStruct* Struct = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
				if (Struct)
				{
					// While loading, our user struct may not have been linked yet
					Struct->ConditionalPreload();
					
					FSoftObjectPathSerializationScope SetPackage(GetOutermost()->GetFName(), NAME_None, ESoftObjectPathCollectType::AlwaysCollect, ESoftObjectPathSerializeType::SkipSerializeIfArchiveHasSize);

					TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(Struct));
					StructData->SetPackage(GetOutermost());

					// Import the literal text to a dummy struct and then serialize that. Hard object references will not properly import, this is only useful for soft references!
					FOutputDeviceNull NullOutput;
					Struct->ImportText(*Pin->DefaultValue, StructData->GetStructMemory(), this, PPF_SerializedAsImportText, &NullOutput, Pin->PinName.ToString());

					if (SavingArchive)
					{
						// If we're saving, use the archive to do any replacements
						Struct->SerializeItem(*SavingArchive, StructData->GetStructMemory(), nullptr);
					}
					
					// Convert back to the default value string as we might have changed
					FString NewValue;
					Struct->ExportText(NewValue, StructData->GetStructMemory(), StructData->GetStructMemory(), this, PPF_SerializedAsImportText, nullptr);

					if (Pin->DefaultValue != NewValue)
					{
						Pin->DefaultValue = NewValue;
					}
				}
			}

			if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
			{
				FSoftObjectPathSerializationScope SetPackage(GetOutermost()->GetFName(), NAME_None, ESoftObjectPathCollectType::AlwaysCollect, ESoftObjectPathSerializeType::SkipSerializeIfArchiveHasSize);

				FSoftObjectPath TempRef(Pin->DefaultValue);

				if (SavingArchive)
				{
					// Serialize it directly, this won't do anything if it's a real archive like LinkerSave
					*SavingArchive << TempRef;
				}
				else
				{
					// Transform it as strings
					TempRef.PostLoadPath(LinkerLoad);
					TempRef.PreSavePath();
				}
				
				Pin->DefaultValue = TempRef.ToString();
			}
		}
	}
}

FText UK2Node::GetToolTipHeading() const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
	FText Heading = FText::GetEmpty();

	if (Blueprint)
	{
		if (FBlueprintBreakpoint* ExistingBreakpoint = FKismetDebugUtilities::FindBreakpointForNode(this, Blueprint))
		{
			if (ExistingBreakpoint->IsEnabled())
			{
				Heading = LOCTEXT("EnabledBreakpoint", "Active Breakpoint - Execution will break at this location.");
			}
			else
			{
				Heading = LOCTEXT("DisabledBreakpoint", "Disabled Breakpoint");
			}
		}
	}

	return Heading;
}

bool UK2Node::CreatePinsForFunctionEntryExit(const UFunction* Function, bool bForFunctionEntry)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// if the generated class is not up to date, use the skeleton's class function to create pins:
	Function = FBlueprintEditorUtils::GetMostUpToDateFunction(Function);

	// Create the inputs and outputs
	bool bAllPinsGood = true;
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Param = *PropIt;

		// Don't create a new pin if one exists already! 
		// @see UE-79032, UE-58390
		if (FindPin(Param->GetFName()))
		{
			continue;
		}

		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);

		if (bIsFunctionInput == bForFunctionEntry)
		{
			const EEdGraphPinDirection Direction = bForFunctionEntry ? EGPD_Output : EGPD_Input;

			UEdGraphPin* Pin = CreatePin(Direction, NAME_None, Param->GetFName());
			const bool bPinGood = K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType);
			K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
			
			UK2Node_CallFunction::GeneratePinTooltipFromFunction(*Pin, Function);

			bAllPinsGood = bAllPinsGood && bPinGood;
		}
	}

	return bAllPinsGood;
}

void UK2Node::AutowireNewNode(UEdGraphPin* FromPin)
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	
	// Do some auto-connection
	if (FromPin)
	{
		TSet<UEdGraphNode*> NodeList;

		// sometimes we don't always find an ideal connection, but we want to exhaust 
		// all our options first... this stores a secondary less-ideal pin to connect to, if nothing better was found
		UEdGraphPin* BackupConnection = NULL;
		// If not dragging an exec pin, auto-connect from dragged pin to first compatible pin on the new node
		for (int32 i=0; i<Pins.Num(); i++)
		{
			UEdGraphPin* Pin = Pins[i];
			check(Pin);

			if (Pin->bHidden)
			{
				// Never consider for auto-wiring a hidden pin being connected to a Wildcard. It is never what the user expects
				if (FromPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
				{
					continue;
				}

				// Never connect wires to hidden WorldContextObject pins
				if (UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(this))
				{				
					if (UFunction* NodeTargetFunction = CallFunctionNode->GetTargetFunction())
					{
						const FString& WorldContextPinName = NodeTargetFunction->GetMetaData(FBlueprintMetadata::MD_WorldContext);
						if (!WorldContextPinName.IsEmpty() && WorldContextPinName == Pin->PinName.ToString())
						{
							continue;
						}
					}
				}
			}

			ECanCreateConnectionResponse ConnectResponse = K2Schema->CanCreateConnection(FromPin, Pin).Response;
			if ((FromPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A))
			{
				InsertNewNode(FromPin, Pin, NodeList);

				// null out the backup connection (so we don't attempt to make it 
				// once we exit the loop... we successfully made this connection!)
				BackupConnection = NULL;
				break;
			}
			else if ((BackupConnection == NULL) && (ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE))
			{
				// save this off, in-case we don't make any connection at all
				BackupConnection = Pin;
			}
			else if ((ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE) ||
				(ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_A) ||
				(ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_B) ||
				(ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_BREAK_OTHERS_AB))
			{
				if (K2Schema->TryCreateConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}

				// null out the backup connection (so we don't attempt to make it 
				// once we exit the loop... we successfully made this connection!)
				BackupConnection = nullptr;
				break;
			}
			else if(ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE_WITH_PROMOTION)
			{
				if (K2Schema->CreatePromotedConnection(FromPin, Pin))
				{
					NodeList.Add(FromPin->GetOwningNode());
					NodeList.Add(this);
				}
				BackupConnection = nullptr;
				break;
			}
		}

		// if we didn't find an ideal connection, then lets connect this pin to 
		// the BackupConnection (something, like a connection that requires a conversion node, etc.)
		if ((BackupConnection != nullptr) && K2Schema->TryCreateConnection(FromPin, BackupConnection))
		{
			NodeList.Add(FromPin->GetOwningNode());
			NodeList.Add(this);
		}

		// If we were not dragging an exec pin, but it was an output pin, try and connect the Then and Execute pins
		if ((FromPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec  && FromPin->Direction == EGPD_Output))
		{
			UEdGraphNode* FromPinNode = FromPin->GetOwningNode();
			UEdGraphPin* FromThenPin = FromPinNode->FindPin(UEdGraphSchema_K2::PN_Then);
			
			// Prefer a pin named "Execute", but also accept "Exec" and "In", since some loop macros use those as their input pin names,
			// and it's easier to add a workaround here that allows them to be better autowired than add more "special" pin names to the schema
			UEdGraphPin* ToExecutePin = FindPin(UEdGraphSchema_K2::PN_Execute);

			static const FName PN_Exec = TEXT("exec");
			if (!ToExecutePin) ToExecutePin = FindPin(PN_Exec);

			static const FName PN_In = TEXT("in");
			if (!ToExecutePin) ToExecutePin = FindPin(PN_In);

			if ((FromThenPin != nullptr) && (FromThenPin->LinkedTo.Num() == 0) && (ToExecutePin != nullptr) && K2Schema->ArePinsCompatible(FromThenPin, ToExecutePin, NULL))
			{
				if (K2Schema->TryCreateConnection(FromThenPin, ToExecutePin))
				{
					NodeList.Add(FromPinNode);
					NodeList.Add(this);
				}
			}
		}

		// Send all nodes that received a new pin connection a notification
		for (UEdGraphNode* Node : NodeList)
		{
			Node->NodeConnectionListChanged();
		}
	}
}

void UK2Node::InsertNewNode(UEdGraphPin* FromPin, UEdGraphPin* NewLinkPin, TSet<UEdGraphNode*>& OutNodeList)
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	// The pin we are creating from already has a connection that needs to be broken. Being an exec pin, we want to "insert" the new node in between, so that the output of the new node is hooked up to 
	UEdGraphPin* OldLinkedPin = FromPin->LinkedTo[0];
	check(OldLinkedPin);

	FromPin->BreakAllPinLinks();

	// Hook up the old linked pin to the first valid output pin on the new node
	for (int32 OutpinPinIdx=0; OutpinPinIdx<Pins.Num(); OutpinPinIdx++)
	{
		UEdGraphPin* OutputExecPin = Pins[OutpinPinIdx];
		check(OutputExecPin);
		if (ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE == K2Schema->CanCreateConnection(OldLinkedPin, OutputExecPin).Response)
		{
			if (K2Schema->TryCreateConnection(OldLinkedPin, OutputExecPin))
			{
				OutNodeList.Add(OldLinkedPin->GetOwningNode());
				OutNodeList.Add(this);
			}
			break;
		}
	}

	if (K2Schema->TryCreateConnection(FromPin, NewLinkPin))
	{
		OutNodeList.Add(FromPin->GetOwningNode());
		OutNodeList.Add(this);
	}
}

void UK2Node::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "GraphNode" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
}

void UK2Node::PinConnectionListChanged(UEdGraphPin* Pin) 
{

	// If the pin has been connected, clear the default values so that we don't hold on to references
	if (Pin->LinkedTo.Num() > 0) 
	{
		// We don't want to reset Output pin defaults, that breaks Function entry nodes
		if (!Pin->bOrphanedPin && Pin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			UEdGraph* OuterGraph = GetGraph();

			// Verify that we have a proper outer at this point, such that the schema will be valid
			if (OuterGraph && OuterGraph->Schema)
			{
				const UEdGraphSchema_K2* Schema = CastChecked<const UEdGraphSchema_K2>(GetSchema());
				Schema->ResetPinToAutogeneratedDefaultValue(Pin);
			}
		}
	}
	// If we're not linked and this pin should no longer exist as part of the node, remove it
	else if (Pin->bOrphanedPin)
	{
		UEdGraph* OuterGraph = GetGraph();

		if (Pin->ParentPin == nullptr)
		{
			RemovePin(Pin);
		}
		else
		{
			const UEdGraphSchema_K2* Schema = CastChecked<const UEdGraphSchema_K2>(GetSchema());
			TFunction<void(UEdGraphPin*)> RemoveNestedPin = [this, Schema, &RemoveNestedPin](UEdGraphPin* NestedPin)
			{
				Modify();
				ensure(Pins.Remove(NestedPin) == 1);

				if (UEdGraphPin* ParentPin = NestedPin->ParentPin)
				{
					ParentPin->Modify();
					ensure(ParentPin->SubPins.Remove(NestedPin) == 1);

					if (ParentPin->SubPins.Num() == 0)
					{
						if (ParentPin->bOrphanedPin)
						{
							RemoveNestedPin(ParentPin);
						}
						else
						{
							Schema->RecombinePin(NestedPin);
						}
					}
				}

				NestedPin->MarkAsGarbage();
			};

			RemoveNestedPin(Pin);
		}

		bool bOrphanedPinsGone = true;
		for (UEdGraphPin* RemainingPin : Pins)
		{
			if (RemainingPin->bOrphanedPin)
			{
				bOrphanedPinsGone = false;
				break;
			}
		}

		if (bOrphanedPinsGone)
		{
			ClearCompilerMessage();
		}

		if (OuterGraph)
		{
			OuterGraph->NotifyNodeChanged(this);
		}

		Pin = nullptr;
	}

	if (Pin)
	{
		NotifyPinConnectionListChanged(Pin);
	}
}

UObject* UK2Node::GetJumpTargetForDoubleClick() const
{
    return GetReferencedLevelActor();
}

bool UK2Node::CanJumpToDefinition() const
{
	return GetJumpTargetForDoubleClick() != nullptr;
}

void UK2Node::JumpToDefinition() const
{
	if (UObject* HyperlinkTarget = GetJumpTargetForDoubleClick())
	{
		FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(HyperlinkTarget);
	}
}

bool UK2Node::CanPlaceBreakpoints() const
{
	// Pure nodes have no execs and therefore cannot have breakpoints placed on them
	if(IsNodePure())
	{
		return false;
	}

	// If this node is within a macro or interface blueprint then it cannot have breakpoints
	// added to it as they will get expanded during compilation and never get hit. 
	if(const UEdGraph* Graph = GetGraph())
	{
		if (const UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph))
		{
			return
				!(Blueprint->BlueprintType == BPTYPE_MacroLibrary ||
				Blueprint->BlueprintType == BPTYPE_Interface ||
				Blueprint->MacroGraphs.Contains(Graph));
		}
	}
	
	return true;
}

void UK2Node::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	AllocateDefaultPins();

	RestoreSplitPins(OldPins);
}

void UK2Node::PostReconstructNode()
{
	if (!IsTemplate())
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			// fix up any pin data if it needs to 
			for (UEdGraphPin* CurrentPin : Pins)
			{
				const FName& PinCategory = CurrentPin->PinType.PinCategory;

				// fix up enum names if it exists in enum redirects
				if (PinCategory == UEdGraphSchema_K2::PC_Byte)
				{
					UEnum* EnumPtr = Cast<UEnum>(CurrentPin->PinType.PinSubCategoryObject.Get());
					if (EnumPtr)
					{
						const FString& PinValue = CurrentPin->DefaultValue;
						// Check for redirected enum names
						int32 EnumIndex = EnumPtr->GetIndexByNameString(PinValue);
						if (EnumIndex != INDEX_NONE)
						{
							FString EnumName = EnumPtr->GetNameStringByIndex(EnumIndex);

							// if the name does not match with pin value, override pin value
							if (EnumName != PinValue)
							{
								// I'm not marking package as dirty 
								// as I know that's not going to work during serialize or post load
								CurrentPin->DefaultValue = EnumName;
								continue;
							}
						}
					}
				}
				else if (PinCategory == UEdGraphSchema_K2::PC_Object)
				{
					UClass const* PinClass = Cast<UClass const>(CurrentPin->PinType.PinSubCategoryObject.Get());
					if ((PinClass != nullptr) && PinClass->IsChildOf(UInterface::StaticClass()))
					{
						CurrentPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Interface;
					}
				}
			}
		}
	}
}

void UK2Node::ReconstructNode()
{
	// Only mark as dirty if inside a transactional scope; otherwise, there's no need to re-save the package
	Modify(/*bAlwaysMarkDirty =*/ false);

	// Clear previously set messages
	ErrorMsg.Reset();

	// Make sure we have a valid Blueprint owner, so we don't crash the editor attempting to access it below.
	if (!ensureMsgf(HasValidBlueprint(), TEXT("UK2Node::ReconstructNode(): Attempting to reconstruct [%s] without a valid Blueprint owner (this is unexpected)."), *GetFullName()))
	{
		return;
	}

	UBlueprint* Blueprint = GetBlueprint();

	FLinkerLoad* Linker = Blueprint->GetLinker();
	const UEdGraphSchema* Schema = GetSchema();

	// Break any links to 'orphan' pins
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		TArray<class UEdGraphPin*> LinkedToCopy = Pin->LinkedTo;
		for (int32 LinkIdx = 0; LinkIdx < LinkedToCopy.Num(); LinkIdx++)
		{
			UEdGraphPin* OtherPin = LinkedToCopy[LinkIdx];
			// If we are linked to a pin that its owner doesn't know about, break that link
			if ((OtherPin == nullptr) || !OtherPin->GetOwningNodeUnchecked() || !OtherPin->GetOwningNode()->Pins.Contains(OtherPin))
			{
				Pin->LinkedTo.Remove(OtherPin);
			}

			if (Blueprint->bIsRegeneratingOnLoad && Linker->UEVer() < VER_UE4_INJECT_BLUEPRINT_STRUCT_PIN_CONVERSION_NODES)
			{
				if (OtherPin == nullptr || (Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Struct))
				{
					continue;
				}

				if (Schema->CanCreateConnection(Pin, OtherPin).Response == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE_WITH_CONVERSION_NODE)
				{
					Schema->CreateAutomaticConversionNodeAndConnections(Pin, OtherPin);
				}
			}
		}
	}

	// Move the existing pins to a saved array
	TArray<UEdGraphPin*> OldPins(Pins);
	Pins.Reset();

	// Recreate the new pins
	TMap<UEdGraphPin*, UEdGraphPin*> NewPinsToOldPins;
	ReallocatePinsDuringReconstruction(OldPins);
	RewireOldPinsToNewPins(OldPins, Pins, &NewPinsToOldPins);

	// Let subclasses do any additional work
	PostReconstructNode();

	if (Cast<UK2Node_CallFunction>(this) &&
		Blueprint->CurrentMessageLog &&
		GetLinkerCustomVersion(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::PinDefaultValuesVerified)
	{
		for (UEdGraphPin* NewPin : Pins)
		{
			UEdGraphPin** OldPinEntry = NewPinsToOldPins.Find(NewPin);
			if (OldPinEntry && *OldPinEntry)
			{
				UEdGraphPin* OldPin = *OldPinEntry;
				if (NewPin->LinkedTo.Num() == 0
					&& !OldPin->AutogeneratedDefaultValue.IsEmpty()
					&& Schema->DoesDefaultValueMatchAutogenerated(*OldPin))
				{
					// Rather than compare default value strings directly, we utilize the schema to convert and compare against the actual default value type.
					auto HasPinDefaultValueChanged = [Schema](const UEdGraphPin& PinRef, FString& PinDefaultValueRef, const FString& OldPinDefaultValue) -> bool
					{
						TGuardValue<FString> TempSetPinDefaultValue(PinDefaultValueRef, OldPinDefaultValue);
						return !Schema->DoesDefaultValueMatchAutogenerated(PinRef);
					};

					// Compare default value strings by temporarily setting the default value and auto-generated default value strings to the equivalent default
					// value string from the old pin that we wish to compare it against. In some cases, the value string may have changed, but wouldn't otherwise
					// result in a different value when converted.
					// 
					// Example: For vector types, {1, 1, 1} is equivalent to {1.000, 1.000, 1.000}, which can be considered a match that would not require a resave.
					if (HasPinDefaultValueChanged(*NewPin, NewPin->DefaultValue, OldPin->AutogeneratedDefaultValue)
						&& HasPinDefaultValueChanged(*NewPin, NewPin->AutogeneratedDefaultValue, OldPin->DefaultValue))
					{
						Blueprint->CurrentMessageLog->Warning(*LOCTEXT("VerifyDefaultValues", "Default value for @@ on @@ has changed and this asset is from a version that may have had incorrect default value information - verify and resave").ToString(), NewPin, this);
					}
				}
			}
		}
	}

	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node::GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const
{
	RedirectPinNames.Add(Pin.PinName.ToString());
}

UK2Node::ERedirectType UK2Node::ShouldRedirectParam(const TArray<FString>& OldPinNames, FName& NewPinName, const UK2Node * NewPinNode) const 
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UK2Node::ShouldRedirectParam"), STAT_LinkerLoad_ShouldRedirectParam, STATGROUP_LoadTimeVerbose);

	if ( ensure(NewPinNode) )
	{
		for (const FString& OldPinName : OldPinNames)
		{
			const FCoreRedirect* ValueRedirect = nullptr;
			FCoreRedirectObjectName NewRedirectName;
			
			if (FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags::Type_Property, OldPinName, NewRedirectName, &ValueRedirect))
			{
				NewPinName = NewRedirectName.ObjectName;
				return (ValueRedirect ? ERedirectType_Value : ERedirectType_Name);
			}
		}
	}

	return ERedirectType_None;
}

void UK2Node::RestoreSplitPins(TArray<UEdGraphPin*>& OldPins)
{
	UEdGraph* OuterGraph = GetGraph();
	if (!OuterGraph || !OuterGraph->Schema)
	{
		return;
	}

	// necessary to recreate split pins and keep their wires
	TArray<UEdGraphPin*> UnmatchedSplitPins;
	for (UEdGraphPin* OldPin : OldPins)
	{
		if (OldPin->ParentPin)
		{
			// find the new pin that corresponds to parent, and split it if it isn't already split
			bool bMatched = false;
			for (UEdGraphPin* NewPin : Pins)
			{
				// The pin we're searching for has the same direction, is not a container, has the same name as our parent pin, and is either a wildcard or a struct
				// We allow sub categories of struct to change because it may be changing to a type that has the same members
				if ((NewPin->Direction == OldPin->Direction) && !NewPin->PinType.IsContainer() && (NewPin->PinName == OldPin->ParentPin->PinName)
					&& (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard || NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct))
				{
					bMatched = true;

					// Make sure we're not dealing with a menu node
					if (NewPin->SubPins.Num() == 0)
					{
						if (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
						{
							NewPin->PinType = OldPin->ParentPin->PinType;
						}
						
						GetSchema()->SplitPin(NewPin, false);
						break;
					}
				}
			}

			if (!bMatched)
			{
				UnmatchedSplitPins.Add(OldPin);
			}
		}
	}

	// try and use redirectors to match remaining pins:
	for(UEdGraphPin* UnmatchedOldPin : UnmatchedSplitPins)
	{
		TArray<FString> OldPinNames;
		GetRedirectPinNames(*UnmatchedOldPin->ParentPin, OldPinNames);

		for (UEdGraphPin* NewPin : Pins)
		{
			FName NewPinName;
			if (ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, this) == ERedirectType_Name && NewPinName == NewPin->PinName)
			{
				// Make sure we're not dealing with a menu node
				if (NewPin->SubPins.Num() == 0)
				{
					if (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
					{
						NewPin->PinType = UnmatchedOldPin->ParentPin->PinType;
					}

					GetSchema()->SplitPin(NewPin, false);
					break;
				}
			}
		}
	}
}

UK2Node::ERedirectType UK2Node::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names and directions do match
	if (NewPin->PinName == OldPin->PinName && NewPin->Direction == OldPin->Direction)
	{
		// If the old pin had a default value, only match the new pin if the type is compatible:
		const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
		if (K2Schema && 
			OldPin->Direction == EGPD_Input &&
			OldPin->LinkedTo.Num() == 0 &&
			!OldPin->DoesDefaultValueMatchAutogenerated() &&
			!K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType))
		{
			RedirectType = ERedirectType_DefaultValue;
		}
		else
		{
			RedirectType = ERedirectType_Name;
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			if (OldPin->ParentPin == nullptr)
			{
				// if you don't have matching pin, now check if there is any redirect param set
				TArray<FString> OldPinNames;
				GetRedirectPinNames(*OldPin, OldPinNames);

				FName NewPinName;
				RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

				// make sure they match
				if ((RedirectType != ERedirectType_None) && (NewPin->PinName != NewPinName))
				{
					RedirectType = ERedirectType_None;
				}
			}
			else
			{
				struct FPropertyDetails
				{
					const UEdGraphPin* Pin;
					FString PropertyName;

					FPropertyDetails(const UEdGraphPin* InPin, const FString& InPropertyName)
						: Pin(InPin), PropertyName(InPropertyName)
					{
					}
				};

				TArray<FPropertyDetails> ParentHierarchy;
				FString NewPinNameStr;
				{
					const UEdGraphPin* CurPin = OldPin;
					do 
					{
						ParentHierarchy.Add(FPropertyDetails(CurPin, CurPin->PinName.ToString().RightChop(CurPin->ParentPin->PinName.ToString().Len() + 1)));
						CurPin = CurPin->ParentPin;
					} while (CurPin->ParentPin);

					// if you don't have matching pin, now check if there is any redirect param set
					TArray<FString> OldPinNames;
					GetRedirectPinNames(*CurPin, OldPinNames);

					FName NewPinName;
					RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

					NewPinNameStr = (RedirectType == ERedirectType_None ? CurPin->PinName.ToString() : NewPinName.ToString());
				}

				for (int32 ParentIndex = ParentHierarchy.Num() - 1; ParentIndex >= 0; --ParentIndex)
				{
					const UEdGraphPin* CurPin = ParentHierarchy[ParentIndex].Pin;
					const UEdGraphPin* ParentPin = CurPin ? CurPin->ParentPin : nullptr;
					UStruct* SubCategoryStruct = ParentPin ? Cast<UStruct>(ParentPin->PinType.PinSubCategoryObject.Get()) : nullptr;

					FName RedirectedPinName = SubCategoryStruct ? FProperty::FindRedirectedPropertyName(SubCategoryStruct, FName(*ParentHierarchy[ParentIndex].PropertyName)) : NAME_None;

					if (RedirectedPinName != NAME_None)
					{
						NewPinNameStr += FString(TEXT("_")) + RedirectedPinName.ToString();
					}
					else
					{
						NewPinNameStr += FString(TEXT("_")) + ParentHierarchy[ParentIndex].PropertyName;
					}
				}

				// make sure they match
				RedirectType = ((NewPin->PinName.ToString() != NewPinNameStr) ? ERedirectType_None : ERedirectType_Name);
			}
		}
	}

	return RedirectType;
}

bool UK2Node::DoesWildcardPinAcceptContainer(const UEdGraphPin* Pin)
{
	check(Pin);
	if (Pin->Direction == EGPD_Input)
	{
		return DoesInputWildcardPinAcceptArray(Pin);
	}
	else if (Pin->Direction == EGPD_Output)
	{
		return DoesOutputWildcardPinAcceptContainer(Pin);
	}

	return false;
}

void UK2Node::ReconstructSinglePin(UEdGraphPin* NewPin, UEdGraphPin* OldPin, ERedirectType RedirectType)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UK2Node::ReconstructSinglePin"), STAT_LinkerLoad_ReconstructSinglePin, STATGROUP_LoadTimeVerbose);

	UBlueprint* Blueprint = GetBlueprint();

	check(NewPin && OldPin);

	// Copy over modified persistent data
	NewPin->MovePersistentDataFromOldPin(*OldPin);

	if (NewPin->DefaultValue != NewPin->AutogeneratedDefaultValue)
	{
		if (RedirectType == ERedirectType_Value)
		{
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			for (const FString& OldPinName : OldPinNames)
			{
				const TMap<FString, FString>* ValueChanges = FCoreRedirects::GetValueRedirects(ECoreRedirectFlags::Type_Property, OldPinName);

				if (ValueChanges)
				{
					const FString* NewValue = ValueChanges->Find(*NewPin->DefaultValue);
					if (NewValue)
					{
						NewPin->DefaultValue = *NewValue;
					}
					break;
				}
			}
		}
	}

	// if OldPin has any watches, swap the watches for NewPin
	TArray<TArray<FName>> WatchedPropertyPaths;
	FKismetDebugUtilities::RemovePinPropertyWatchesByPredicate(
		Blueprint, 
		[OldPin, &WatchedPropertyPaths](const FBlueprintWatchedPin& WatchedPin)
		{
			if (WatchedPin.Get() == OldPin)
			{
				WatchedPropertyPaths.Add(WatchedPin.GetPathToProperty());
				return true;
			}

			return false;
		}
	);

	for (TArray<FName>& WatchedPropertyPath : WatchedPropertyPaths)
	{
		FBlueprintWatchedPin WatchedPin(NewPin, MoveTemp(WatchedPropertyPath));
		FKismetDebugUtilities::AddPinWatch(Blueprint, MoveTemp(WatchedPin));
	}
}

void UK2Node::ValidateOrphanPins(FCompilerResultsLog& MessageLog, bool bStore) const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->bOrphanedPin)
		{
			ValidateOrphanPin(Pin, MessageLog, bStore);
		}
	}
}

void UK2Node::ValidateOrphanPin(UEdGraphPin* Pin, FCompilerResultsLog& MessageLog, bool bStore) const
{
	check(Pin);
	check(Pin->bOrphanedPin);

	if (Pin->LinkedTo.Num())
	{
		const FText LinkedMessage = LOCTEXT("RemovedConnectedPin", "In use pin @@ no longer exists on node @@. Please refresh node or break links to remove pin.");
		if (bStore)
		{
			MessageLog.StorePotentialError(this, *LinkedMessage.ToString(), Pin, this);
		}
		else
		{
			MessageLog.Error(*LinkedMessage.ToString(), Pin, this);
		}
	}
	else if (!Pin->bHidden && !Pin->DoesDefaultValueMatchAutogenerated())
	{
		const FText NonDefaultMessage = LOCTEXT("RemovedNonDefaultPin", "Input pin @@ specifying non-default value no longer exists on node @@. Please refresh node or reset pin to default value to remove pin.");
		if (bStore)
		{
			MessageLog.StorePotentialWarning(this, *NonDefaultMessage.ToString(), Pin, this);
		}
		else
		{
			MessageLog.Warning(*NonDefaultMessage.ToString(), Pin, this);
		}
	}
}

void UK2Node::ValidateLinkedPinTypes(UEdGraphPin* OutputPin, FCompilerResultsLog& MessageLog) const
{
	check(OutputPin);
	check(OutputPin->Direction == EEdGraphPinDirection::EGPD_Output);

	const UEdGraphSchema_K2* K2Schema = CastChecked<const UEdGraphSchema_K2>(GetSchema());

	// Rewiring can mutate the linked pins arrays.
	TArray<UEdGraphPin*> LinkedToPins = OutputPin->LinkedTo;

	for (UEdGraphPin* InputPin : LinkedToPins)
	{
		check(InputPin);

		const bool bInputIsSelfPin = K2Schema->IsSelfPin(*InputPin);

		// Function call nodes implicitly handle interface->object type redirects on self pins.
		const bool bIsSelfInterfaceObjectConnection =
			bInputIsSelfPin &&
			(OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface) &&
			(InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
		;

		const bool bNeedsConversionNode =
			!bIsSelfInterfaceObjectConnection &&
			!K2Schema->ArePinTypesCompatible(OutputPin->PinType, InputPin->PinType, GetBlueprintClassFromNode())
		;

		// Blueprints actually support the connection of an array output to a scalar input for self pins.
		// Even though the types are actually incompatible, nodes that support multiple selfs will expand
		// to provide automatic iteration through the array.
		const UK2Node* InputPinOwner = CastChecked<UK2Node>(InputPin->GetOwningNode());
		const bool bAllowMultipleSelfs = InputPinOwner->AllowMultipleSelfs(true);
		const bool bNotAContainer = !InputPin->PinType.IsContainer();
		const bool bHasMultipleSelfsConnections =
			bInputIsSelfPin &&
			bAllowMultipleSelfs &&
			bNotAContainer
		;

		if (bNeedsConversionNode && !bHasMultipleSelfsConnections)
		{
			if (K2Schema->CreateAutomaticConversionNodeAndConnections(OutputPin, InputPin))
			{
				FFormatNamedArguments Args;

				Args.Add(TEXT("PinName"), FText::FromName(OutputPin->GetFName()));
				Args.Add(TEXT("LinkedPinName"), FText::FromName(InputPin->GetFName()));

				FText ConversionInfo = FText::Format(
					NSLOCTEXT("K2Node", "PinConversion_InfoFmt", "Pin '{PinName}' on node @@ doesn't match the type of '{LinkedPinName}' on node @@. A conversion node had to be inserted."),
					Args
				);

				if (const UEdGraphPin* SourcePin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(OutputPin))
				{
					Message_Note(ConversionInfo.ToString(), SourcePin->GetOwningNode(), InputPin->GetOwningNode());
				}
				else
				{
					const UK2Node* OutputPinOwner = Cast<UK2Node>(OutputPin->GetOwningNode());

					UE_LOGF(LogBlueprint, Verbose, "Missing compiler relevant pin '%ls' on node '%ls'", *OutputPin->GetName(), (OutputPinOwner ? *OutputPinOwner->GetFullName() : TEXT("<none>")));
				}
			}
		}
	}
}

bool UK2Node::TryInsertDefaultValueConversionNode(const UEdGraphPin& OldPin, UEdGraphPin& NewPin) const
{
	static TMap<FName, FName> MakeLiteralFunctionTable =
	{
		{ UEdGraphSchema_K2::PC_Boolean,	TEXT("MakeLiteralBool")		},
		{ UEdGraphSchema_K2::PC_Byte,		TEXT("MakeLiteralByte")		},
		{ UEdGraphSchema_K2::PC_Int,		TEXT("MakeLiteralInt")		},
		{ UEdGraphSchema_K2::PC_Int64,		TEXT("MakeLiteralInt64")	},
		{ UEdGraphSchema_K2::PC_Name,		TEXT("MakeLiteralName")		},
		{ UEdGraphSchema_K2::PC_Real,		TEXT("MakeLiteralDouble")	},
		{ UEdGraphSchema_K2::PC_String,		TEXT("MakeLiteralString")	},
		{ UEdGraphSchema_K2::PC_Text,		TEXT("MakeLiteralText")		},
	};

	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	const bool bHasCastFunction = K2Schema->SearchForAutocastFunction(OldPin.PinType, NewPin.PinType).IsSet();
	const FName* MakeLiteralFunctionName = MakeLiteralFunctionTable.Find(OldPin.PinType.PinCategory);

	const bool bCanConvertDefaultValue =
		bHasCastFunction &&
		(MakeLiteralFunctionName != nullptr)
		;

	if (bCanConvertDefaultValue)
	{
		UFunction* MakeLiteralFunction = UKismetSystemLibrary::StaticClass()->FindFunctionByName(*MakeLiteralFunctionName);
		check(MakeLiteralFunction);

		const int32 NodeXOffset = -600;
		FVector2D SpawnLocation(NodePosX + NodeXOffset, NodePosY);

		UK2Node_CallFunction* MakeLiteralNode = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_CallFunction>(
			GetGraph(),
			SpawnLocation,
			EK2NewNodeFlags::None,
			[MakeLiteralFunction](UK2Node_CallFunction* NewInstance)
			{
				NewInstance->SetFromFunction(MakeLiteralFunction);
			}
		);

		UEdGraphPin* InputPin = MakeLiteralNode->FindPinByPredicate(
			[K2Schema](UEdGraphPin* InPin)
			{
				check(InPin);
				return (InPin->Direction == EGPD_Input) && !K2Schema->IsMetaPin(*InPin);
			}
		);
		check(InputPin);
		InputPin->DefaultValue = OldPin.DefaultValue;

		UEdGraphPin* OutputPin = MakeLiteralNode->GetReturnValuePin();
		check(OutputPin);

		if (ensure(K2Schema->CreateAutomaticConversionNodeAndConnections(OutputPin, &NewPin)))
		{
			FFormatNamedArguments Args;

			Args.Add(TEXT("PinName"), FText::FromName(OutputPin->GetFName()));

			FText ConversionInfo = FText::Format(
				NSLOCTEXT("K2Node", "PinDefaultConversion_InfoFmt", "Pin '{PinName}' on node @@ has a default value that doesn't match the original type. A literal node and a conversion node had to be inserted."),
				Args
			);

			const UEdGraphPin* SourcePin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(OutputPin);
			check(SourcePin);
			Message_Note(ConversionInfo.ToString(), this);
		}
	}

	return bCanConvertDefaultValue;
}

bool UK2Node::TryRewireOldPinsToNewPinsFast(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins, TMap<UEdGraphPin*, UEdGraphPin*>* NewPinToOldPin)
{
	if(InOldPins.Num() != InNewPins.Num() )
	{
		return false;
	}

	for(UEdGraphPin* Pin : InOldPins)
	{
		if(Pin->ParentPin || Pin->bOrphanedPin)
		{
			return false;
		}
	}
	
	const int32 NumPins = InOldPins.Num();
	for(int32 Idx = 0; Idx < NumPins; ++Idx)
	{
		UEdGraphPin* OldPin = InOldPins[Idx];
		UEdGraphPin* NewPin = InNewPins[Idx];
		// don't trust anything except the base implementation - customizations
		// may do very relaxed matching (e.g. K2Node_MakeStruct):
		ERedirectType RedirectType = UK2Node::DoPinsMatchForReconstruction(NewPin, Idx, OldPin, Idx);
		if(RedirectType != ERedirectType_Name)
		{
			return false;
		}
	}

	// We are commmitted, preconditions validated, naive, fast rewiring can be performed:

	// check for mutation buried in DoPinsMatchForReconstruction:
	checkSlow(InOldPins.Num() == InNewPins.Num() && InOldPins.Num() == NumPins);
	// all pins match, rewire them:
	for(int32 Idx = 0; Idx < NumPins; ++Idx)
	{
		UEdGraphPin* OldPin = InOldPins[Idx];
		UEdGraphPin* NewPin = InNewPins[Idx];
		ReconstructSinglePin(NewPin, OldPin, ERedirectType_Name);
		checkSlow(InOldPins.Num() == InNewPins.Num() && InOldPins.Num() == NumPins);
		if (NewPinToOldPin)
		{
			NewPinToOldPin->Add(NewPin, OldPin);
		}
	}

	return true;
}

void UK2Node::EarlyValidation(FCompilerResultsLog& MessageLog) const
{
	UBlueprint* BP = GetBlueprint();
	const UEdGraphSchema_K2* K2Schema = CastChecked<const UEdGraphSchema_K2>(GetSchema());

	for (UEdGraphPin* Pin : Pins)
	{
		check(Pin);

		// We only have to validate pin type compatability after the BP is first loaded.
		// After the initial load, we can assume that the type won't change again.
		// 
		// Additionally, we can just check the output pins for type compatability.
		// It would be redundant to also include checking input pins.
		const bool bCheckPinForTypeChange =
			BP->bIsRegeneratingOnLoad &&
			!Pin->bOrphanedPin &&
			!K2Schema->IsExecPin(*Pin) &&
			(Pin->Direction == EEdGraphPinDirection::EGPD_Output)
		;

		if (Pin->bOrphanedPin)
		{
			ValidateOrphanPin(Pin, MessageLog, true);
		}
		else if (bCheckPinForTypeChange)
		{
			ValidateLinkedPinTypes(Pin, MessageLog);
		}
	}
}

void UK2Node::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	// Since this might be an expanded node, the validation will have been done on the source node (which will return this if not expanded)
	if (const UK2Node* SourceNode = Cast<UK2Node>(MessageLog.FindSourceObject(this)))
	{
		// If we don't commit any messages for the source node, then see if this node has messages to commit.
		// This is primarily needed for nodes generated from inside macros.
		if (!MessageLog.CommitPotentialMessages(const_cast<UK2Node*>(SourceNode)))
		{
			ValidateOrphanPins(MessageLog, false);
		}
	}

	FBlueprintEditorUtils::ValidateEditorOnlyNodes(this, MessageLog);
}

FString UK2Node::GetPinMetaData(FName InPinName, FName InKey)
{
	UEdGraphPin* Pin = FindPin(InPinName);

	// For split pins check the struct's metadata
	if (Pin && Pin->ParentPin && Pin->ParentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		FName NewPinPropertyName = FName(*Pin->PinName.ToString().RightChop(Pin->ParentPin->PinName.ToString().Len() + 1));

		UStruct* StructType = Cast<UStruct>(Pin->ParentPin->PinType.PinSubCategoryObject.Get());
		if (StructType)
		{
			for (TFieldIterator<FProperty> It(StructType); It; ++It)
			{
				const FProperty* Property = *It;
				if (Property && Property->GetFName() == NewPinPropertyName)
				{
					return Property->GetMetaData(InKey);
				}
			}
		}
	}
	return FString();
}

void UK2Node::RewireOldPinsToNewPins(TArray<UEdGraphPin*>& InOldPins, TArray<UEdGraphPin*>& InNewPins, TMap<UEdGraphPin*, UEdGraphPin*>* NewPinToOldPin)
{
	if(TryRewireOldPinsToNewPinsFast(InOldPins, InNewPins, NewPinToOldPin))
	{
		return;
	}

	TArray<UEdGraphPin*> OrphanedOldPins;
	TMap<UEdGraphPin*, UEdGraphPin*> MatchedPins; // Old to New

	const bool bSaveUnconnectedDefaultPins = (InNewPins.Num() == 0 && (IsA<UK2Node_CallFunction>() || IsA<UK2Node_MacroInstance>()));

	// Rewire any connection to pins that are matched by name (O(N^2) right now)
	// NOTE: we iterate backwards through the list because ReconstructSinglePin()
	//       destroys pins as we go along (clearing out parent pointers, etc.); 
	//       we need the parent pin chain intact for DoPinsMatchForReconstruction();              
	//       we want to destroy old pins from the split children (leafs) up, so 
	//       we do this since split child pins are ordered later in the list 
	//       (after their parents) 
	const auto FindNewPin = [&](UEdGraphPin* const OldPin, const int32 OldPinIndex, TFunctionRef<ERedirectType(UEdGraphPin*, int32)> Matcher) -> bool
	{
		bool bMatched = false;
		const int32 NumNewPins = InNewPins.Num();
		for (int32 NewPinIndex = NumNewPins - 1; NewPinIndex >= 0; --NewPinIndex)
		{
			UEdGraphPin* NewPin = InNewPins[NewPinIndex];

			ERedirectType RedirectType = Matcher(NewPin, NewPinIndex);
			if (RedirectType == ERedirectType_DefaultValue)
			{
				if (!TryInsertDefaultValueConversionNode(*OldPin, *NewPin))
				{
					RedirectType = ERedirectType_None;
				}
			}

			if (RedirectType != ERedirectType_None)
			{
				ReconstructSinglePin(NewPin, OldPin, RedirectType);
				MatchedPins.Add(OldPin, NewPin);
				if (NewPinToOldPin)
				{
					NewPinToOldPin->Add(NewPin, OldPin);
				}
				bMatched = true;
				break;
			}
		}
		return bMatched;
	};

	for (int32 OldPinIndex = InOldPins.Num()-1; OldPinIndex >= 0; --OldPinIndex)
	{
		UEdGraphPin* OldPin = InOldPins[OldPinIndex];

		bool bMatched = FindNewPin(OldPin, OldPinIndex, [this, OldPin, OldPinIndex](UEdGraphPin* NewPin, int32 NewPinIndex)
		{
			return UK2Node::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
		});

		if(!bMatched)
		{
			// do proper vtable dispatch of DoPinsMatchForReconstruction in a second pass:
			bMatched = FindNewPin(OldPin, OldPinIndex, [this, OldPin, OldPinIndex](UEdGraphPin* NewPin, int32 NewPinIndex)
				{
					return DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
				});
		}

		// Orphaned pins are those that existed in the OldPins array but do not in the NewPins.
		// We will save these pins and add them to the NewPins array if they are linked to other pins or have non-default value unless:
		// * The node has been flagged to not save orphaned pins
		// * The pin has been flagged not be saved if orphaned
		// * The pin is hidden and not a split pin
		const bool bVisibleOrSplitPin = (!OldPin->bHidden || (OldPin->SubPins.Num() > 0));
		if (UEdGraphPin::AreOrphanPinsEnabled() && !bDisableOrphanPinSaving && !bMatched && bVisibleOrSplitPin && OldPin->ShouldSavePinIfOrphaned())
		{
			// The node can specify to save no pins, all pins, or all but exec pins. However, even if all is specified Execute and Then are never saved
			 const bool bSaveOrphanedPin = ((OrphanedPinSaveMode == ESaveOrphanPinMode::SaveAll) ||
											 ((OrphanedPinSaveMode == ESaveOrphanPinMode::SaveAllButExec) && !UEdGraphSchema_K2::IsExecPin(*OldPin)));

			if (bSaveOrphanedPin)
			{
				bool bSavePin = bSaveUnconnectedDefaultPins || (OldPin->LinkedTo.Num() > 0);

				if (!bSavePin && OldPin->SubPins.Num() > 0)
				{
					// If this is a split pin then we need to save it if any of its children are being saved
					for (UEdGraphPin* OldSubPin : OldPin->SubPins)
					{
						if (OldSubPin->bOrphanedPin)
						{
							bSavePin = true;
							break;
						}
					}
					// Once we know we are going to be saving it we need to clean up the SubPins list to be only pins being saved
					if (bSavePin)
					{
						for (int32 SubPinIndex = OldPin->SubPins.Num() - 1; SubPinIndex >= 0; --SubPinIndex)
						{
							UEdGraphPin* SubPin = OldPin->SubPins[SubPinIndex];
							if (!SubPin->bOrphanedPin)
							{
								OldPin->SubPins.RemoveAt(SubPinIndex, EAllowShrinking::No);
								SubPin->MarkAsGarbage();
							}
						}
					}
				}

				// Input pins with non-default value should be saved
				if (!bSavePin && OldPin->Direction == EGPD_Input && !OldPin->DoesDefaultValueMatchAutogenerated())
				{
					bSavePin = true;
				}

				if (bSavePin)
				{
					OldPin->bOrphanedPin = true;
					OldPin->bNotConnectable = true;
					OrphanedOldPins.Add(OldPin);
					InOldPins.RemoveAt(OldPinIndex, EAllowShrinking::No);
				}
			}
		}
	}

	// The orphaned pins get placed after the rest of the new pins unless it is a child of a split pin and other
	// children of that split pin were matched in which case it will be at the end of the list of its former siblings
	for (int32 OrphanedIndex = OrphanedOldPins.Num() - 1; OrphanedIndex >= 0; --OrphanedIndex)
	{
		UEdGraphPin* OrphanedPin = OrphanedOldPins[OrphanedIndex];
		if (OrphanedPin->ParentPin == nullptr)
		{
			InNewPins.Add(OrphanedPin);
		}
		// Otherwise we need to work out where we fit in the list
		else 
		{
			UEdGraphPin* ParentPin = OrphanedPin->ParentPin;
			if (!ParentPin->bOrphanedPin)
			{
				// Our parent pin was matched, so we need to go to the end of the new pins sub pin section
				ParentPin->SubPins.Remove(OrphanedPin);
				ParentPin = MatchedPins.FindChecked(ParentPin);
				ParentPin->SubPins.Add(OrphanedPin);
				OrphanedPin->ParentPin = ParentPin;
			}
			int32 InsertIndex = InNewPins.Find(ParentPin);
			while (++InsertIndex < InNewPins.Num())
			{
				UEdGraphPin* PinToConsider = InNewPins[InsertIndex];
				if (PinToConsider->ParentPin != ParentPin)
				{
					break;
				}
				int32 WalkOffIndex = InsertIndex + PinToConsider->SubPins.Num();
				for (;InsertIndex < WalkOffIndex;++InsertIndex)
				{
					WalkOffIndex += InNewPins[WalkOffIndex]->SubPins.Num();
				}
			};

			ParentPin->SubPins.Remove(OrphanedPin);
			OrphanedPin->ParentPin = nullptr;
			InNewPins.Insert(OrphanedPin, InsertIndex);
		}
	}

	DestroyPinList(InOldPins);
}

void UK2Node::DestroyPinList(TArray<UEdGraphPin*>& InPins)
{
	UBlueprint* Blueprint = GetBlueprint();
	// Throw away the original pins
	for (UEdGraphPin* Pin : InPins)
	{
		Pin->Modify(/*bAlwaysMarkDirty =*/ false);
		Pin->BreakAllPinLinks(!Blueprint->bIsRegeneratingOnLoad, /*bAlwaysMarkDirty =*/ false);

		UEdGraphNode::DestroyPin(Pin);
	}
}

bool UK2Node::CanSplitPin(const UEdGraphPin* Pin) const
{
	return (Pin->GetOwningNode() == this && !Pin->bNotConnectable && Pin->LinkedTo.Num() == 0 && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct);
}

UK2Node* UK2Node::ExpandSplitPin(FKismetCompilerContext* CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* Pin)
{
	const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(CompilerContext ? CompilerContext->GetSchema() : SourceGraph->GetSchema());
	UK2Node* ExpandedNode = nullptr;

	if (Pins.Contains(Pin))
	{
		ExpandedNode = Schema->CreateSplitPinNode(Pin, UEdGraphSchema_K2::FCreateSplitPinNodeParams(CompilerContext, SourceGraph));

		int32 SubPinIndex = 0;

		for (int32 ExpandedPinIndex = 0; ExpandedPinIndex < ExpandedNode->Pins.Num(); ++ExpandedPinIndex)
		{
			UEdGraphPin* ExpandedPin = ExpandedNode->Pins[ExpandedPinIndex];

			if (!ExpandedPin->bHidden && !ExpandedPin->bOrphanedPin)
			{
				if (ExpandedPin->Direction == Pin->Direction)
				{
					if (Pin->SubPins.Num() == SubPinIndex)
					{
						if (CompilerContext)
						{
							CompilerContext->MessageLog.Error(*LOCTEXT("PinExpansionError", "Failed to expand pin @@, likely due to bad logic in node @@").ToString(), Pin, Pin->GetOwningNode());
						}
						break;
					}

					UEdGraphPin* SubPin = Pin->SubPins[SubPinIndex++];
					if (CompilerContext)
					{
						CompilerContext->MovePinLinksToIntermediate(*SubPin, *ExpandedPin);
					}
					else
					{
						Schema->MovePinLinks(*SubPin, *ExpandedPin);
					}
				}
				else
				{
					Schema->TryCreateConnection(Pin, ExpandedPin);
				}
			}
		}

		for(UEdGraphPin* SubPin : Pin->SubPins)
		{
			Pins.Remove(SubPin);
			SubPin->ParentPin = nullptr;
			SubPin->MarkAsGarbage();
		}
		Pin->SubPins.Empty();
	}

	return ExpandedNode;
}

void UK2Node::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	ExpandSplitPins(CompilerContext, SourceGraph);
}

void UK2Node::ExpandSplitPins(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// We iterate the array in reverse so we can both remove the subpins safely after we've read them and
	// so we have split nested structs we combine them back together in the right order
	for (int32 PinIndex=Pins.Num() - 1; PinIndex >= 0; --PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		if (Pin->SubPins.Num() > 0)
		{
			ExpandSplitPin(&CompilerContext, SourceGraph, Pin);
		}
	}
}

bool UK2Node::HasValidBlueprint() const
{
	// Perform an unchecked search here, so we don't crash if this is a transient node without a valid outer blueprint
	return (FBlueprintEditorUtils::FindBlueprintForNode(this) != nullptr);
}

UBlueprint* UK2Node::GetBlueprint() const
{
	return FBlueprintEditorUtils::FindBlueprintForNode(this);
}

TNotNull<UBlueprint*> UK2Node::GetBlueprintChecked() const
{
	return FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
}

FLinearColor UK2Node::GetNodeTitleColor() const
{
	// Different color for pure operations
	if (IsNodePure())
	{
		return GetDefault<UGraphEditorSettings>()->PureFunctionCallNodeTitleColor;
	}

	return GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
}

ERenamePinResult UK2Node::RenameUserDefinedPin(const FName OldName, const FName NewName, bool bTest)
{
	ERenamePinResult Result = RenameUserDefinedPinImpl(OldName, NewName, bTest);

	// If we actually applied the change (!bTest) and it was successful, broadcast the change notification
	if (bTest == false && Result == ERenamePinResult_Success)
	{
		BroadcastUserDefinedPinRenamed(OldName, NewName);
	}

	return Result;
}

void UK2Node::BroadcastUserDefinedPinRenamed(FName OldName, FName NewName)
{
	using namespace UK2Node_Private;

	// We're not concerned with thread safety here, but the annotation API forces thread safety by returning copies of data
	if (const FPinRenamedAnnotation* Annotation = GOnUserDefinedPinRenamedAnnotation.GetAnnotationMap().Find(this))
	{
		Annotation->PinRenamedEvent.Broadcast(this, OldName, NewName);
	}
}

FOnUserDefinedPinRenamed& UK2Node::OnUserDefinedPinRenamed()
{
	using namespace UK2Node_Private;

	// We're not concerned with thread safety here, but the annotation API forces thread safety by returning copies of data
	const TMap<const UObjectBase*,FPinRenamedAnnotation>& Map = GOnUserDefinedPinRenamedAnnotation.GetAnnotationMap();
	if (!Map.Contains(this))
	{
		FPinRenamedAnnotation NewEvent;
		NewEvent.bIsDefault = false;
		GOnUserDefinedPinRenamedAnnotation.AddAnnotation(this, MoveTemp(NewEvent));
	}

	return Map.FindChecked(this).PinRenamedEvent;
}

ERenamePinResult UK2Node::RenameUserDefinedPinImpl(const FName OldName, const FName NewName, bool bTest)
{
	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (OldName == CurrentPin->PinName)
		{
			Pin = CurrentPin;
		}
		else if (NewName == CurrentPin->PinName)
		{
			return ERenamePinResult::ERenamePinResult_NameCollision;
		}
	}

	if(!Pin)
	{
		return ERenamePinResult::ERenamePinResult_NoSuchPin;
	}

	if(!bTest)
	{
		Pin->Modify();
		Pin->PinName = NewName;
		if(!Pin->DefaultTextValue.IsEmpty())
		{
			Pin->GetSchema()->TrySetDefaultText(*Pin, Pin->DefaultTextValue);
		}

		if (Pin->SubPins.Num() > 0)
		{
			TArray<UEdGraphPin*> PinsToUpdate = Pin->SubPins;

			while (PinsToUpdate.Num() > 0)
			{
				UEdGraphPin* PinToRename = PinsToUpdate.Pop(EAllowShrinking::No);
				if (PinToRename->SubPins.Num() > 0)
				{
					PinsToUpdate.Append(PinToRename->SubPins);
				}
				PinToRename->Modify();

				const int32 OldNameLength = OldName.ToString().Len();
				FString NewNameStr = NewName.ToString();

				PinToRename->PinName = *(NewNameStr + PinToRename->PinName.ToString().RightChop(OldNameLength));
				PinToRename->PinFriendlyName = FText::AsCultureInvariant(MoveTemp(NewNameStr) + PinToRename->PinFriendlyName.ToString().RightChop(OldNameLength));
			}
		}
	}

	return ERenamePinResult::ERenamePinResult_Success;
}

/////////////////////////////////////////////////////
// FOptionalPinManager

void FOptionalPinManager::GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
{
	Record.bShowPin = true;
	Record.bCanToggleVisibility = true;
}

bool FOptionalPinManager::CanTreatPropertyAsOptional(FProperty* TestProperty) const
{
	return TestProperty->HasAnyPropertyFlags(CPF_Edit|CPF_BlueprintVisible); // TODO: ANIMREFACTOR: Maybe only CPF_Edit?
}

void FOptionalPinManager::RebuildPropertyList(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct)
{
	// Save the old visibility
	TMap<FName, FOldOptionalPinSettings> OldPinSettings;
	for (const FOptionalPinFromProperty& PropertyEntry : Properties)
	{
		OldPinSettings.Add(PropertyEntry.PropertyName, FOldOptionalPinSettings(PropertyEntry.bShowPin, PropertyEntry.bIsOverrideEnabled, PropertyEntry.bIsSetValuePinVisible, PropertyEntry.bIsOverridePinVisible));
	}

	// This exists to support legacy struct types that used an older convention to declare
	// override flag properties with a "bOverride_" prefix. Any property that starts with
	// "bOverride_" will be implicitly excluded from the optional pin set if the suffix matches
	// another property name within the same scope. For example, a property named "bOverride_MyValue"
	// will not be included in the output set if we find a matching property named "MyValue".
	TMap<FString, FProperty*> OverridesMap;
	const FString OverridePrefix(TEXT("bOverride_"));

	// Rebuild the property list
	Properties.Reset();
	TSet<FProperty*> TestProperties;
	for (TFieldIterator<FProperty> It(SourceStruct, EFieldIteratorFlags::IncludeSuper); It; ++It)
	{
		FProperty* Property = *It;
		if (CanTreatPropertyAsOptional(Property))
		{
			TestProperties.Add(Property);

			// find all "bOverride_" properties
			FString OriginalName = Property->GetName();
			if (OriginalName.RemoveFromStart(OverridePrefix) && !OriginalName.IsEmpty())
			{
				OverridesMap.Add(OriginalName, Property);
			}
		}
	}

	// handle regular properties
	for (FProperty* TestProperty : TestProperties)
	{
		if (!TestProperty->GetName().StartsWith(OverridePrefix))
		{
			FName CategoryName = NAME_None;
#if WITH_EDITOR
			CategoryName = FObjectEditorUtils::GetCategoryFName(TestProperty);
#endif //WITH_EDITOR

			OverridesMap.Remove(TestProperty->GetName());
			RebuildProperty(TestProperty, CategoryName, Properties, SourceStruct, OldPinSettings);
		}
	}

	// add remaining "bOverride_" properties
	for (const TPair<FString, FProperty*>& Pair : OverridesMap)
	{
		FProperty* TestProperty = Pair.Value;

		FName CategoryName = NAME_None;
#if WITH_EDITOR
		CategoryName = FObjectEditorUtils::GetCategoryFName(TestProperty);
#endif //WITH_EDITOR

		RebuildProperty(TestProperty, CategoryName, Properties, SourceStruct, OldPinSettings);
	}
}

void FOptionalPinManager::RebuildProperty(FProperty* TestProperty, FName CategoryName, TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct, TMap<FName, FOldOptionalPinSettings>& OldSettings)
{
	FOptionalPinFromProperty* Record = new (Properties)FOptionalPinFromProperty;
	Record->PropertyName = TestProperty->GetFName();
	Record->PropertyFriendlyName = UEditorEngine::GetFriendlyName(TestProperty, SourceStruct);
	Record->PropertyTooltip = TestProperty->GetToolTipText();
	Record->CategoryName = CategoryName;

	// Determine if the property is bound to a mutable edit condition flag, which historically indicates an additional optional
	// "override" pin is needed. Note that some manager subtypes may choose not to include these fields in the optional pin set.
	bool bNegate = false;
	FProperty* OverrideProperty = PropertyCustomizationHelpers::GetEditConditionProperty(TestProperty, bNegate);
	Record->bHasOverridePin = OverrideProperty != nullptr && OverrideProperty->HasAllPropertyFlags(CPF_BlueprintVisible) && !OverrideProperty->HasAllPropertyFlags(CPF_BlueprintReadOnly);

	Record->bIsMarkedForAdvancedDisplay = TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay);

	// Get the defaults
	GetRecordDefaults(TestProperty, *Record);

	// If this is a refresh, propagate the old visibility
	if (Record->bCanToggleVisibility)
	{
		if (FOldOptionalPinSettings* OldSetting = OldSettings.Find(Record->PropertyName))
		{
			Record->bShowPin = OldSetting->bOldVisibility;
			Record->bIsOverrideEnabled = OldSetting->bIsOldOverrideEnabled;
			Record->bIsSetValuePinVisible = OldSetting->bIsOldSetValuePinVisible;
			Record->bIsOverridePinVisible = OldSetting->bIsOldOverridePinVisible;
		}
	}
}

void FOptionalPinManager::CreateVisiblePins(TArray<FOptionalPinFromProperty>& Properties, UStruct* SourceStruct, EEdGraphPinDirection Direction, UK2Node* TargetNode, uint8* StructBasePtr, uint8* DefaultsPtr)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	for (FOptionalPinFromProperty& PropertyEntry : Properties)
	{
		if (FProperty* OuterProperty = FindFieldChecked<FProperty>(SourceStruct, PropertyEntry.PropertyName))
		{
			// Do we treat an array property as one pin, or a pin per entry in the array?
			// Depends on if we have an instance of the struct to work with.
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(OuterProperty);
			if ((ArrayProperty != nullptr) && (StructBasePtr != nullptr))
			{
				FProperty* InnerProperty = ArrayProperty->Inner;

				FEdGraphPinType PinType;
				if (Schema->ConvertPropertyToPinType(InnerProperty, /*out*/ PinType))
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, StructBasePtr);

					for (int32 Index = 0; Index < ArrayHelper.Num(); ++Index)
					{
						// Create the pin
						UEdGraphPin* NewPin = NULL;
						if (PropertyEntry.bShowPin)
						{
							const FName PinName = *FString::Printf(TEXT("%s_%d"), *PropertyEntry.PropertyName.ToString(), Index);

							FFormatNamedArguments Args;
							Args.Add(TEXT("PinName"), FText::AsCultureInvariant(PropertyEntry.PropertyFriendlyName.IsEmpty() ? PinName.ToString() : PropertyEntry.PropertyFriendlyName));
							Args.Add(TEXT("Index"), Index);
							const FText PinFriendlyName = FText::Format(LOCTEXT("PinFriendlyNameWithIndex", "{PinName} {Index}"), Args);

							NewPin = TargetNode->CreatePin(Direction, PinType, PinName);
							NewPin->PinFriendlyName = PinFriendlyName;
							NewPin->bNotConnectable = !PropertyEntry.bIsSetValuePinVisible;
							NewPin->bDefaultValueIsIgnored = !PropertyEntry.bIsSetValuePinVisible;
							if (UEdGraphSchema_K2::IsPropertyDeprecated(OuterProperty))
							{
								NewPin->bDeprecated = true;
								NewPin->DeprecationMessage = OuterProperty->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
							}
							Schema->ConstructBasicPinTooltip(*NewPin, PropertyEntry.PropertyTooltip, NewPin->PinToolTip);

							// Allow the derived class to customize the created pin
							CustomizePinData(NewPin, PropertyEntry.PropertyName, Index, InnerProperty);
						}

						// Let derived classes take a crack at transferring default values
						uint8* ValuePtr = ArrayHelper.GetRawPtr(Index);
						uint8* DefaultValuePtr = nullptr;

						if (DefaultsPtr)
						{
							FScriptArrayHelper_InContainer DefaultsArrayHelper(ArrayProperty, DefaultsPtr);

							if (DefaultsArrayHelper.IsValidIndex(Index))
							{
								DefaultValuePtr = DefaultsArrayHelper.GetRawPtr(Index);
							}
						}
						if (NewPin != nullptr)
						{
							PostInitNewPin(NewPin, PropertyEntry, Index, ArrayProperty->Inner, ValuePtr, DefaultValuePtr);
						}
						else
						{
							PostRemovedOldPin(PropertyEntry, Index, ArrayProperty->Inner, ValuePtr, DefaultValuePtr);
						}
					}
				}
			}
			else
			{
				// Not an array property

				FEdGraphPinType PinType;
				if (Schema->ConvertPropertyToPinType(OuterProperty, /*out*/ PinType))
				{
					// Create the pin
					UEdGraphPin* NewPin = nullptr;
					if (PropertyEntry.bShowPin)
					{
						const FName PinName = PropertyEntry.PropertyName;
						NewPin = TargetNode->CreatePin(Direction, PinType, PinName);
						NewPin->PinFriendlyName = FText::AsCultureInvariant(PropertyEntry.PropertyFriendlyName.IsEmpty() ? PinName.ToString() : PropertyEntry.PropertyFriendlyName);
						NewPin->bNotConnectable = !PropertyEntry.bIsSetValuePinVisible;
						NewPin->bDefaultValueIsIgnored = !PropertyEntry.bIsSetValuePinVisible;
						if (UEdGraphSchema_K2::IsPropertyDeprecated(OuterProperty))
						{
							NewPin->bDeprecated = true;
							NewPin->DeprecationMessage = OuterProperty->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
						}
						Schema->ConstructBasicPinTooltip(*NewPin, PropertyEntry.PropertyTooltip, NewPin->PinToolTip);

						// Allow the derived class to customize the created pin
						CustomizePinData(NewPin, PropertyEntry.PropertyName, INDEX_NONE, OuterProperty);
					}

					// Let derived classes take a crack at transferring default values
					if (StructBasePtr != nullptr)
					{
						uint8* ValuePtr = OuterProperty->ContainerPtrToValuePtr<uint8>(StructBasePtr);
						uint8* DefaultValuePtr = DefaultsPtr ? OuterProperty->ContainerPtrToValuePtr<uint8>(DefaultsPtr) : nullptr;
						if (NewPin != nullptr)
						{
							PostInitNewPin(NewPin, PropertyEntry, INDEX_NONE, OuterProperty, ValuePtr, DefaultValuePtr);
						}
						else
						{
							PostRemovedOldPin(PropertyEntry, INDEX_NONE, OuterProperty, ValuePtr, DefaultValuePtr);
						}
					}
				}
			}
		}
	}
}

void FOptionalPinManager::CacheShownPins(const TArray<FOptionalPinFromProperty>& OptionalPins, TArray<FName>& OldShownPins)
{
	for (const FOptionalPinFromProperty& ShowPinForProperty : OptionalPins)
	{
		if (ShowPinForProperty.bShowPin)
		{
			OldShownPins.Add(ShowPinForProperty.PropertyName);
		}
	}
}

void FOptionalPinManager::EvaluateOldShownPins(const TArray<FOptionalPinFromProperty>& OptionalPins, TArray<FName>& OldShownPins, UK2Node* Node)
{
	for (const FOptionalPinFromProperty& ShowPinForProperty : OptionalPins)
	{
		if (ShowPinForProperty.bShowPin == false && OldShownPins.Contains(ShowPinForProperty.PropertyName))
		{
			if (UEdGraphPin* Pin = Node->FindPin(ShowPinForProperty.PropertyName))
			{
				Pin->SetSavePinIfOrphaned(false);
			}
		}
	}
	OldShownPins.Reset();
}

UEdGraphPin* UK2Node::GetExecPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Execute);
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node::GetThenPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Then);
	check(Pin == nullptr || Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	UEdGraphPin* PassThroughPin = nullptr;

	if (FromPin && UEdGraphSchema_K2::IsExecPin(*FromPin))
	{
		// We only allow execution passing if there is exactly one input and one output exec pin otherwise there is
		// ambiguity (e.g., on a branch or sequence node or timeline) so we want no passthrough and commenting it out will kill subsequent code
		bool bFoundFromPin = false;
		int32 NumInputs = 0;
		int32 NumOutputs = 0;
		UEdGraphPin* PotentialResult = nullptr;

		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin == FromPin)
			{
				bFoundFromPin = true;
			}

			if (UEdGraphSchema_K2::IsExecPin(*Pin))
			{
				if (Pin->Direction == EGPD_Input)
				{
					++NumInputs;
				}
				else
				{
					++NumOutputs;
				}

				if (Pin->Direction != FromPin->Direction)
				{
					PotentialResult = Pin;
				}
			}
		}

		if ((NumInputs == 1) && (NumOutputs == 1) && bFoundFromPin)
		{
			PassThroughPin = PotentialResult;
		}
	}

	return PassThroughPin;
}

bool UK2Node::IsInDevelopmentMode() const
{
	// Check class setting (which can override the default setting)
	const UBlueprint* OwningBP = GetBlueprint();
	if(OwningBP != nullptr
		&& OwningBP->CompileMode != EBlueprintCompileMode::Default)
	{
		return OwningBP->CompileMode == EBlueprintCompileMode::Development;
	}

	// Check default setting
	return Super::IsInDevelopmentMode();
}

bool UK2Node::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	return DesiredSchema->GetClass()->IsChildOf(UEdGraphSchema_K2::StaticClass());
}

FString UK2Node::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Blueprint");
}

void UK2Node::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	// start with the default hover text (from the pin's tool-tip)
	Super::GetPinHoverText(Pin, HoverTextOut);

	// if the Pin wasn't initialized with a tool-tip of its own
	if (HoverTextOut.IsEmpty())
	{
		UEdGraphSchema const* Schema = GetSchema();
		check(Schema != nullptr);
		Schema->ConstructBasicPinTooltip(Pin, FText::GetEmpty(), HoverTextOut);
	}	

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);

	// If this is an orphaned node, just end now
	if (Blueprint == nullptr)
	{
		return;
	}

	// If this resides in an intermediate graph, show the UObject name for debug purposes
	if(Blueprint->IntermediateGeneratedGraphs.Contains(GetGraph()))
	{
		HoverTextOut = FString::Printf(TEXT("%s\n\n%s"), *Pin.GetName(), *HoverTextOut);
	}

	UObject* ActiveObject = Blueprint->GetObjectBeingDebugged();
	// if there is no object being debugged, then we don't need to tack on any of the following data
	if (ActiveObject == nullptr)
	{
		return;
	}

	// Switch the blueprint to the one that generated the object being debugged (e.g. in case we're inside a Macro BP while debugging)
	Blueprint = Cast<UBlueprint>(ActiveObject->GetClass()->ClassGeneratedBy);
	if (Blueprint == nullptr)
	{
		return;
	}

	// if the blueprint doesn't have debug data, notify the user
	/*if (!FKismetDebugUtilities::HasDebuggingData(Blueprint))
	{
		HoverTextOut += TEXT("\n(NO DEBUGGING INFORMATION GENERATED, NEED TO RECOMPILE THE BLUEPRINT)");
	}*/

	//@TODO: For exec pins, show when they were last executed


	// grab the debug value of the pin
	FString WatchText;
	const FKismetDebugUtilities::EWatchTextResult WatchStatus = FKismetDebugUtilities::GetWatchText(/*inout*/ WatchText, Blueprint, ActiveObject, &Pin);
	// if this is an container pin, then we possibly have too many lines (too many entries)
	if (Pin.PinType.IsContainer())
	{
		int32 LineCounter = 0;
		int32 OriginalWatchTextLen = WatchText.Len();

		// walk the string, finding line breaks (counting lines)
		for (int32 NewWatchTextLen = 0; NewWatchTextLen < OriginalWatchTextLen; )
		{
			++LineCounter;

			int32 NewLineIndex = WatchText.Find("\n", ESearchCase::IgnoreCase,  ESearchDir::FromStart, NewWatchTextLen);
			// if we've reached the end of the string (it's not to long)
			if (NewLineIndex == INDEX_NONE)
			{
				break;
			}

			NewWatchTextLen = NewLineIndex + 1;
			// if we're at the end of the string (but it ends with a newline)
			if (NewWatchTextLen >= OriginalWatchTextLen)
			{
				break;
			}

			// if we've hit the max number of lines allowed in a tooltip
			if (LineCounter >= MaxArrayPinTooltipLineCount)
			{
				// truncate WatchText so it contains a finite number of lines
				WatchText.LeftInline(NewWatchTextLen, EAllowShrinking::No);
				WatchText += "..."; // WatchText should already have a trailing newline (no need to prepend this with one)
				break;
			}
		}
	} // if Pin.PinType.IsContainer()...


	switch (WatchStatus)
	{
	case FKismetDebugUtilities::EWTR_Valid:
		HoverTextOut += FString::Printf(TEXT("\nCurrent value = \n%s"), *WatchText); //@TODO: Print out object being debugged name?
		break;
	case FKismetDebugUtilities::EWTR_NotInScope:
		HoverTextOut += TEXT("\n(Variable is not in scope)");
		break;

	default:
	case FKismetDebugUtilities::EWTR_NoDebugObject:
	case FKismetDebugUtilities::EWTR_NoProperty:
		break;
	}
}

UClass* UK2Node::GetBlueprintClassFromNode() const
{
	UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForNode(this);
	UClass* BPClass = BP
		? (BP->SkeletonGeneratedClass ? *BP->SkeletonGeneratedClass : *BP->GeneratedClass)
		: nullptr;
	return BPClass;
}

#undef LOCTEXT_NAMESPACE
