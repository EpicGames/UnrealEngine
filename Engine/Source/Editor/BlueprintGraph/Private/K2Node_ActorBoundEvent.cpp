// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_ActorBoundEvent.h"

#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Engine/MemberReference.h"
#include "EventEntryHandler.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetCompiler.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ActorBoundEvent)

struct FKismetFunctionContext;

#define LOCTEXT_NAMESPACE "K2Node_ActorBoundEvent"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_ActorBoundEventEntry

class FKCHandler_ActorBoundEventEntry : public FKCHandler_EventEntry
{
public:
	FKCHandler_ActorBoundEventEntry(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_EventEntry(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		// Check to make sure that the object the event is bound to is valid
		const UK2Node_ActorBoundEvent* BoundEventNode = Cast<UK2Node_ActorBoundEvent>(Node);
		if (BoundEventNode && BoundEventNode->EventOwner)
		{
			FKCHandler_EventEntry::Compile(Context, Node);
		}
		else
		{
			CompilerContext.MessageLog.Error(*FString(*LOCTEXT("FindNodeBoundEvent_Error", "Couldn't find object for bound event node @@").ToString()), Node);
		}
	}
};

UK2Node_ActorBoundEvent::UK2Node_ActorBoundEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FNodeHandlingFunctor* UK2Node_ActorBoundEvent::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_ActorBoundEventEntry(CompilerContext);
}

void UK2Node_ActorBoundEvent::ReconstructNode()
{
	// We need to fixup our event reference as it may have changed or been redirected
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();

	// If we couldn't find the target delegate, then try to find it in the property remap table
	if (!TargetDelegateProp)
	{
		FMulticastDelegateProperty* NewProperty = FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
		if (NewProperty)
		{
			// Found a remapped property, update the node
			TargetDelegateProp = NewProperty;
			DelegatePropertyName = NewProperty->GetFName();
		}
	}

	if (TargetDelegateProp && TargetDelegateProp->SignatureFunction)
	{
		EventReference.SetFromField<UFunction>(TargetDelegateProp->SignatureFunction, false);
	}

	CachedNodeTitle.MarkDirty();

	Super::ReconstructNode();
}

void UK2Node_ActorBoundEvent::DestroyNode()
{
	if (EventOwner)
	{
		// If we have an event owner, remove the delegate referencing this event, if any
		const ULevel* TargetLevel = EventOwner->GetLevel();
		if (TargetLevel)
		{
			ALevelScriptActor* LSA = TargetLevel->GetLevelScriptActor();
			if (LSA)
			{
				// Create a delegate of the correct signature to remove
				FScriptDelegate Delegate;
				Delegate.BindUFunction(LSA, CustomFunctionName);

				// Attempt to remove it from the target's MC delegate
				if (FMulticastDelegateProperty* TargetDelegate = GetTargetDelegateProperty())
				{
					TargetDelegate->RemoveDelegate(Delegate, EventOwner);
				}
			}
		}

	}

	Super::DestroyNode();
}

bool UK2Node_ActorBoundEvent::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bDisallowPaste = !Super::CanPasteHere(TargetGraph);
	if (!bDisallowPaste)
	{
		const AActor* ReferencedLevelActor = GetReferencedLevelActor();
		ULevel* Level = ReferencedLevelActor ? ReferencedLevelActor->GetLevel() : nullptr;
		const UBlueprint* LevelBP = Level ? Level->GetLevelScriptBlueprint(true) : nullptr;
		if (FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph) == LevelBP)
		{
			if (const UK2Node_Event* PreExistingNode = FKismetEditorUtilities::FindBoundEventForActor(GetReferencedLevelActor(), DelegatePropertyName))
			{
				//UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because it is flagged as an internal event.", *GetFName().ToString());
				bDisallowPaste = true;
			}
		}
		else
		{
			///UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because it is flagged as an internal event.", *GetFName().ToString());
		}
	}
	return !bDisallowPaste;
}

FText UK2Node_ActorBoundEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (EventOwner == nullptr)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DelegatePropertyName"), GetTargetDelegateDisplayName());
		return FText::Format(LOCTEXT("ActorBoundEventTitleNoOwner", "{DelegatePropertyName} (None)"), Args);
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DelegatePropertyName"), GetTargetDelegateDisplayName());
		Args.Add(TEXT("TargetName"), FText::FromString(EventOwner->GetActorLabel()));

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("ActorBoundEventTitle", "{DelegatePropertyName} ({TargetName})"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_ActorBoundEvent::GetTooltipText() const
{
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();
	if (TargetDelegateProp)
	{
		return TargetDelegateProp->GetToolTipText();
	}
	else
	{
		return FText::FromName(DelegatePropertyName);
	}
}

FString UK2Node_ActorBoundEvent::GetDocumentationLink() const
{
	if (UClass* EventSignatureClass = EventReference.GetMemberParentClass(GetBlueprintClassFromNode()))
	{
		return FString::Printf(TEXT("Shared/GraphNodes/Blueprint/%s%s"), EventSignatureClass->GetPrefixCPP(), *EventSignatureClass->GetName());
	}

	return FString();
}

FString UK2Node_ActorBoundEvent::GetDocumentationExcerptName() const
{
	return DelegatePropertyName.ToString();
}


AActor* UK2Node_ActorBoundEvent::GetReferencedLevelActor() const
{
	return EventOwner;
}

void UK2Node_ActorBoundEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	// make sure that the actor still exists:
	AActor* TargetActor = GetReferencedLevelActor();
	if(!TargetActor)
	{
		MessageLog.Warning(
			*NSLOCTEXT("KismetCompiler", "MissingActor_ActorBoundEvent", "@@ is referencing an Actor that no longer exists. Attached logic will never execute.").ToString(), 
			this
		);
	}
	else if(DelegateOwnerClass == nullptr)
	{
		MessageLog.Warning(
			*NSLOCTEXT("KismetCompiler", "MissingClass_ActorBoundEvent", "@@ is trying to find an Event Dispatcher named @@ in a class that no longer exists. Attached logic will never execute.").ToString(), 
			this,
			*DelegatePropertyName.ToString()
		);
	}
	else if( GetTargetDelegatePropertyFromSkel() == nullptr )
	{
		MessageLog.Warning(
			*NSLOCTEXT("KismetCompiler", "MissingDelegate_ActorBoundEvent", "@@ is referencing an Event Dispatcher named @@ that no longer exists in class @@. Attached logic will never execute.").ToString(),
			this,
			*DelegatePropertyName.ToString(),
			DelegateOwnerClass
		);
	}
}

void UK2Node_ActorBoundEvent::InitializeActorBoundEventParams(AActor* InEventOwner, const FMulticastDelegateProperty* InDelegateProperty)
{
	if (InEventOwner && InDelegateProperty)
	{
		EventOwner = InEventOwner;
		DelegatePropertyName = InDelegateProperty->GetFName();
		DelegateOwnerClass = CastChecked<UClass>(InDelegateProperty->GetOwner<UObject>())->GetAuthoritativeClass();
		EventReference.SetFromField<UFunction>(InDelegateProperty->SignatureFunction, false);
		CustomFunctionName = FName(*FString::Printf(TEXT("BndEvt__%s_%s_%s_%s"), *GetBlueprint()->GetName(), *InEventOwner->GetName(), *GetName(), *EventReference.GetMemberName().ToString()));
		bOverrideFunction = false;
		bInternalEvent = true;
		CachedNodeTitle.MarkDirty();
	}
}

FMulticastDelegateProperty* UK2Node_ActorBoundEvent::GetTargetDelegateProperty() const
{
	return FindFProperty<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
}

FMulticastDelegateProperty* UK2Node_ActorBoundEvent::GetTargetDelegatePropertyFromSkel() const
{
	return FindFProperty<FMulticastDelegateProperty>(FBlueprintEditorUtils::GetMostUpToDateClass(DelegateOwnerClass), DelegatePropertyName);
}

FText UK2Node_ActorBoundEvent::GetTargetDelegateDisplayName() const
{
	FMulticastDelegateProperty* Prop = GetTargetDelegateProperty();
	return Prop ? Prop->GetDisplayNameText() : FText::FromName(DelegatePropertyName);
}

bool UK2Node_ActorBoundEvent::IsUsedByAuthorityOnlyDelegate() const
{
	const FMulticastDelegateProperty* TargetDelegateProp = FindFProperty<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
	return (TargetDelegateProp && TargetDelegateProp->HasAnyPropertyFlags(CPF_BlueprintAuthorityOnly));
}

void UK2Node_ActorBoundEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Fix up legacy nodes that may not yet have a delegate pin
	if (Ar.IsLoading())
	{
		if (Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE)
		{
			DelegateOwnerClass = EventSignatureClass_DEPRECATED;	
		}
	}
}

#undef LOCTEXT_NAMESPACE
