// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_AddComponent.h"

#include "Components/ActorComponent.h"
#include "Components/ChildActorComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "DiffResults.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Math/Transform.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/BlueprintsObjectVersion.h" // for ComponentTemplateClassSupport
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AddComponent)

class UEdGraph;

#define LOCTEXT_NAMESPACE "K2Node_AddComponent"

const FName UK2Node_AddComponent::NAME_RelativeTransform(TEXT("RelativeTransform"));
const FName UK2Node_AddComponent::NAME_ManualAttachment(TEXT("bManualAttachment"));

//////////////////////////////////////////////////////////////////////////
// FKCHandler_AddComponent

UK2Node_AddComponent::UK2Node_AddComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bDefaultsToPureFunc = false;
}

// We add this prefix to template object names.
const FString UK2Node_AddComponent::ComponentTemplateNamePrefix(TEXT("NODE_Add"));

void UK2Node_AddComponent::Serialize(FArchive& Ar)
{
	if (!TemplateType && Ar.IsSaving() && GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::ComponentTemplateClassSupport)
	{
		UActorComponent* Template = GetTemplateFromNode();
		TemplateType = Template ? Template->GetClass() : nullptr;
	}

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::AddComponentNodeTemplateUniqueNames)
		{
			if (Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
			{
				UBlueprint* Blueprint = GetBlueprint();
				UActorComponent* Template = GetTemplateFromNode();

				// Fix up old "generic" template names to conform to the new unique naming convention.
				if (Template != nullptr && !Template->GetName().StartsWith(ComponentTemplateNamePrefix))
				{
					// Find a new, unique name for the template.
					const FName NewTemplateName = MakeNewComponentTemplateName(Template->GetOuter(), Template->GetClass());

					// Map the old template name to the new name (note: GetBlueprint() is internally checked and should be non-NULL).
					GetBlueprint()->OldToNewComponentTemplateNames.Add(Template->GetFName()) = NewTemplateName;

					// Rename the component template to conform to the new name.
					Template->Rename(*NewTemplateName.ToString(), Template->GetOuter(), REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

					// Update the template name pin's default value to match the template name.
					GetTemplateNamePinChecked()->DefaultValue = Template->GetName();
				}
			}
		}
	}
}

void UK2Node_AddComponent::AllocateDefaultPins()
{
	AllocateDefaultPinsWithoutExposedVariables();
	AllocatePinsForExposedVariables();

	UEdGraphPin* ManualAttachmentPin = GetManualAttachmentPin();

	UEdGraphSchema const* Schema = GetSchema();
	check(Schema != NULL);
	Schema->ConstructBasicPinTooltip(*ManualAttachmentPin, LOCTEXT("ManualAttachmentPinTooltip", "Defines whether the component should attach to the root automatically, or be left unattached for the user to manually attach later."), ManualAttachmentPin->PinToolTip);

	UEdGraphPin* TransformPin = GetRelativeTransformPin();
	Schema->ConstructBasicPinTooltip(*TransformPin, LOCTEXT("TransformPinTooltip", "Defines where to position the component (relative to its parent). If the component is left unattached, then the transform is relative to the world."), TransformPin->PinToolTip);
}

void UK2Node_AddComponent::AllocatePinsForExposedVariables()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const UClass* ComponentClass = GetSpawnedType();
	if (ComponentClass != nullptr)
	{
		const UObject* ClassDefaultObject = ComponentClass->GetDefaultObject(false);

		for (TFieldIterator<FProperty> PropertyIt(ComponentClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
		{
			FProperty* Property = *PropertyIt;
			const bool bNotDelegate = !Property->IsA(FMulticastDelegateProperty::StaticClass());
			const bool bIsExposedToSpawn = UEdGraphSchema_K2::IsPropertyExposedOnSpawn(Property);
			const bool bIsVisible = Property->HasAllPropertyFlags(CPF_BlueprintVisible);
			const bool bNotParam = !Property->HasAllPropertyFlags(CPF_Parm);
			const bool bStillExists = FBlueprintEditorUtils::PropertyStillExists(Property);
			if(bNotDelegate && bIsExposedToSpawn && bIsVisible && bNotParam && bStillExists)
			{
				FEdGraphPinType PinType;
				K2Schema->ConvertPropertyToPinType(Property, /*out*/ PinType);	
				const bool bIsUnique = (nullptr == FindPin(Property->GetFName()));
				if (K2Schema->FindSetVariableByNameFunction(PinType) && bIsUnique)
				{
					UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Property->GetFName());
					Pin->PinType = PinType;
					bHasExposedVariable = true;

					if ((ClassDefaultObject != nullptr) && K2Schema->PinDefaultValueIsEditable(*Pin))
					{
						FString DefaultValueAsString;
						const bool bDefaultValueSet = FBlueprintEditorUtils::PropertyValueToString(Property, reinterpret_cast<const uint8*>(ClassDefaultObject), DefaultValueAsString, this);
						check(bDefaultValueSet);
						K2Schema->SetPinAutogeneratedDefaultValue(Pin, DefaultValueAsString);
					}

					// Copy tooltip from the property.
					K2Schema->ConstructBasicPinTooltip(*Pin, Property->GetToolTipText(), Pin->PinToolTip);
				}
			}
		}
	}

	// Hide transform and attachment pins if it is not a scene component
	const bool bHideTransformPins = (ComponentClass != nullptr) ? !ComponentClass->IsChildOf(USceneComponent::StaticClass()) : false;

	UEdGraphPin* ManualAttachmentPin = GetManualAttachmentPin();
	ManualAttachmentPin->SafeSetHidden(bHideTransformPins);

	UEdGraphPin* TransformPin = GetRelativeTransformPin();
	TransformPin->SafeSetHidden(bHideTransformPins);
}

FName UK2Node_AddComponent::GetAddComponentFunctionName()
{
    static const FName AddComponentFunctionName(GET_FUNCTION_NAME_CHECKED(AActor, AddComponent));
    return AddComponentFunctionName;
}

UClass* UK2Node_AddComponent::GetSpawnedType() const
{
	if (TemplateType != nullptr)
	{
		return TemplateType->GetAuthoritativeClass();
	}
	
	const UActorComponent* TemplateComponent = GetTemplateFromNode();
	return TemplateComponent ? TemplateComponent->GetClass()->GetAuthoritativeClass() : nullptr;
}

void UK2Node_AddComponent::AllocateDefaultPinsWithoutExposedVariables()
{
	Super::AllocateDefaultPins();
	
	// set properties on template pin
	UEdGraphPin* TemplateNamePin = GetTemplateNamePinChecked();
	TemplateNamePin->bDefaultValueIsReadOnly = true;
	TemplateNamePin->bNotConnectable = true;
	TemplateNamePin->bHidden = true;

	// set properties on relative transform pin
	UEdGraphPin* RelativeTransformPin = GetRelativeTransformPin();
	RelativeTransformPin->bDefaultValueIsIgnored = true;

	// Override this as a non-ref param, because the identity transform is hooked up in the compiler under the hood
	RelativeTransformPin->PinType.bIsReference = false;
}

void UK2Node_AddComponent::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	UEdGraphPin* TemplateNamePin = GetTemplateNamePinChecked();
	for(int32 OldPinIdx = 0; TemplateNamePin && OldPinIdx < OldPins.Num(); ++OldPinIdx)
	{
		if(TemplateNamePin && (TemplateNamePin->PinName == OldPins[OldPinIdx]->PinName))
		{
			TemplateNamePin->DefaultValue = OldPins[OldPinIdx]->DefaultValue;
		}
	}

	AllocatePinsForExposedVariables();
}

void UK2Node_AddComponent::PostReconstructNode()
{
	Super::PostReconstructNode();

	// Set the return type to the right class of component
	UEdGraphPin* ReturnPin = GetReturnValuePin();
	UClass* ComponentClass = GetSpawnedType();
	if (ReturnPin && ComponentClass)
	{
		ReturnPin->PinType.PinSubCategoryObject = ComponentClass->GetAuthoritativeClass();
	}
}

void UK2Node_AddComponent::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UClass* TemplateClass = GetSpawnedType();
	if (TemplateClass)
	{
		if (!FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(TemplateClass))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("TemplateClass"), FText::FromString(TemplateClass->GetName()));
			Args.Add(TEXT("NodeTitle"), GetNodeTitle(ENodeTitleType::FullTitle));
			MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "InvalidComponentTemplate_Error", "Invalid class '{TemplateClass}' used as template by '{NodeTitle}' for @@"), Args).ToString(), this);
		}

		if (UChildActorComponent const* ChildActorComponent = Cast<UChildActorComponent const>(GetTemplateFromNode()))
		{
			UBlueprint const* Blueprint = GetBlueprint();

			UClass const* ChildActorClass = ChildActorComponent->GetChildActorClass();
			if (ChildActorClass == Blueprint->GeneratedClass)
			{
				UEdGraph const* ParentGraph = GetGraph();
				UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

				if (K2Schema->IsConstructionScript(ParentGraph))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("ChildActorClass"), FText::FromString(ChildActorClass->GetName()));
					MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "AddSelfComponent_Error", "@@ cannot add a '{ChildActorClass}' component in the construction script (could cause infinite recursion)."), Args).ToString(), this);
				}
			}
			else if (ChildActorClass != nullptr && ChildActorClass->GetDefaultObject(false))
			{
				AActor const* ChildActor = Cast<AActor>(ChildActorClass->GetDefaultObject(false));
				check(ChildActor != nullptr);
				USceneComponent* RootComponent = ChildActor->GetRootComponent();

				if ((RootComponent != nullptr) && (RootComponent->Mobility == EComponentMobility::Static) && (ChildActorComponent->Mobility != EComponentMobility::Static))
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("ChildActorClass"), FText::FromString(ChildActorClass->GetName()));
					MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "AddStaticChildActorComponent_Error", "@@ cannot add a '{ChildActorClass}' component as it has static mobility, and the ChildActorComponent does not."), Args).ToString(), this);
				}
			}
		}
	}
	else
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeTitle"), GetNodeTitle(ENodeTitleType::FullTitle));
		MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingComponentTemplate_Error", "Unknown template referenced by '{NodeTitle}' for @@"), Args).ToString(), this);
	}
}

UActorComponent* UK2Node_AddComponent::GetTemplateFromNode() const
{
	UBlueprint* BlueprintObj = GetBlueprint();

	// Find the template name input pin, to get the name from
	UEdGraphPin* TemplateNamePin = GetTemplateNamePin();
	if (TemplateNamePin)
	{
		const FString& TemplateName = TemplateNamePin->DefaultValue;
		return BlueprintObj->FindTemplateByName(FName(*TemplateName));
	}

	return NULL;
}

void UK2Node_AddComponent::DestroyNode()
{
	// See if this node has a template
	UActorComponent* Template = GetTemplateFromNode();
	if (Template != NULL)
	{
		// Save current template state - this is needed in order to ensure that we restore to the correct Outer in the case of a compile prior to the undo/redo action.
		Template->Modify();

		// Get the blueprint so we can remove it from it
		UBlueprint* BlueprintObj = GetBlueprint();

		// remove it
		BlueprintObj->Modify();
		BlueprintObj->ComponentTemplates.Remove(Template);
	}

	Super::DestroyNode();
}

void UK2Node_AddComponent::PrepareForCopying()
{
	TemplateBlueprint = GetBlueprint()->GetPathName();
}

void UK2Node_AddComponent::PostPasteNode()
{
	Super::PostPasteNode();
	
	// Create a unique component template for the pasted node (this).
	MakeNewComponentTemplate();
}

FText UK2Node_AddComponent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText CachedAssetTitle;
	FText CachedNodeTitle;
	UEdGraphPin* TemplateNamePin = GetTemplateNamePin();
	if (TemplateNamePin != nullptr)
	{
		FString TemplateName = TemplateNamePin->DefaultValue;
		UBlueprint* Blueprint = GetBlueprint();

		if (UActorComponent* SourceTemplate = Blueprint->FindTemplateByName(FName(*TemplateName)))
		{
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ComponentType"), SourceTemplate->GetClass()->GetDisplayNameText());
				CachedNodeTitle = FText::Format(LOCTEXT("AddClass", "Add {ComponentType}"), Args);
			}

			UChildActorComponent* SubActorComp = Cast<UChildActorComponent>(SourceTemplate);

			if (const UObject* AssociatedAsset = SourceTemplate->AdditionalStatObject())
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("AssetType"), AssociatedAsset->GetClass()->GetDisplayNameText());
				Args.Add(TEXT("AssetName"), FText::FromString(AssociatedAsset->GetName()));
				CachedAssetTitle = FText::Format(LOCTEXT("AddComponentAssetDescription", "{AssetType} {AssetName}"), Args);
			}
			else if ((SubActorComp != nullptr) && (SubActorComp->GetChildActorClass() != nullptr))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("ComponentClassName"), SubActorComp->GetChildActorClass()->GetDisplayNameText());
				CachedAssetTitle = FText::Format(LOCTEXT("AddChildActorComponent", "Actor Class {ComponentClassName}"), Args);
			}
			else
			{
				CachedAssetTitle = FText::GetEmpty();
			}
		}
	}

	if (!CachedNodeTitle.IsEmpty())
	{
		if (TitleType == ENodeTitleType::FullTitle)
		{
			return FText::Format(LOCTEXT("FullAddComponentTitle", "{0}\n{1}"), CachedNodeTitle, CachedAssetTitle);
		}
		else if (!CachedAssetTitle.IsEmpty())
		{
			return FText::Format(LOCTEXT("ShortAddComponentTitle", "{0} [{1}]"), CachedNodeTitle, CachedAssetTitle);
		}
		else
		{
			return CachedNodeTitle;
		}
	}
	return Super::GetNodeTitle(TitleType);
}

FString UK2Node_AddComponent::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Blueprint/UK2Node_AddComponent");
}

FString UK2Node_AddComponent::GetDocumentationExcerptName() const
{
	UEdGraphPin* TemplateNamePin = GetTemplateNamePin();
	UBlueprint* Blueprint = GetBlueprint();

	if ((TemplateNamePin != NULL) && (Blueprint != NULL))
	{
		FString TemplateName = TemplateNamePin->DefaultValue;
		UActorComponent* SourceTemplate = Blueprint->FindTemplateByName(FName(*TemplateName));

		if (SourceTemplate != NULL)
		{
			return SourceTemplate->GetClass()->GetName();
		}
	}

	return Super::GetDocumentationExcerptName();
}

bool UK2Node_AddComponent::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	return (Blueprint != nullptr) && FBlueprintEditorUtils::IsActorBased(Blueprint) && Super::IsCompatibleWithGraph(Graph);
}

void UK2Node_AddComponent::ReconstructNode()
{
	Super::ReconstructNode();

	if (GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::ComponentTemplateClassSupport)
	{
		UActorComponent* Template = GetTemplateFromNode();
		if (Template != nullptr)
		{
			TemplateType = Template->GetClass();
		}
	}
}

void UK2Node_AddComponent::FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	if (UK2Node_AddComponent* OtherGraphNode = Cast<UK2Node_AddComponent>(OtherNode))
	{
		UActorComponent* MyComponent = GetTemplateFromNode();
		UActorComponent* OtherComponent = OtherGraphNode->GetTemplateFromNode();
		if (MyComponent && OtherComponent)
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.ToolTip = LOCTEXT("DIF_ComponentTemplatePropertyToolTip", "A property of the component template has changed");
			Diff.Category = EDiffType::MODIFICATION;

			DiffProperties(MyComponent->GetClass(), OtherComponent->GetClass(), MyComponent, OtherComponent, Results, Diff);
		}
	}
}

void UK2Node_AddComponent::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* TransformPin = GetRelativeTransformPin();
	if (TransformPin && !TransformPin->LinkedTo.Num())
	{
		FString DefaultValue;

		// Try and find the template and get relative transform from it
		UEdGraphPin* TemplateNamePin = GetTemplateNamePinChecked();
		const FString TemplateName = TemplateNamePin->DefaultValue;
		check(CompilerContext.Blueprint);
		USceneComponent* SceneCompTemplate = Cast<USceneComponent>(CompilerContext.Blueprint->FindTemplateByName(FName(*TemplateName)));
		if (SceneCompTemplate)
		{
			FTransform TemplateTransform = FTransform(SceneCompTemplate->GetRelativeRotation(), SceneCompTemplate->GetRelativeLocation(), SceneCompTemplate->GetRelativeScale3D());
			DefaultValue = TemplateTransform.ToString();
		}

		UEdGraphPin* ValuePin = InnerHandleAutoCreateRef(this, TransformPin, CompilerContext, SourceGraph, !DefaultValue.IsEmpty());
		if (ValuePin)
		{
			ValuePin->DefaultValue = DefaultValue;
		}
	}

	if (bHasExposedVariable)
	{
		UK2Node_AddComponent* NewNode = CompilerContext.SpawnIntermediateNode<UK2Node_AddComponent>(this, SourceGraph); 
		NewNode->SetFromFunction(GetTargetFunction());
		NewNode->AllocateDefaultPinsWithoutExposedVariables();

		// function parameters
		CompilerContext.MovePinLinksToIntermediate(*FindPin(UEdGraphSchema_K2::PN_Self), *NewNode->FindPin(UEdGraphSchema_K2::PN_Self));
		CompilerContext.MovePinLinksToIntermediate(*GetTemplateNamePinChecked(), *NewNode->GetTemplateNamePinChecked());
		CompilerContext.MovePinLinksToIntermediate(*GetRelativeTransformPin(), *NewNode->GetRelativeTransformPin());
		CompilerContext.MovePinLinksToIntermediate(*GetManualAttachmentPin(), *NewNode->GetManualAttachmentPin());

		static const FName DeferredFinishPinName = TEXT("bDeferredFinish");

		UEdGraphPin* DeferredFinishPin = NewNode->FindPinChecked(DeferredFinishPinName);
		DeferredFinishPin->DefaultValue = TEXT("true");

		UEdGraphPin* ReturnPin = NewNode->GetReturnValuePin();
		UEdGraphPin* OriginalReturnPin = GetReturnValuePin();
		check(ReturnPin && OriginalReturnPin);
		ReturnPin->PinType = OriginalReturnPin->PinType;
		CompilerContext.MovePinLinksToIntermediate(*OriginalReturnPin, *ReturnPin);
		// exec in
		CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *NewNode->GetExecPin());

		UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes( CompilerContext, SourceGraph, NewNode, this, ReturnPin, GetSpawnedType() );

		static const FName FinishAddFuncName = GET_FUNCTION_NAME_CHECKED(AActor, FinishAddComponent);
		static const FName ComponentParamName = TEXT("Component");

		UK2Node_CallFunction* FinishAdd = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		FinishAdd->FunctionReference.SetExternalMember(FinishAddFuncName, AActor::StaticClass());
		FinishAdd->AllocateDefaultPins();

		UEdGraphPin* FinishAdd_ComponentPin = FinishAdd->FindPinChecked(ComponentParamName);
		UEdGraphPin* FinishAdd_RelativeTransformPin = FinishAdd->FindPinChecked(NAME_RelativeTransform);
		UEdGraphPin* FinishAdd_ManualAttachmentPin = FinishAdd->FindPinChecked(NAME_ManualAttachment);

		FinishAdd_ComponentPin->MakeLinkTo(ReturnPin);
		CompilerContext.CopyPinLinksToIntermediate(*NewNode->GetRelativeTransformPin(), *FinishAdd_RelativeTransformPin);
		CompilerContext.CopyPinLinksToIntermediate(*NewNode->GetManualAttachmentPin(), *FinishAdd_ManualAttachmentPin);

		LastThen->MakeLinkTo(FinishAdd->GetExecPin());
		CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *FinishAdd->GetThenPin());
		BreakAllNodeLinks();
	}
}

FName UK2Node_AddComponent::MakeNewComponentTemplateName(UObject* InOuter, UClass* InComponentClass)
{
	FName TemplateName = NAME_None;

	int32& Counter = GetBlueprint()->ComponentTemplateNameIndex.FindOrAdd(InComponentClass->GetFName());
	FString ComponentClassNameAsString = InComponentClass->GetName();
	if (InComponentClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		ComponentClassNameAsString.RemoveFromEnd("_C");
	}

	do
	{
		TemplateName = FName(*FString::Printf(TEXT("%s%s-%d"), *ComponentTemplateNamePrefix, *ComponentClassNameAsString, Counter++));
		if (StaticFindObjectFast(InComponentClass, InOuter, TemplateName) != nullptr)
		{
			TemplateName = NAME_None;
		}

	} while (TemplateName == NAME_None);

	return TemplateName;
}

void UK2Node_AddComponent::MakeNewComponentTemplate()
{
	// There is a template associated with this node that should be unique, but after a node is duplicated, it either points to a
	// template associated with the original node, or to nothing (when pasting into a different blueprint)
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UBlueprint* Blueprint = GetBlueprint();

	// Find the template name and return type pins
	UEdGraphPin* TemplateNamePin = GetTemplateNamePin();
	UEdGraphPin* ReturnPin = GetReturnValuePin();
	if ((TemplateNamePin != nullptr) && (ReturnPin != nullptr))
	{
		// Find the current template if it exists
		const FName TemplateName = FName(*TemplateNamePin->DefaultValue);
		UActorComponent* SourceTemplate = Blueprint->FindTemplateByName(TemplateName);

		// Determine the type of the component needed
		UClass* ComponentClass = Cast<UClass>(ReturnPin->PinType.PinSubCategoryObject.Get());

		if (ComponentClass && FKismetEditorUtilities::IsClassABlueprintSpawnableComponent(ComponentClass))
		{
			ensure(nullptr != Cast<UBlueprintGeneratedClass>(Blueprint->GeneratedClass));

			// Create a new template object name
			const FName NewTemplateName = MakeNewComponentTemplateName(Blueprint->GeneratedClass, ComponentClass);

			// Create a new template object and update the template pin to point to it
			UActorComponent* NewTemplate = NewObject<UActorComponent>(Blueprint->GeneratedClass, ComponentClass, NewTemplateName, RF_ArchetypeObject | RF_Public);
			Blueprint->ComponentTemplates.Add(NewTemplate);

			TemplateNamePin->DefaultValue = NewTemplate->GetName();

			// Copy the old template data over to the new template if it's compatible
			if ((SourceTemplate != nullptr) && (SourceTemplate->GetClass()->IsChildOf(ComponentClass)))
			{
				UEngine::CopyPropertiesForUnrelatedObjects(SourceTemplate, NewTemplate);
			}
			else if (TemplateBlueprint.Len() > 0)
			{
				// try to find/load our blueprint to copy the template
				UBlueprint* SourceBlueprint = FindObject<UBlueprint>(nullptr, *TemplateBlueprint);
				if (SourceBlueprint != nullptr)
				{
					SourceTemplate = SourceBlueprint->FindTemplateByName(TemplateName);
					if ((SourceTemplate != nullptr) && (SourceTemplate->GetClass()->IsChildOf(ComponentClass)))
					{
						UEngine::CopyPropertiesForUnrelatedObjects(SourceTemplate, NewTemplate);
					}
				}

				TemplateBlueprint.Empty();
			}
		}
		else
		{
			// Clear the template connection; can't resolve the type of the component to create
			TemplateNamePin->DefaultValue = TEXT("");
		}
	}
}

#undef LOCTEXT_NAMESPACE
