// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_AddComponentByClass.h"

#include "Components/ActorComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AddComponentByClass)


#define LOCTEXT_NAMESPACE "ActorComponent"

struct FK2Node_AddComponentByClassHelper
{
	static const FName ClassPinName;
	static const FName TransformPinName;
	static const FName ManualAttachmentPinName;
	static const FName DeferredFinishPinName;
	static const FName ComponentClassPinName;
	static const FName ActorComponentPinName;
};

const FName FK2Node_AddComponentByClassHelper::ClassPinName(TEXT("Class"));
const FName FK2Node_AddComponentByClassHelper::TransformPinName(TEXT("RelativeTransform"));
const FName FK2Node_AddComponentByClassHelper::ManualAttachmentPinName(TEXT("bManualAttachment"));
const FName FK2Node_AddComponentByClassHelper::DeferredFinishPinName(TEXT("bDeferredFinish"));
const FName FK2Node_AddComponentByClassHelper::ComponentClassPinName(TEXT("Class"));
const FName FK2Node_AddComponentByClassHelper::ActorComponentPinName(TEXT("Component"));

UK2Node_AddComponentByClass::UK2Node_AddComponentByClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Adds a component to an actor");
}

void UK2Node_AddComponentByClass::AllocateDefaultPins()
{
	// Create a dummy AActor::AddComponentByClass node to copy pins off of
	UK2Node_CallFunction* AddComponentByClassNode = NewObject<UK2Node_CallFunction>(GetGraph());
	AddComponentByClassNode->SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, AddComponentByClass)));
	AddComponentByClassNode->AllocateDefaultPins();

	auto CreatePinCopy = [&](UEdGraphPin* ProtoPin)
	{
		const FEdGraphPinType& ProtoPinType = ProtoPin->PinType;
		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.ContainerType = ProtoPinType.ContainerType;
		PinParams.ValueTerminalType = ProtoPinType.PinValueType;
		UEdGraphPin* Pin = CreatePin(ProtoPin->Direction, ProtoPinType.PinCategory, ProtoPinType.PinSubCategory, ProtoPinType.PinSubCategoryObject.Get(), ProtoPin->PinName, PinParams);
		Pin->PinToolTip = MoveTemp(ProtoPin->PinToolTip);
		Pin->PinFriendlyName = MoveTemp(ProtoPin->PinFriendlyName);
		return Pin;
	};

	// Create the self pin first because we want it to come before the class pin
	UEdGraphPin* SelfPin = CreatePinCopy(GetDefault<UEdGraphSchema_K2>()->FindSelfPin(*AddComponentByClassNode, EGPD_Input));

	Super::AllocateDefaultPins();

	// Put the exec pin first in the Pins array so it appears before the self pin
	UEdGraphPin* ExecPin = GetExecPin();
	Pins.Remove(ExecPin);
	Pins.Insert(ExecPin, 0);

	CreatePinCopy(AddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::ManualAttachmentPinName));
	CreatePinCopy(AddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::TransformPinName));

	AddComponentByClassNode->DestroyNode();

	SetSceneComponentPinsHidden(true);
}

FText UK2Node_AddComponentByClass::GetBaseNodeTitle() const
{
	return LOCTEXT("AddComponent_BaseTitle", "Add Component by Class");
}

FText UK2Node_AddComponentByClass::GetDefaultNodeTitle() const
{
	return GetBaseNodeTitle();
}

FText UK2Node_AddComponentByClass::GetNodeTitleFormat() const
{
	return LOCTEXT("AddComponent", "Add {ClassName}");
}

UClass* UK2Node_AddComponentByClass::GetClassPinBaseClass() const
{
	return UActorComponent::StaticClass();
}

void UK2Node_AddComponentByClass::SetSceneComponentPinsHidden(const bool bHidden)
{
	UEdGraphPin* ManualAttachmentPin = GetManualAttachmentPin();
	ManualAttachmentPin->SafeSetHidden(bHidden);

	UEdGraphPin* TransformPin = GetRelativeTransformPin();
	TransformPin->SafeSetHidden(bHidden);
}

void UK2Node_AddComponentByClass::PinDefaultValueChanged(UEdGraphPin* ChangedPin)
{
	if (ChangedPin && (ChangedPin->PinName == FK2Node_AddComponentByClassHelper::ClassPinName))
	{
		// If the class pin changes set the scene components hidden and then CreatePinsForClass will unhide if appropriate
		SetSceneComponentPinsHidden(true);
	}

	Super::PinDefaultValueChanged(ChangedPin);
}

void UK2Node_AddComponentByClass::CreatePinsForClass(UClass* InClass, TArray<UEdGraphPin*>* OutClassPins)
{
	Super::CreatePinsForClass(InClass, OutClassPins);

	// Based on the supplied class we hide the transform and manual attachment pins
	const bool bIsSceneComponent = InClass ? InClass->IsChildOf<USceneComponent>() : false;
	SetSceneComponentPinsHidden(!bIsSceneComponent);
}

UEdGraphPin* UK2Node_AddComponentByClass::GetRelativeTransformPin() const
{
	return FindPinChecked(FK2Node_AddComponentByClassHelper::TransformPinName);
}

UEdGraphPin* UK2Node_AddComponentByClass::GetManualAttachmentPin() const
{
	return FindPinChecked(FK2Node_AddComponentByClassHelper::ManualAttachmentPinName);
}

bool UK2Node_AddComponentByClass::IsSceneComponent() const
{
	if (UEdGraphPin* SpawnClassPin = GetClassPin())
	{
		if (UClass* SpawnClass = Cast<UClass>(SpawnClassPin->DefaultObject))
		{
			return SpawnClass->IsChildOf<USceneComponent>();
		}
	}
	return false;
}

bool UK2Node_AddComponentByClass::IsSpawnVarPin(UEdGraphPin* Pin) const
{
	return(Super::IsSpawnVarPin(Pin)
	       && Pin->PinName != UEdGraphSchema_K2::PN_Self
	       && Pin->PinName != FK2Node_AddComponentByClassHelper::TransformPinName
	       && Pin->PinName != FK2Node_AddComponentByClassHelper::ManualAttachmentPinName);	  
}

void UK2Node_AddComponentByClass::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);


	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* SpawnNodeExec = GetExecPin();
	UEdGraphPin* SpawnOwnerPin = K2Schema->FindSelfPin(*this, EGPD_Input);
	UEdGraphPin* SpawnClassPin = GetClassPin();
	UEdGraphPin* SpawnTransformPin = GetRelativeTransformPin();
	UEdGraphPin* SpawnManualAttachmentPin = GetManualAttachmentPin();

	UEdGraphPin* SpawnNodeThen = GetThenPin();
	UEdGraphPin* SpawnNodeResult = GetResultPin();

	UClass* SpawnClass = (SpawnClassPin ? Cast<UClass>(SpawnClassPin->DefaultObject) : nullptr);
	if (SpawnClassPin == nullptr || ((SpawnClass == nullptr) && (SpawnClassPin->LinkedTo.Num() == 0)))
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("AddComponentByClassNodeMissingClass_Error", "Spawn node @@ must have a class specified.").ToString(), this);
		// we break exec links so this is the only error we get, don't want the AddComponentByClass node being considered and giving 'unexpected node' type warnings
		BreakAllNodeLinks();
		return;
	}

	UK2Node_CallFunction* CallAddComponentByClassNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallAddComponentByClassNode->SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, AddComponentByClass)));
	CallAddComponentByClassNode->AllocateDefaultPins();

	// store off the class to spawn before we mutate pin connections:
	UClass* ClassToSpawn = GetClassToSpawn();

	UEdGraphPin* CallAddComponentByClassExec = CallAddComponentByClassNode->GetExecPin();
	UEdGraphPin* CallAddComponentByClassTypePin = CallAddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::ComponentClassPinName);
	UEdGraphPin* CallAddComponentByClassOwnerPin = K2Schema->FindSelfPin(*CallAddComponentByClassNode, EGPD_Input);
	UEdGraphPin* CallAddComponentByClassTransformPin = CallAddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::TransformPinName);
	UEdGraphPin* CallAddComponentByClassManualAttachmentPin = CallAddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::ManualAttachmentPinName);
	UEdGraphPin* CallAddComponentByClassResult = CallAddComponentByClassNode->GetReturnValuePin();

	// set properties on relative transform pin to allow it to be unconnected
	CallAddComponentByClassTransformPin->bDefaultValueIsIgnored = true;
	CallAddComponentByClassTransformPin->PinType.bIsReference = false;

	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeExec, *CallAddComponentByClassExec);
	CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallAddComponentByClassTypePin);
	CompilerContext.MovePinLinksToIntermediate(*SpawnOwnerPin, *CallAddComponentByClassOwnerPin);
	CompilerContext.MovePinLinksToIntermediate(*SpawnTransformPin, *CallAddComponentByClassTransformPin);
	CompilerContext.MovePinLinksToIntermediate(*SpawnManualAttachmentPin, *CallAddComponentByClassManualAttachmentPin);

	// Move result connection from spawn node to specific AddComponent function
	CallAddComponentByClassResult->PinType = SpawnNodeResult->PinType; // Copy type so it uses the right actor subclass
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallAddComponentByClassResult);

	//////////////////////////////////////////////////////////////////////////
	// create 'set var' nodes

	UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes(CompilerContext, SourceGraph, CallAddComponentByClassNode, this, CallAddComponentByClassResult, ClassToSpawn, CallAddComponentByClassTypePin);

	if (LastThen != CallAddComponentByClassNode->GetThenPin())
	{
		UEdGraphPin* CallAddComponentByClassDeferredFinishPin = CallAddComponentByClassNode->FindPinChecked(FK2Node_AddComponentByClassHelper::DeferredFinishPinName);
		CallAddComponentByClassDeferredFinishPin->DefaultValue = TEXT("true");

		UK2Node_CallFunction* CallRegisterComponentNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallRegisterComponentNode->SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, FinishAddComponent)));
		CallRegisterComponentNode->AllocateDefaultPins();

		// Links execution from last assignment to 'RegisterComponent '
		LastThen->MakeLinkTo(CallRegisterComponentNode->GetExecPin());

		// Link the pins to RegisterComponent node
		UEdGraphPin* CallRegisterComponentByClassOwnerPin = K2Schema->FindSelfPin(*CallRegisterComponentNode, EGPD_Input);
		UEdGraphPin* CallRegisterComponentByClassComponentPin = CallRegisterComponentNode->FindPinChecked(FK2Node_AddComponentByClassHelper::ActorComponentPinName);
		UEdGraphPin* CallRegisterComponentByClassTransformPin = CallRegisterComponentNode->FindPinChecked(FK2Node_AddComponentByClassHelper::TransformPinName);
		UEdGraphPin* CallRegisterComponentByClassManualAttachmentPin = CallRegisterComponentNode->FindPinChecked(FK2Node_AddComponentByClassHelper::ManualAttachmentPinName);

		// set properties on relative transform pin to allow it to be unconnected
		CallRegisterComponentByClassTransformPin->bDefaultValueIsIgnored = true;
		CallRegisterComponentByClassTransformPin->PinType.bIsReference = false;

		CompilerContext.CopyPinLinksToIntermediate(*CallAddComponentByClassOwnerPin, *CallRegisterComponentByClassOwnerPin);
		CompilerContext.CopyPinLinksToIntermediate(*CallAddComponentByClassTransformPin, *CallRegisterComponentByClassTransformPin);
		CompilerContext.CopyPinLinksToIntermediate(*CallAddComponentByClassManualAttachmentPin, *CallRegisterComponentByClassManualAttachmentPin);

		CallRegisterComponentByClassComponentPin->MakeLinkTo(CallAddComponentByClassResult);

		LastThen = CallRegisterComponentNode->GetThenPin();
	}

	// Move 'then' connection from AddComponent node to the last 'then'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *LastThen);

	// Break any links to the expanded node
	BreakAllNodeLinks();
}


#undef LOCTEXT_NAMESPACE
