// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AddPinInterface.h"

#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AddPinInterface)

FName IK2Node_AddPinInterface::GetNameForAdditionalPin(int32 PinIndex)
{
	check(PinIndex < GetMaxInputPinsNum());
	const FName Name(*FString::Chr(TCHAR('A') + static_cast<TCHAR>(PinIndex)));
	return Name;
}
