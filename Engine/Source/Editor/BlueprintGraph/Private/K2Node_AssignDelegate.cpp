// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AssignDelegate.h"

#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "ObjectEditorUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AssignDelegate)

class UBlueprint;

#define LOCTEXT_NAMESPACE "K2Node_AssignDelegate"

//------------------------------------------------------------------------------
UK2Node_AssignDelegate::UK2Node_AssignDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
FText UK2Node_AssignDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		if (CachedListTitle.IsOutOfDate(this))
		{
			FText PropertyName;
			if (FProperty* Property = GetProperty())
			{
				bool const bShowFriendlyNames = GetDefault<UEditorStyleSettings>()->bShowFriendlyNames;
				PropertyName = FText::FromString(bShowFriendlyNames ? UEditorEngine::GetFriendlyName(Property) : Property->GetName());

				// FText::Format() is slow, so we cache this to save on performance
				CachedListTitle.SetCachedText(FText::Format(LOCTEXT("AssignDelegateTitle", "Assign {0}"), PropertyName), this);
			}
			else
			{
				return LOCTEXT("InvalidPropertyTitle", "Assign <invalid delegate>");
			}
		}
		return CachedListTitle;
	}
	
	return Super::GetNodeTitle(TitleType);
}

//------------------------------------------------------------------------------
FText UK2Node_AssignDelegate::GetMenuCategory() const
{
	FText MenuCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Delegates);
	if (FProperty* Property = GetProperty())
	{
		MenuCategory = FText::FromString(FObjectEditorUtils::GetCategory(Property));
	}
	return MenuCategory;
}

//------------------------------------------------------------------------------
bool UK2Node_AssignDelegate::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool bIsCompatible = Super::IsCompatibleWithGraph(TargetGraph);

	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	check(Blueprint != nullptr);
	UEdGraphSchema const* Schema = TargetGraph->GetSchema();

	EGraphType GraphType = Schema->GetGraphType(TargetGraph);
	bool const bAllowEvents = (GraphType == GT_Ubergraph) && FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint);
	
	return bAllowEvents && Super::IsCompatibleWithGraph(TargetGraph);
}

//------------------------------------------------------------------------------
void UK2Node_AssignDelegate::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();
	
	UEdGraphPin* InDelegatePin = GetDelegatePin();
	if (InDelegatePin->LinkedTo.Num() == 0)
	{
		if (FMulticastDelegateProperty* DelegateProp = CastField<FMulticastDelegateProperty>(GetProperty()))
		{
			FVector2D const LocationOffset(-150.f, +150.f);

			FText DelegateName = FText::FromName(DelegateProp->GetFName());
			FText DesiredEventName = FText::Format(LOCTEXT("BindedEventName", "{0}_Event"), DelegateName);
			FName EventName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprint(), DesiredEventName.ToString());

			UK2Node_CustomEvent* EventNode = UK2Node_CustomEvent::CreateFromFunction(
				FVector2D(NodePosX, NodePosY) + LocationOffset,
				GetGraph(), EventName.ToString(), DelegateProp->SignatureFunction, /*bSelectNewNode =*/false);

			if (EventNode != nullptr)
			{
				UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();
				UEdGraphPin* OutDelegatePin = EventNode->FindPinChecked(UK2Node_CustomEvent::DelegateOutputName);

				K2Schema->TryCreateConnection(OutDelegatePin, InDelegatePin);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
