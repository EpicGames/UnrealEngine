// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_AssignmentStatement.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "VariableSetHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AssignmentStatement)

struct FBPTerminal;

#define LOCTEXT_NAMESPACE "K2Node_AssignmentStatement"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_AssignmentStatement

class FKCHandler_AssignmentStatement : public FKCHandler_VariableSet
{
public:
	FKCHandler_AssignmentStatement(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_VariableSet(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		using namespace UE::KismetCompiler;

		UEdGraphPin* VariablePin = Node->FindPin(TEXT("Variable"));
		UEdGraphPin* ValuePin = Node->FindPin(TEXT("Value"));

		if ((VariablePin == NULL) || (ValuePin == NULL))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MissingPins_Error", "Missing pin(s) on @@; expected a pin named Variable and a pin named Value").ToString(), Node);
			return;
		}

		if (VariablePin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoVariableConnected_Error", "A variable needs to be connected to @@").ToString(), VariablePin);
			return;
		}

		ValidateAndRegisterNetIfLiteral(Context, ValuePin);

		UEdGraphPin* VariablePinNet = FEdGraphUtilities::GetNetFromPin(VariablePin);
		UEdGraphPin* ValuePinNet = FEdGraphUtilities::GetNetFromPin(ValuePin);

		if (VariablePinNet && ValuePinNet)
		{
			CastingUtils::FConversion Conversion =
				CastingUtils::GetFloatingPointConversion(*ValuePinNet, *VariablePinNet);

			if (Conversion.Type != CastingUtils::FloatingPointCastType::None)
			{
				FBPTerminal* NewTerm = CastingUtils::MakeImplicitCastTerminal(Context, VariablePinNet, Node);

				Context.ImplicitCastMap.Add(VariablePin, CastingUtils::FImplicitCastParams{Conversion, NewTerm, Node});
			}
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoVariableOrValueNets_Error", "Expected Variable and Value pins to have valid connections in @@").ToString(), Node);
		}
	}


	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		using namespace UE::KismetCompiler;

		UEdGraphPin* VariablePin = Node->FindPin(TEXT("Variable"));
		UEdGraphPin* ValuePin = Node->FindPin(TEXT("Value"));

		check(VariablePin);
		check(VariablePin->LinkedTo.Num() == 1);
		check(ValuePin);

		InnerAssignment(Context, Node, VariablePin, ValuePin);

		// Generate the output impulse from this node
		GenerateSimpleThenGoto(Context, *Node);

		// The assignment node is very much a non-standard node: its pins don't directly reference any data, but their nets do.
		// The only cast that could happen is from the value pin's net to the variable's pin net, which we handle in RegisterNets.
		// Due to how RegisterImplicitCasts works, it can actually register an erroneous entry to the value pin,
		// which we need to remove ourselves.
		CastingUtils::RemoveRegisteredImplicitCast(Context, ValuePin);
	}

protected:
	virtual bool UsesVariablePinAsKey() const override { return true; }
};


FName UK2Node_AssignmentStatement::VariablePinName(TEXT("Variable"));
FName UK2Node_AssignmentStatement::ValuePinName(TEXT("Value"));


UK2Node_AssignmentStatement::UK2Node_AssignmentStatement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_AssignmentStatement::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	UEdGraphPin* VariablePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, VariablePinName);
	UEdGraphPin* ValuePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, ValuePinName);

	Super::AllocateDefaultPins();
}

FText UK2Node_AssignmentStatement::GetTooltipText() const
{
	return LOCTEXT("AssignmentStatementTooltip", "Assigns Value to Variable");
}

FText UK2Node_AssignmentStatement::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Assign", "Assign");
}

bool UK2Node_AssignmentStatement::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool bIsCompatible = Super::IsCompatibleWithGraph(TargetGraph);
	if (bIsCompatible)
	{
		const EGraphType GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
		bIsCompatible = (GraphType == GT_Macro);
	}

	return bIsCompatible;
}

bool UK2Node_AssignmentStatement::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// These nodes can be pasted anywhere that UK2Node's are compatible with the graph
	// Avoiding the call to IsCompatibleWithGraph because these nodes should normally only
	// be placed in Macros, but it's nice to be able to paste Macro functionality anywhere.
	return Super::IsCompatibleWithGraph(TargetGraph);
}

void UK2Node_AssignmentStatement::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	UEdGraphPin* VariablePin = FindPin(TEXT("Variable"));
	UEdGraphPin* ValuePin = FindPin(TEXT("Value"));

	if ((VariablePin->LinkedTo.Num() == 0) && (ValuePin->LinkedTo.Num() == 0))
	{
		// Restore the wildcard status
		VariablePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
		VariablePin->PinType.PinSubCategory = TEXT("");
		VariablePin->PinType.PinSubCategoryObject = NULL;
		ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
		ValuePin->PinType.PinSubCategory = TEXT("");
		ValuePin->PinType.PinSubCategoryObject = NULL;
	}
	else if (Pin->LinkedTo.Num() > 0 && 
		(	Pin->LinkedTo[0]->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard ||
			Pin->LinkedTo[0]->PinType.PinCategory == Pin->PinType.PinCategory) )
	{
		Pin->PinType = Pin->LinkedTo[0]->PinType;

		// Enforce the type on the other pin
		if (VariablePin == Pin)
		{
			ValuePin->PinType = VariablePin->PinType;
			UEdGraphSchema_K2::ValidateExistingConnections(ValuePin);
		}
		else
		{
			VariablePin->PinType = ValuePin->PinType;
			UEdGraphSchema_K2::ValidateExistingConnections(VariablePin);
		}
	}
}

void UK2Node_AssignmentStatement::PostReconstructNode()
{
	UEdGraphPin* VariablePin = FindPin(TEXT("Variable"));
	UEdGraphPin* ValuePin = FindPin(TEXT("Value"));

	PinConnectionListChanged(VariablePin);
	PinConnectionListChanged(ValuePin);

	Super::PostReconstructNode();
}

UEdGraphPin* UK2Node_AssignmentStatement::GetVariablePin() const
{
	UEdGraphPin* Pin = FindPin(VariablePinName);
	check(Pin != NULL);
	return Pin;
}

UEdGraphPin* UK2Node_AssignmentStatement::GetValuePin() const
{
	UEdGraphPin* Pin = FindPin(ValuePinName);
	check(Pin != NULL);
	return Pin;
}

FNodeHandlingFunctor* UK2Node_AssignmentStatement::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_AssignmentStatement(CompilerContext);
}

void UK2Node_AssignmentStatement::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_AssignmentStatement::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Macro);
}

#undef LOCTEXT_NAMESPACE
