// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_AsyncAction.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintNodeSpawner.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_AsyncAction)


#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_AsyncAction::UK2Node_AsyncAction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ProxyActivateFunctionName = GET_FUNCTION_NAME_CHECKED(UBlueprintAsyncActionBase, Activate);
}

void UK2Node_AsyncAction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeFunc(UEdGraphNode* NewNode, bool /*bIsTemplateNode*/, TWeakObjectPtr<UFunction> FunctionPtr)
		{
			UK2Node_AsyncAction* AsyncTaskNode = CastChecked<UK2Node_AsyncAction>(NewNode);
			if (FunctionPtr.IsValid())
			{
				UFunction* Func = FunctionPtr.Get();
				FObjectProperty* ReturnProp = CastFieldChecked<FObjectProperty>(Func->GetReturnProperty());
						
				AsyncTaskNode->ProxyFactoryFunctionName = Func->GetFName();
				AsyncTaskNode->ProxyFactoryClass        = Func->GetOuterUClass();
				AsyncTaskNode->ProxyClass               = ReturnProp->PropertyClass;
			}
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterClassFactoryActions<UBlueprintAsyncActionBase>(FBlueprintActionDatabaseRegistrar::FMakeFuncSpawnerDelegate::CreateLambda([NodeClass](const UFunction* FactoryFunc)->UBlueprintNodeSpawner*
	{
		UClass* FactoryClass = FactoryFunc ? FactoryFunc->GetOwnerClass() : nullptr;
		if (FactoryClass && FactoryClass->HasMetaData(TEXT("HasDedicatedAsyncNode")))
		{
			// Wants to use a more specific blueprint node to handle the async action
			return nullptr;
		}

		UBlueprintNodeSpawner* NodeSpawner = UBlueprintFunctionNodeSpawner::Create(FactoryFunc);
		check(NodeSpawner != nullptr);
		NodeSpawner->NodeClass = NodeClass;

		TWeakObjectPtr<UFunction> FunctionPtr = MakeWeakObjectPtr(const_cast<UFunction*>(FactoryFunc));
		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(GetMenuActions_Utils::SetNodeFunc, FunctionPtr);

		return NodeSpawner;
	}) );
}

void UK2Node_AsyncAction::InitializeProxyFromFunction(const UFunction* ProxyFunction)
{
	if (ensure(ProxyFunction))
	{
		const FObjectProperty* ReturnProp = CastField<FObjectProperty>(ProxyFunction->GetReturnProperty());
		if (ensure(ReturnProp))
		{
			ProxyFactoryFunctionName = ProxyFunction->GetFName();
			ProxyFactoryClass = ProxyFunction->GetOuterUClass();
			ProxyClass = ReturnProp->PropertyClass;
		}
	}
}

#undef LOCTEXT_NAMESPACE
