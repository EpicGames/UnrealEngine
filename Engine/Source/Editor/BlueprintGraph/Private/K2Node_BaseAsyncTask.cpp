// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_BaseAsyncTask.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Self.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "ObjectTools.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_BaseAsyncTask)

struct FStringFormatArg;

#define LOCTEXT_NAMESPACE "UK2Node_BaseAsyncTask"

UK2Node_BaseAsyncTask::UK2Node_BaseAsyncTask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ProxyFactoryFunctionName(NAME_None)
	, ProxyFactoryClass(nullptr)
	, ProxyClass(nullptr)
	, ProxyActivateFunctionName(NAME_None)
	, bPinTooltipsValid(false)
{
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAll;
}

FText UK2Node_BaseAsyncTask::GetTooltipText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("FunctionTooltip"), FText::FromString(ObjectTools::GetDefaultTooltipForFunction(GetFactoryFunction())));
	Args.Add(TEXT("LatentString"), NSLOCTEXT("K2Node", "LatentFunction", "Latent. This node will complete at a later time. Latent nodes can only be placed in event graphs."));
	
	return FText::Format(LOCTEXT("AsyncTaskTooltip", "{FunctionTooltip}\n\n{LatentString}"), Args);
}

FText UK2Node_BaseAsyncTask::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (GetFactoryFunction() == nullptr)
	{
		return FText(LOCTEXT("UK2Node_BaseAsyncTaskGetNodeTitle", "Async Task: Missing Function"));
	}
	const FText FunctionToolTipText = ObjectTools::GetUserFacingFunctionName(GetFactoryFunction());
	return FunctionToolTipText;
}

bool UK2Node_BaseAsyncTask::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool bIsCompatible = false;
	// Can only place events in ubergraphs and macros (other code will help prevent macros with latents from ending up in functions), and basicasync task creates an event node:
	EGraphType GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
	if (GraphType == EGraphType::GT_Ubergraph || GraphType == EGraphType::GT_Macro)
	{
		bIsCompatible = true;
	}
	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

void UK2Node_BaseAsyncTask::AllocateDefaultPins()
{
	InvalidatePinTooltips();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	bool bExposeProxy = false;
	bool bHideThen = false;
	FText ExposeProxyDisplayName;
	for (const UStruct* TestStruct = ProxyClass; TestStruct; TestStruct = TestStruct->GetSuperStruct())
	{
		bExposeProxy |= TestStruct->HasMetaData(TEXT("ExposedAsyncProxy"));
		bHideThen |= TestStruct->HasMetaData(TEXT("HideThen"));
		if (ExposeProxyDisplayName.IsEmpty())
		{
			ExposeProxyDisplayName = TestStruct->GetMetaDataText(TEXT("ExposedAsyncProxy"));
		}
	}

	if (!bHideThen)
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	}

	if (bExposeProxy)
	{
		UEdGraphPin* ProxyPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, ProxyClass, FBaseAsyncTaskHelper::GetAsyncTaskProxyName());
		if (!ExposeProxyDisplayName.IsEmpty())
		{
			ProxyPin->PinFriendlyName = ExposeProxyDisplayName;
		}
	}

	UFunction* Function = GetFactoryFunction();
	if (!bHideThen && Function)
	{
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;
			// invert the check for function inputs (below) and exclude the factory func's return param - the assumption is
			// that the factory method will be returning the proxy object, and that other outputs should be forwarded along 
			// with the 'then' pin
			const bool bIsFunctionOutput = Param->HasAnyPropertyFlags(CPF_OutParm) && !Param->HasAnyPropertyFlags(CPF_ReferenceParm) && !Param->HasAnyPropertyFlags(CPF_ReturnParm);
			if (bIsFunctionOutput)
			{
				UEdGraphPin* Pin = CreatePin(EGPD_Output, NAME_None, Param->GetFName());
				K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType);
			}
		}
	}

	UFunction* DelegateSignatureFunction = nullptr;
	for (TFieldIterator<FProperty> PropertyIt(ProxyClass); PropertyIt; ++PropertyIt)
	{
		if (FMulticastDelegateProperty* Property = CastField<FMulticastDelegateProperty>(*PropertyIt))
		{
			UEdGraphPin* ExecPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, Property->GetFName());
			ExecPin->PinToolTip = Property->GetToolTipText().ToString();
			ExecPin->PinFriendlyName = Property->GetDisplayNameText();

			if (!DelegateSignatureFunction)
			{
				DelegateSignatureFunction = Property->SignatureFunction;
			}
		}
	}

	if (DelegateSignatureFunction)
	{
		for (TFieldIterator<FProperty> PropIt(DelegateSignatureFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;
			const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);
			if (bIsFunctionInput)
			{
				UEdGraphPin* Pin = CreatePin(EGPD_Output, NAME_None, Param->GetFName());
				K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType);

				UK2Node_CallFunction::GeneratePinTooltipFromFunction(*Pin, DelegateSignatureFunction);
			}
		}
	}

	bool bAllPinsGood = true;
	if (Function)
	{
		TSet<FName> PinsToHide;
		FBlueprintEditorUtils::GetHiddenPinsForFunction(GetGraph(), Function, PinsToHide);
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;
			const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);
			if (!bIsFunctionInput)
			{
				// skip function output, it's internal node data 
				continue;
			}

			UEdGraphNode::FCreatePinParams PinParams;
			PinParams.bIsReference = Param->HasAnyPropertyFlags(CPF_ReferenceParm) && bIsFunctionInput;
			UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Param->GetFName(), PinParams);
			const bool bPinGood = (Pin && K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType));

			if (bPinGood)
			{
				// Check for a display name override
				const FString& PinDisplayName = Param->GetMetaData(FBlueprintMetadata::MD_DisplayName);
				if (!PinDisplayName.IsEmpty())
				{
					Pin->PinFriendlyName = FText::FromString(PinDisplayName);
				}
				
				//Flag pin as read only for const reference property
				Pin->bDefaultValueIsIgnored = Param->HasAllPropertyFlags(CPF_ConstParm | CPF_ReferenceParm) && (!Function->HasMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm) || Pin->PinType.IsContainer());

				const bool bAdvancedPin = Param->HasAllPropertyFlags(CPF_AdvancedDisplay);
				Pin->bAdvancedView = bAdvancedPin;
				if(bAdvancedPin && (ENodeAdvancedPins::NoPins == AdvancedPinDisplay))
				{
					AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
				}

				FString ParamValue;
				if (K2Schema->FindFunctionParameterDefaultValue(Function, Param, ParamValue))
				{
					K2Schema->SetPinAutogeneratedDefaultValue(Pin, ParamValue);
				}
				else
				{
					K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
				}

				if (PinsToHide.Contains(Pin->PinName))
				{
					Pin->bHidden = true;
				}
			}

			bAllPinsGood = bAllPinsGood && bPinGood;
		}
	}


	Super::AllocateDefaultPins();
}

const FName UK2Node_BaseAsyncTask::FBaseAsyncTaskHelper::GetAsyncTaskProxyName()
{
	static const FName Name(TEXT("AsyncTaskProxy"));
	return Name;
}

bool UK2Node_BaseAsyncTask::FBaseAsyncTaskHelper::ValidDataPin(const UEdGraphPin* Pin, EEdGraphPinDirection Direction)
{
	const bool bValidDataPin = Pin
		&& !Pin->bOrphanedPin
		&& (Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec);

	const bool bProperDirection = Pin && (Pin->Direction == Direction);

	return bValidDataPin && bProperDirection;
}

bool UK2Node_BaseAsyncTask::FBaseAsyncTaskHelper::CreateDelegateForNewFunction(UEdGraphPin* DelegateInputPin, FName FunctionName, UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext)
{
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(DelegateInputPin && Schema && CurrentNode && SourceGraph && (FunctionName != NAME_None));
	bool bResult = true;

	// WORKAROUND, so we can create delegate from nonexistent function by avoiding check at expanding step
	// instead simply: Schema->TryCreateConnection(AddDelegateNode->GetDelegatePin(), CurrentCENode->FindPinChecked(UK2Node_CustomEvent::DelegateOutputName));
	UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(CurrentNode, SourceGraph);
	SelfNode->AllocateDefaultPins();

	UK2Node_CreateDelegate* CreateDelegateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CreateDelegate>(CurrentNode, SourceGraph);
	CreateDelegateNode->AllocateDefaultPins();
	bResult &= Schema->TryCreateConnection(DelegateInputPin, CreateDelegateNode->GetDelegateOutPin());
	bResult &= Schema->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), CreateDelegateNode->GetObjectInPin());
	CreateDelegateNode->SetFunction(FunctionName);

	return bResult;
}

bool UK2Node_BaseAsyncTask::FBaseAsyncTaskHelper::CopyEventSignature(UK2Node_CustomEvent* CENode, UFunction* Function, const UEdGraphSchema_K2* Schema)
{
	check(CENode && Function && Schema);

	bool bResult = true;
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		const FProperty* Param = *PropIt;
		if (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm))
		{
			FEdGraphPinType PinType;
			bResult &= Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);
			bResult &= (nullptr != CENode->CreateUserDefinedPin(Param->GetFName(), PinType, EGPD_Output));
		}
	}
	return bResult;
}

bool UK2Node_BaseAsyncTask::FBaseAsyncTaskHelper::HandleDelegateImplementation(
	FMulticastDelegateProperty* CurrentProperty, const TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable>& VariableOutputs,
	UEdGraphPin* ProxyObjectPin, UEdGraphPin*& InOutLastThenPin, UEdGraphPin*& OutLastActivatedThenPin,
	UK2Node* CurrentNode, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext)
{
	bool bIsErrorFree = true;
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(CurrentProperty && ProxyObjectPin && InOutLastThenPin && CurrentNode && SourceGraph && Schema);

	UEdGraphPin* PinForCurrentDelegateProperty = CurrentNode->FindPin(CurrentProperty->GetFName());
	if (!PinForCurrentDelegateProperty || (UEdGraphSchema_K2::PC_Exec != PinForCurrentDelegateProperty->PinType.PinCategory))
	{
		FText ErrorMessage = FText::Format(LOCTEXT("WrongDelegateProperty", "BaseAsyncTask: Cannot find execution pin for delegate "), FText::FromString(CurrentProperty->GetName()));
		CompilerContext.MessageLog.Error(*ErrorMessage.ToString(), CurrentNode);
		return false;
	}

	UK2Node_CustomEvent* CurrentCENode = CompilerContext.SpawnIntermediateNode<UK2Node_CustomEvent>(CurrentNode, SourceGraph);
	{
		UK2Node_AddDelegate* AddDelegateNode = CompilerContext.SpawnIntermediateNode<UK2Node_AddDelegate>(CurrentNode, SourceGraph);
		AddDelegateNode->SetFromProperty(CurrentProperty, false, CurrentProperty->GetOwnerClass());
		AddDelegateNode->AllocateDefaultPins();
		bIsErrorFree &= Schema->TryCreateConnection(AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), ProxyObjectPin);
		bIsErrorFree &= Schema->TryCreateConnection(InOutLastThenPin, AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute));
		InOutLastThenPin = AddDelegateNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
		CurrentCENode->CustomFunctionName = *FString::Printf(TEXT("%s_%s"), *CurrentProperty->GetName(), *CompilerContext.GetGuid(CurrentNode));
		CurrentCENode->AllocateDefaultPins();

		bIsErrorFree &= FBaseAsyncTaskHelper::CreateDelegateForNewFunction(AddDelegateNode->GetDelegatePin(), CurrentCENode->GetFunctionName(), CurrentNode, SourceGraph, CompilerContext);
		bIsErrorFree &= FBaseAsyncTaskHelper::CopyEventSignature(CurrentCENode, AddDelegateNode->GetDelegateSignature(), Schema);
	}

	OutLastActivatedThenPin = CurrentCENode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
	for (const FBaseAsyncTaskHelper::FOutputPinAndLocalVariable& OutputPair : VariableOutputs) // CREATE CHAIN OF ASSIGMENTS
	{
		UEdGraphPin* PinWithData = CurrentCENode->FindPin(OutputPair.OutputPin->PinName);
		if (PinWithData == nullptr)
		{
			/*FText ErrorMessage = FText::Format(LOCTEXT("MissingDataPin", "ICE: Pin @@ was expecting a data output pin named {0} on @@ (each delegate must have the same signature)"), FText::FromString(OutputPair.OutputPin->PinName));
			CompilerContext.MessageLog.Error(*ErrorMessage.ToString(), OutputPair.OutputPin, CurrentCENode);
			return false;*/
			continue;
		}

		UK2Node_AssignmentStatement* AssignNode = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(CurrentNode, SourceGraph);
		AssignNode->AllocateDefaultPins();
		bIsErrorFree &= Schema->TryCreateConnection(OutLastActivatedThenPin, AssignNode->GetExecPin());
		bIsErrorFree &= Schema->TryCreateConnection(OutputPair.TempVar->GetVariablePin(), AssignNode->GetVariablePin());
		AssignNode->NotifyPinConnectionListChanged(AssignNode->GetVariablePin());
		bIsErrorFree &= Schema->TryCreateConnection(AssignNode->GetValuePin(), PinWithData);
		AssignNode->NotifyPinConnectionListChanged(AssignNode->GetValuePin());

		OutLastActivatedThenPin = AssignNode->GetThenPin();
	}

	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*PinForCurrentDelegateProperty, *OutLastActivatedThenPin).CanSafeConnect();
	return bIsErrorFree;
}

bool UK2Node_BaseAsyncTask::ExpandDefaultToSelfPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node_CallFunction* IntermediateProxyNode)
{
	if(SourceGraph && IntermediateProxyNode)
	{
		// Connect a self reference pin if there is a TScriptInterface default to self
		if (const UFunction* TargetFunc = IntermediateProxyNode->GetTargetFunction())
		{
			const FString& MetaData = TargetFunc->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
			if (!MetaData.IsEmpty())
			{
				// Find the default to self value pin
				if (UEdGraphPin* DefaultToSelfPin = IntermediateProxyNode->FindPinChecked(MetaData, EGPD_Input))
				{
					// If it has no links then spawn a new self node here
					if (DefaultToSelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface && DefaultToSelfPin->LinkedTo.Num() == 0)
					{
						const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

						UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
						SelfNode->AllocateDefaultPins();
						UEdGraphPin* SelfPin = SelfNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
						// Make a connection from this intermediate self pin to here
						return Schema->TryCreateConnection(DefaultToSelfPin, SelfPin);
					}
				}
			}
		}	
	}
	return true;
}

void UK2Node_BaseAsyncTask::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
    Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(SourceGraph && Schema);
	bool bIsErrorFree = true;

	// Create a call to factory the proxy object
	UK2Node_CallFunction* const CallCreateProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallCreateProxyObjectNode->FunctionReference.SetExternalMember(ProxyFactoryFunctionName, ProxyFactoryClass);
	CallCreateProxyObjectNode->AllocateDefaultPins();
	if (CallCreateProxyObjectNode->GetTargetFunction() == nullptr)
	{
		const FText ClassName = ProxyFactoryClass ? FText::FromString(ProxyFactoryClass->GetName()) : LOCTEXT("MissingClassString", "Unknown Class");
		const FString FormattedMessage = FText::Format(
			LOCTEXT("AsyncTaskErrorFmt", "BaseAsyncTask: Missing function {0} from class {1} for async task @@"),
			FText::FromString(ProxyFactoryFunctionName.GetPlainNameString()),
			ClassName
		).ToString();

		CompilerContext.MessageLog.Error(*FormattedMessage, this);
		return;
	}

	bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Execute), *CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute)).CanSafeConnect();

	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Input))
		{
			UEdGraphPin* DestPin = CallCreateProxyObjectNode->FindPin(CurrentPin->PinName); // match function inputs, to pass data to function from CallFunction node
			bIsErrorFree &= DestPin && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
		}
	}

	UEdGraphPin* const ProxyObjectPin = CallCreateProxyObjectNode->GetReturnValuePin();
	check(ProxyObjectPin);
	UEdGraphPin* OutputAsyncTaskProxy = FindPin(FBaseAsyncTaskHelper::GetAsyncTaskProxyName());
	bIsErrorFree &= !OutputAsyncTaskProxy || CompilerContext.MovePinLinksToIntermediate(*OutputAsyncTaskProxy, *ProxyObjectPin).CanSafeConnect();

	bIsErrorFree &= ExpandDefaultToSelfPin(CompilerContext, SourceGraph, CallCreateProxyObjectNode);		

	// GATHER OUTPUT PARAMETERS AND PAIR THEM WITH LOCAL VARIABLES
	TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable> VariableOutputs;
	bool bPassedFactoryOutputs = false;
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if ((OutputAsyncTaskProxy != CurrentPin) && FBaseAsyncTaskHelper::ValidDataPin(CurrentPin, EGPD_Output))
		{
			if (!bPassedFactoryOutputs)
			{
				UEdGraphPin* DestPin = CallCreateProxyObjectNode->FindPin(CurrentPin->PinName);
				bIsErrorFree &= DestPin && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *DestPin).CanSafeConnect();
			}
			else
			{
				const FEdGraphPinType& PinType = CurrentPin->PinType;
				UK2Node_TemporaryVariable* TempVarOutput = CompilerContext.SpawnInternalVariable(
					this, PinType.PinCategory, PinType.PinSubCategory, PinType.PinSubCategoryObject.Get(), PinType.ContainerType, PinType.PinValueType);
				bIsErrorFree &= TempVarOutput->GetVariablePin() && CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *TempVarOutput->GetVariablePin()).CanSafeConnect();
				VariableOutputs.Add(FBaseAsyncTaskHelper::FOutputPinAndLocalVariable(CurrentPin, TempVarOutput));
			}
		}
		else if (!bPassedFactoryOutputs && CurrentPin && CurrentPin->Direction == EGPD_Output)
		{
			// the first exec that isn't the node's then pin is the start of the asyc delegate pins
			// once we hit this point, we've iterated beyond all outputs for the factory function
			bPassedFactoryOutputs = (CurrentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec) && (CurrentPin->PinName != UEdGraphSchema_K2::PN_Then);
		}
	}

	// FOR EACH DELEGATE DEFINE EVENT, CONNECT IT TO DELEGATE AND IMPLEMENT A CHAIN OF ASSIGMENTS
	UEdGraphPin* LastThenPin = CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

	UK2Node_CallFunction* IsValidFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	const FName IsValidFuncName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, IsValid);
	IsValidFuncNode->FunctionReference.SetExternalMember(IsValidFuncName, UKismetSystemLibrary::StaticClass());
	IsValidFuncNode->AllocateDefaultPins();
	UEdGraphPin* IsValidInputPin = IsValidFuncNode->FindPinChecked(TEXT("Object"));

	bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, IsValidInputPin);

	UK2Node_IfThenElse* ValidateProxyNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	ValidateProxyNode->AllocateDefaultPins();
	bIsErrorFree &= Schema->TryCreateConnection(IsValidFuncNode->GetReturnValuePin(), ValidateProxyNode->GetConditionPin());

	bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ValidateProxyNode->GetExecPin());
	LastThenPin = ValidateProxyNode->GetThenPin();

	bIsErrorFree &= HandleDelegates(VariableOutputs, ProxyObjectPin, LastThenPin, SourceGraph, CompilerContext);

	if (CallCreateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then) == LastThenPin)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingDelegateProperties", "BaseAsyncTask: Proxy has no delegates defined. @@").ToString(), this);
		return;
	}

	// Create a call to activate the proxy object if necessary
	if (ProxyActivateFunctionName != NAME_None)
	{
		UK2Node_CallFunction* const CallActivateProxyObjectNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		CallActivateProxyObjectNode->FunctionReference.SetExternalMember(ProxyActivateFunctionName, ProxyClass);
		CallActivateProxyObjectNode->AllocateDefaultPins();

		// Hook up the self connection
		UEdGraphPin* ActivateCallSelfPin = Schema->FindSelfPin(*CallActivateProxyObjectNode, EGPD_Input);
		check(ActivateCallSelfPin);

		bIsErrorFree &= Schema->TryCreateConnection(ProxyObjectPin, ActivateCallSelfPin);

		// Hook the activate node up in the exec chain
		UEdGraphPin* ActivateExecPin = CallActivateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute);
		UEdGraphPin* ActivateThenPin = CallActivateProxyObjectNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);

		bIsErrorFree &= Schema->TryCreateConnection(LastThenPin, ActivateExecPin);

		LastThenPin = ActivateThenPin;
	}

	// Move the connections from the original node then pin to the last internal then pin

	UEdGraphPin* OriginalThenPin = FindPin(UEdGraphSchema_K2::PN_Then);

	if (OriginalThenPin)
	{
		bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*OriginalThenPin, *LastThenPin).CanSafeConnect();
	}
	bIsErrorFree &= CompilerContext.CopyPinLinksToIntermediate(*LastThenPin, *ValidateProxyNode->GetElsePin()).CanSafeConnect();

	if (!bIsErrorFree)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InternalConnectionError", "BaseAsyncTask: Internal connection error. @@").ToString(), this);
	}

	// Make sure we caught everything
	BreakAllNodeLinks();
}

bool UK2Node_BaseAsyncTask::HandleDelegates(const TArray<FBaseAsyncTaskHelper::FOutputPinAndLocalVariable>& VariableOutputs, UEdGraphPin* ProxyObjectPin, UEdGraphPin*& InOutLastThenPin, UEdGraph* SourceGraph, FKismetCompilerContext& CompilerContext)
{
	bool bIsErrorFree = true;
	for (TFieldIterator<FMulticastDelegateProperty> PropertyIt(ProxyClass); PropertyIt && bIsErrorFree; ++PropertyIt)
	{
		UEdGraphPin* LastActivatedThenPin = nullptr;
		bIsErrorFree &= FBaseAsyncTaskHelper::HandleDelegateImplementation(*PropertyIt, VariableOutputs, ProxyObjectPin, InOutLastThenPin, LastActivatedThenPin, this, SourceGraph, CompilerContext);
	}
	return bIsErrorFree;
}

bool UK2Node_BaseAsyncTask::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	const bool bProxyFactoryResult = (ProxyFactoryClass != NULL) && (ProxyFactoryClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bProxyFactoryResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(ProxyFactoryClass);
	}

	const bool bProxyResult = (ProxyClass != NULL) && (ProxyClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bProxyResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(ProxyClass);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bProxyFactoryResult || bProxyResult || bSuperResult;
}

bool UK2Node_BaseAsyncTask::IsLatentForMacros() const
{
	return true;
}

FName UK2Node_BaseAsyncTask::GetCornerIcon() const
{
	return TEXT("Graph.Latent.LatentIcon");
}

FText UK2Node_BaseAsyncTask::GetToolTipHeading() const
{
	return LOCTEXT("LatentFunc", "Latent");
}

FText UK2Node_BaseAsyncTask::GetMenuCategory() const
{	
	UFunction* TargetFunction = GetFactoryFunction();
	return UK2Node_CallFunction::GetDefaultCategoryForFunction(TargetFunction, FText::GetEmpty());
}

void UK2Node_BaseAsyncTask::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

UFunction* UK2Node_BaseAsyncTask::GetFactoryFunction() const
{
	if (ProxyFactoryClass == nullptr)
	{
		UE_LOGF(LogBlueprint, Error, "ProxyFactoryClass null in %ls. Was a class deleted or saved on a non promoted build?", *GetFullName());
		return nullptr;
	}
	
	FMemberReference FunctionReference;
	FunctionReference.SetExternalMember(ProxyFactoryFunctionName, ProxyFactoryClass);

	UFunction* FactoryFunction = FunctionReference.ResolveMember<UFunction>(GetBlueprint());
	
	if (FactoryFunction == nullptr)
	{
		FactoryFunction = ProxyFactoryClass->FindFunctionByName(ProxyFactoryFunctionName);
		UE_CLOGF(FactoryFunction == nullptr, LogBlueprint, Error, "FactoryFunction %ls null in %ls. Was a class deleted or saved on a non promoted build?", *ProxyFactoryFunctionName.ToString(), *GetFullName());
	}

	return FactoryFunction;
}

void UK2Node_BaseAsyncTask::GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const
{
	TMap<FString, FStringFormatArg> Args;

	if (ProxyClass)
	{
		Args.Add(TEXT("ProxyClass"), ProxyClass->GetName());
		Args.Add(TEXT("ProxyClassSeparator"), TEXT("."));
	}
	else
	{
		Args.Add(TEXT("ProxyClass"), TEXT(""));
		Args.Add(TEXT("ProxyClassSeparator"), TEXT(""));
	}
	if (ProxyFactoryFunctionName != NAME_None)
	{
		Args.Add(TEXT("ProxyFactoryFunction"), ProxyFactoryFunctionName.ToString());
		Args.Add(TEXT("ProxyFactoryFunctionSeparator"), TEXT("."));
	}
	else
	{
		Args.Add(TEXT("ProxyFactoryFunction"), TEXT(""));
		Args.Add(TEXT("ProxyFactoryFunctionSeparator"), TEXT(""));
	}

	Args.Add(TEXT("PinName"), Pin.PinName.ToString());

	FString FullPinName;
	FullPinName = FString::Format(TEXT("{ProxyClass}{ProxyClassSeparator}{ProxyFactoryFunction}{ProxyFactoryFunctionSeparator}{PinName}"), Args);

	RedirectPinNames.Add(FullPinName);
}

void UK2Node_BaseAsyncTask::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if(UObject const* SourceObject = MessageLog.FindSourceObject(this))
	{
		// Lets check if it's a result of macro expansion, to give a helpful error
		if(UK2Node_MacroInstance const* MacroInstance = Cast<UK2Node_MacroInstance>(SourceObject))
		{
			// Since it's not possible to check the graph's type, just check if this is a ubergraph using the schema's name for it
			if(!(GetGraph()->HasAnyFlags(RF_Transient) && GetGraph()->GetName().StartsWith(UEdGraphSchema_K2::FN_ExecuteUbergraphBase.ToString())))
			{
				MessageLog.Error(*LOCTEXT("AsyncTaskInFunctionFromMacro", "@@ is being used in Function '@@' resulting from expansion of Macro '@@'").ToString(), this, GetGraph(), MacroInstance);
			}
		}
	}
}

TMap<FName, FAsyncTaskPinRedirectMapInfo> UK2Node_BaseAsyncTask::AsyncTaskPinRedirectMap;
bool UK2Node_BaseAsyncTask::bAsyncTaskPinRedirectMapInitialized = false;

UK2Node::ERedirectType UK2Node_BaseAsyncTask::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	if (GConfig && ProxyClass)
	{
		// Initialize remap table from INI
		if (!bAsyncTaskPinRedirectMapInitialized)
		{
			bAsyncTaskPinRedirectMapInitialized = true;
			const FConfigSection* PackageRedirects = GConfig->GetSection(TEXT("/Script/Engine.Engine"), false, GEngineIni);
			for (FConfigSection::TConstIterator It(*PackageRedirects); It; ++It)
			{
				if (It.Key() == TEXT("K2AsyncTaskPinRedirects"))
				{
					FString ProxyClassString;
					FString OldPinString;
					FString NewPinString;

					FParse::Value(*It.Value().GetValue(), TEXT("ProxyClassName="), ProxyClassString);
					FParse::Value(*It.Value().GetValue(), TEXT("OldPinName="), OldPinString);
					FParse::Value(*It.Value().GetValue(), TEXT("NewPinName="), NewPinString);

					UClass* RedirectProxyClass = UClass::TryFindTypeSlow<UClass>(ProxyClassString);
					if (RedirectProxyClass)
					{
						FAsyncTaskPinRedirectMapInfo& PinRedirectInfo = AsyncTaskPinRedirectMap.FindOrAdd(*OldPinString);
						TArray<UClass*>& ProxyClassArray = PinRedirectInfo.OldPinToProxyClassMap.FindOrAdd(*NewPinString);
						ProxyClassArray.AddUnique(RedirectProxyClass);
					}
				}
			}
		}

		// See if these pins need to be remapped.
		if (FAsyncTaskPinRedirectMapInfo* PinRedirectInfo = AsyncTaskPinRedirectMap.Find(OldPin->PinName))
		{
			if (TArray<UClass*>* ProxyClassArray = PinRedirectInfo->OldPinToProxyClassMap.Find(NewPin->PinName))
			{
				for (UClass* RedirectedProxyClass : *ProxyClassArray)
				{
					if (ProxyClass->IsChildOf(RedirectedProxyClass))
					{
						return UK2Node::ERedirectType_Name;
					}
				}
			}
		}
	}

	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

void UK2Node_BaseAsyncTask::GeneratePinTooltip(UEdGraphPin& Pin) const
{
	ensure(Pin.GetOwningNode() == this);

	UEdGraphSchema const* Schema = GetSchema();
	check(Schema);
	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(Schema);

	if (K2Schema == nullptr)
	{
		Schema->ConstructBasicPinTooltip(Pin, FText::GetEmpty(), Pin.PinToolTip);
		return;
	}

	// get the class function object associated with this node
	// Slight change from UK2Node_CallFunction (where this code is copied from)
	// We're getting the Factory function instead of GetTargetFunction
	UFunction* Function = GetFactoryFunction(); 
	if (Function == nullptr)
	{
		Schema->ConstructBasicPinTooltip(Pin, FText::GetEmpty(), Pin.PinToolTip);
		return;
	}

	UK2Node_CallFunction::GeneratePinTooltipFromFunction(Pin, Function);
}

void UK2Node_BaseAsyncTask::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	if (!bPinTooltipsValid)
	{
		for (UEdGraphPin* P : Pins)
		{
			if (P->Direction == EGPD_Input)
			{
				P->PinToolTip.Reset();
				GeneratePinTooltip(*P);
			}
		}

		bPinTooltipsValid = true;
	}

	return UK2Node::GetPinHoverText(Pin, HoverTextOut);
}

FString UK2Node_BaseAsyncTask::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData = Super::GetPinMetaData(InPinName, InKey);

	// If there's no metadata directly on the pin then check for metadata on the function
	if (MetaData.IsEmpty())
	{
		if (UFunction* Function = GetFactoryFunction())
		{
			// Find the corresponding property for the pin and search that first
			if (FProperty* Property = Function->FindPropertyByName(InPinName))
			{
				MetaData = Property->GetMetaData(InKey);
			}

			// Also look for metadata like DefaultToSelf on the function itself
			if (MetaData.IsEmpty())
			{
				MetaData = Function->GetMetaData(InKey);
				if (MetaData != InPinName.ToString())
				{
					// Only return if the value matches the pin name as we don't want general function metadata
					MetaData.Empty();
				}
			}
		}
	}

	return MetaData;
}

#undef LOCTEXT_NAMESPACE
