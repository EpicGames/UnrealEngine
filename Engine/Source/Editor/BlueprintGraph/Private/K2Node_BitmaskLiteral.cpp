// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_BitmaskLiteral.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetSystemLibrary.h"
#include "KismetCompiler.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Serialization/Archive.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_BitmaskLiteral)

#define LOCTEXT_NAMESPACE "UK2Node_BitmaskLiteral"

const FName& UK2Node_BitmaskLiteral::GetBitmaskInputPinName()
{
	static const FName PinName(TEXT("Bitmask"));
	return PinName;
}

UK2Node_BitmaskLiteral::UK2Node_BitmaskLiteral(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, BitflagsEnum(nullptr)
{
	
}

void UK2Node_BitmaskLiteral::ValidateBitflagsEnumType()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if (BitflagsEnum != nullptr)
	{
		// Reset enum type reference if it no longer has the proper meta data.
		const FString BitflagsMetaDataKey = FBlueprintMetadata::MD_Bitflags.ToString();
		if (!IsValid(BitflagsEnum) || !BitflagsEnum->HasMetaData(*BitflagsMetaDataKey))
		{
			// Note: The input pin's default value is intentionally not reset here. Losing an associated enum type means the node will now expose the max
			// number of bitflags, so this will ensure that we preserve the previous default value when the enum type representing the bitflags is removed.
			BitflagsEnum = nullptr;
		}
		else
		{
			// Adjust the default value in case an entry was added or removed, or in case the enum/value mode was changed.
			int32 ValidBitflagsMask = 0;
			const int32 BitmaskBitCount = sizeof(int32) << 3;
			UEdGraphPin* InputPin = FindPinChecked(GetBitmaskInputPinName());
			int32 OldDefaultValue = FCString::Atoi(*InputPin->DefaultValue);
			const bool bUseEnumValuesAsMaskValues = BitflagsEnum->GetBoolMetaData(FBlueprintMetadata::MD_UseEnumValuesAsMaskValuesInEditor);

			// Note: This loop is not inclusive of (BitflagsEnum->NumEnums() - 1) in order to exclude the implicit "MAX" value that gets added to the enum type at compile time.
			for (int32 BitflagsEnumIndex = 0; BitflagsEnumIndex < BitflagsEnum->NumEnums() - 1; ++BitflagsEnumIndex)
			{
				const int64 EnumValue = BitflagsEnum->GetValueByIndex(BitflagsEnumIndex);
				if (EnumValue >= 0)
				{
					if (bUseEnumValuesAsMaskValues)
					{
						if (EnumValue < MAX_int32 && FMath::IsPowerOfTwo(EnumValue))
						{
							ValidBitflagsMask |= EnumValue;
						}
					}
					else if (EnumValue < BitmaskBitCount)
					{
						ValidBitflagsMask |= 1 << static_cast<int32>(EnumValue);
					}
				}
			}

			const int32 NewDefaultValue = OldDefaultValue & ValidBitflagsMask;
			if (NewDefaultValue != OldDefaultValue)
			{
				Schema->SetPinAutogeneratedDefaultValue(InputPin, FString::FromInt(NewDefaultValue));
			}
		}
	}
}

void UK2Node_BitmaskLiteral::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Post-load validation of the enum type.
	if (Ar.IsLoading() && Ar.IsPersistent() && !Ar.HasAnyPortFlags(PPF_Duplicate | PPF_DuplicateForPIE))
	{
		// If valid, ensure that the enum type is loaded.
		if (BitflagsEnum != nullptr)
		{
			Ar.Preload(BitflagsEnum);
		}

		ValidateBitflagsEnumType();
	}
}

void UK2Node_BitmaskLiteral::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* InputPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, UEdGraphSchema_K2::PSC_Bitmask, BitflagsEnum, GetBitmaskInputPinName());
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(InputPin);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Int, UEdGraphSchema_K2::PSC_Bitmask, BitflagsEnum, UEdGraphSchema_K2::PN_ReturnValue);

	Super::AllocateDefaultPins();
}

void UK2Node_BitmaskLiteral::ReconstructNode()
{
	// Validate the enum type prior to reconstruction so that the input pin's default value is reset first (if needed).
	ValidateBitflagsEnumType();

	Super::ReconstructNode();
}

void UK2Node_BitmaskLiteral::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralInt);
	UK2Node_CallFunction* MakeLiteralInt = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	MakeLiteralInt->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(FunctionName));
	MakeLiteralInt->AllocateDefaultPins();

	UEdGraphPin* OrgInputPin = FindPinChecked(GetBitmaskInputPinName());
	UEdGraphPin* NewInputPin = MakeLiteralInt->FindPinChecked(TEXT("Value"));
	check(nullptr != NewInputPin);
	CompilerContext.MovePinLinksToIntermediate(*OrgInputPin, *NewInputPin);

	UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* NewReturnPin = MakeLiteralInt->GetReturnValuePin();
	check(nullptr != NewReturnPin);
	CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *NewReturnPin);
}

FText UK2Node_BitmaskLiteral::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Make Bitmask");
}

void UK2Node_BitmaskLiteral::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_BitmaskLiteral::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Math);
}

void UK2Node_BitmaskLiteral::ReloadEnum(class UEnum* InEnum)
{
	BitflagsEnum = InEnum;
}

#undef LOCTEXT_NAMESPACE
