// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_BreakStruct.h"

#include "BPTerminal.h"
#include "BlueprintEditorSettings.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "StructUtils/UserDefinedStruct.h"
#include "EngineLogs.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_StructOperation.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "Serialization/Archive.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_BreakStruct)

#define LOCTEXT_NAMESPACE "K2Node_BreakStruct"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_BreakStruct

class FKCHandler_BreakStruct : public FNodeHandlingFunctor
{
public:
	FKCHandler_BreakStruct(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	FBPTerminal* RegisterInputTerm(FKismetFunctionContext& Context, UK2Node_BreakStruct* Node)
	{
		check(NULL != Node);

		if(NULL == Node->StructType)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("BreakStruct_UnknownStructure_Error", "Unknown structure to break for @@").ToString(), Node);
			return NULL;
		}

		//Find input pin
		UEdGraphPin* InputPin = NULL;
		for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Node->Pins[PinIndex];
			if(Pin && (EGPD_Input == Pin->Direction))
			{
				InputPin = Pin;
				break;
			}
		}
		check(NULL != InputPin);

		//Find structure source net
		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(InputPin);
		check(NULL != Net);

		FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
		FBPTerminal* Term = FoundTerm ? *FoundTerm : NULL;
		if(NULL == Term)
		{
			// Dont allow literal
			if ((Net->Direction == EGPD_Input) && (Net->LinkedTo.Num() == 0))
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("InvalidNoInputStructure_Error", "No input structure to break for @@").ToString(), Net);
				return NULL;
			}
			// standard register net
			else
			{
				Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
			}
			Context.NetMap.Add(Net, Term);
		}
		UStruct* StructInTerm = Cast<UStruct>(Term->Type.PinSubCategoryObject.Get());
		if(NULL == StructInTerm || !StructInTerm->IsChildOf(Node->StructType))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("BreakStruct_NoMatch_Error", "Structures don't match for @@").ToString(), Node);
		}

		return Term;
	}

	void RegisterOutputTerm(FKismetFunctionContext& Context, UScriptStruct* StructType, UEdGraphPin* Net, FBPTerminal* ContextTerm)
	{
		if (FProperty* BoundProperty = FindFProperty<FProperty>(StructType, Net->PinName))
		{
			if (BoundProperty->HasAnyPropertyFlags(CPF_Deprecated) && Net->LinkedTo.Num())
			{
				FText Message = FText::Format(LOCTEXT("BreakStruct_DeprecatedField_Warning", "@@ : Member '{0}' of struct '{1}' is deprecated.")
					, BoundProperty->GetDisplayNameText()
					, StructType->GetDisplayNameText());
				CompilerContext.MessageLog.Warning(*Message.ToString(), Net->GetOuter());
			}

			UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
			FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Net->PinName.ToString());
			Term->bPassedByReference = ContextTerm->bPassedByReference;
			Term->AssociatedVarProperty = BoundProperty;
			Context.NetMap.Add(Net, Term);
			Term->Context = ContextTerm;

			if (BoundProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				Term->bIsConst = true;
			}
		}
		else
		{
			CompilerContext.MessageLog.Error(TEXT("Failed to find a struct member for @@"), Net);
		}
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override
	{
		UK2Node_BreakStruct* Node = Cast<UK2Node_BreakStruct>(InNode);
		check(Node);

		if(!UK2Node_BreakStruct::CanBeBroken(Node->StructType, Node->IsIntermediateNode()))
		{
			CompilerContext.MessageLog.Warning(*LOCTEXT("BreakStruct_NoBreak_Error", "The structure cannot be broken using generic 'break' node @@. Try use specialized 'break' function if available.").ToString(), Node);
		}

		if(FBPTerminal* StructContextTerm = RegisterInputTerm(Context, Node))
		{
			for (UEdGraphPin* Pin : Node->Pins)
			{
				if(Pin && EGPD_Output == Pin->Direction)
				{
					RegisterOutputTerm(Context, Node->StructType, Pin, StructContextTerm);
				}
			}
		}
	}
};


UK2Node_BreakStruct::UK2Node_BreakStruct(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bMadeAfterOverridePinRemoval(false)
{
}

static bool CanCreatePinForProperty(const FProperty* Property)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	FEdGraphPinType DumbGraphPinType;
	const bool bConvertable = Schema->ConvertPropertyToPinType(Property, /*out*/ DumbGraphPinType);

	const bool bVisible = (Property && Property->HasAnyPropertyFlags(CPF_BlueprintVisible));
	return bVisible && bConvertable;
}

bool UK2Node_BreakStruct::CanBeBroken(const UScriptStruct* Struct, const bool bForInternalUse)
{
	if (Struct && !Struct->HasMetaData(FBlueprintMetadata::MD_NativeBreakFunction) && UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Struct, bForInternalUse))
	{
		for (TFieldIterator<FProperty> It(Struct); It; ++It)
		{
			if (CanCreatePinForProperty(*It))
			{
				return true;
			}
		}
	}
	return false;
}

void UK2Node_BreakStruct::AllocateDefaultPins()
{
	if (StructType)
	{
		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.bIsConst = true;
		PinParams.bIsReference = true;

		PreloadObject(StructType);
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, StructType, StructType->GetFName(), PinParams);
		
		struct FBreakStructPinManager : public FStructOperationOptionalPinManager
		{
			virtual bool CanTreatPropertyAsOptional(FProperty* TestProperty) const override
			{
				return CanCreatePinForProperty(TestProperty);
			}
		};

		{
			FBreakStructPinManager OptionalPinManager;
			OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructType);
			OptionalPinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Output, this);
		}

		// When struct has a lot of fields, mark their pins as advanced
		if(Pins.Num() > 5)
		{
			if(ENodeAdvancedPins::NoPins == AdvancedPinDisplay)
			{
				AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
			}

			for(int32 PinIndex = 3; PinIndex < Pins.Num(); ++PinIndex)
			{
				if(UEdGraphPin * EdGraphPin = Pins[PinIndex])
				{
					EdGraphPin->bAdvancedView = true;
				}
			}
		}
	}
}

void UK2Node_BreakStruct::PreloadRequiredAssets()
{
	PreloadObject(StructType);
	Super::PreloadRequiredAssets();
}

FText UK2Node_BreakStruct::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (StructType == nullptr)
	{
		return LOCTEXT("BreakNullStruct_Title", "Break <unknown struct>");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("StructName"), StructType->GetDisplayNameText());

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("BreakNodeTitle", "Break {StructName}"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_BreakStruct::GetTooltipText() const
{
	if (StructType == nullptr)
	{
		return LOCTEXT("BreakNullStruct_Tooltip", "Adds a node that breaks an '<unknown struct>' into its member fields");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(
			LOCTEXT("BreakStruct_Tooltip", "Adds a node that breaks a '{0}' into its member fields"),
			StructType->GetDisplayNameText()
		), this);
	}
	return CachedTooltip;
}

void UK2Node_BreakStruct::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog); 

	if(!StructType)
	{
		MessageLog.Error(*LOCTEXT("NoStruct_Error", "No Struct in @@").ToString(), this);
	}
	else
	{
		bool bHasAnyBlueprintVisibleProperty = false;
		for (TFieldIterator<FProperty> It(StructType); It; ++It)
		{
			const FProperty* Property = *It;
			if (CanCreatePinForProperty(Property))
			{
				const bool bIsBlueprintVisible = Property->HasAnyPropertyFlags(CPF_BlueprintVisible) || (Property->GetOwnerStruct() && Property->GetOwnerStruct()->IsA<UUserDefinedStruct>());
				bHasAnyBlueprintVisibleProperty |= bIsBlueprintVisible;

				const UEdGraphPin* Pin = FindPin(Property->GetFName());
				const bool bIsLinked = Pin && Pin->LinkedTo.Num();

				if (!bIsBlueprintVisible && bIsLinked)
				{
					MessageLog.Warning(*LOCTEXT("PropertyIsNotBPVisible_Warning", "@@ - the native property is not tagged as BlueprintReadWrite or BlueprintReadOnly, the pin will be removed in a future release.").ToString(), Pin);
				}

				if ((Property->ArrayDim > 1) && bIsLinked)
				{
					MessageLog.Warning(*LOCTEXT("StaticArray_Warning", "@@ - the native property is a static array, which is not supported by blueprints").ToString(), Pin);
				}
			}
		}

		if (!bHasAnyBlueprintVisibleProperty)
		{
			MessageLog.Warning(*LOCTEXT("StructHasNoBPVisibleProperties_Warning", "@@ has no property tagged as BlueprintReadWrite or BlueprintReadOnly. The node will be removed in a future release.").ToString(), this);
		}

		if (!bMadeAfterOverridePinRemoval)
		{
			MessageLog.Warning(*NSLOCTEXT("K2Node", "OverridePinRemoval_BreakStruct", "Override pins have been removed from @@ in @@, please verify the Blueprint works as expected! See tooltips for enabling pin visibility for more details. This warning will go away after you resave the asset!").ToString(), this, GetBlueprint());
		}
	}
}

FSlateIcon UK2Node_BreakStruct::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.BreakStruct_16x");
	return Icon;
}

FLinearColor UK2Node_BreakStruct::GetNodeTitleColor() const 
{
	if(const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>())
	{
		FEdGraphPinType PinType;
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = StructType;
		return K2Schema->GetPinTypeColor(PinType);
	}
	return UK2Node::GetNodeTitleColor();
}

UK2Node::ERedirectType UK2Node_BreakStruct::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType Result = UK2Node::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if ((ERedirectType_None == Result) && DoRenamedPinsMatch(NewPin, OldPin, true))
	{
		Result = ERedirectType_Name;
	}

	return Result;
}

FNodeHandlingFunctor* UK2Node_BreakStruct::CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_BreakStruct(CompilerContext);
}

void UK2Node_BreakStruct::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::SetupMenuActions(ActionRegistrar, FMakeStructSpawnerAllowedDelegate::CreateStatic(&UK2Node_BreakStruct::CanBeBroken), EGPD_Output);
}

FText UK2Node_BreakStruct::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Struct);
}

void UK2Node_BreakStruct::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
	if (Blueprint && !Blueprint->bBeingCompiled)
	{
		bMadeAfterOverridePinRemoval = true;
	}
}

void UK2Node_BreakStruct::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// New nodes automatically have this set.
	bMadeAfterOverridePinRemoval = true;
}

void UK2Node_BreakStruct::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && !bMadeAfterOverridePinRemoval)
	{
		// Check if this node actually requires warning the user that functionality has changed.

		bMadeAfterOverridePinRemoval = true;
		FOptionalPinManager PinManager;

		// Have to check if this node is even in danger.
		for (TFieldIterator<FProperty> It(StructType, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			FProperty* TestProperty = *It;
			if (PinManager.CanTreatPropertyAsOptional(TestProperty))
			{
				bool bNegate = false;
				if (FProperty* OverrideProperty = PropertyCustomizationHelpers::GetEditConditionProperty(TestProperty, bNegate))
				{
					// We have confirmed that there is a property that uses an override variable to enable it, so set it to true.
					bMadeAfterOverridePinRemoval = false;
					break;
				}
			}
		}
	}
}

void UK2Node_BreakStruct::ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// User may have since deleted the struct type
	if (StructType == nullptr)
	{
		return;
	}

	// Check to see if the struct has a native make/break that we should try to convert to.
	if (StructType->HasMetaData(FBlueprintMetadata::MD_NativeBreakFunction))
	{
		UFunction* BreakNodeFunction = nullptr;

		// If any pins need to change their names during the conversion, add them to the map.
		TMap<FName, FName> OldPinToNewPinMap;

		if (StructType == TBaseStructure<FRotator>::Get())
		{
			BreakNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, BreakRotator));
			OldPinToNewPinMap.Add(TEXT("Rotator"), TEXT("InRot"));
		}
		else if (StructType == TBaseStructure<FVector>::Get())
		{
			BreakNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED_FourParams(UKismetMathLibrary, BreakVector, FVector, double&, double&, double&));
			OldPinToNewPinMap.Add(TEXT("Vector"), TEXT("InVec"));
		}
		else if (StructType == TBaseStructure<FVector2D>::Get())
		{
			BreakNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED_ThreeParams(UKismetMathLibrary, BreakVector2D, FVector2D, double&, double&));
			OldPinToNewPinMap.Add(TEXT("Vector2D"), TEXT("InVec"));
		}
		else
		{
			const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeBreakFunction);
			BreakNodeFunction = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);
			if (BreakNodeFunction)
			{
				// Look for the first parameter
				for (TFieldIterator<FProperty> FieldIterator(BreakNodeFunction); FieldIterator && (FieldIterator->PropertyFlags & CPF_Parm); ++FieldIterator)
				{
					if (FieldIterator->PropertyFlags & CPF_Parm && !(FieldIterator->PropertyFlags & CPF_ReturnParm))
					{
						OldPinToNewPinMap.Add(StructType->GetFName(), FieldIterator->GetFName());
						break;
					}
				}

				// If map is empty, didn't find parameter
				if (OldPinToNewPinMap.Num() == 0)
				{
					const UBlueprint* Blueprint = GetBlueprint();
					UE_LOGF(LogBlueprint, Warning, "BackwardCompatibilityNodeConversion Error 'cannot find input pin for break node function %ls in blueprint: %ls",
						*BreakNodeFunction->GetName(),
						Blueprint ? *Blueprint->GetName() : TEXT("Unknown"));

					BreakNodeFunction = nullptr;
				}
			}
		}

		if (BreakNodeFunction)
		{
			Schema->ConvertDeprecatedNodeToFunctionCall(this, BreakNodeFunction, OldPinToNewPinMap, Graph);
		}
	}
}

#undef LOCTEXT_NAMESPACE
