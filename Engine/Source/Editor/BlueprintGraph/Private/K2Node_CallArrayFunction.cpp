// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CallArrayFunction.h"

#include "Algo/Transform.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "K2Node_GetArrayItem.h"
#include "Kismet/KismetArrayLibrary.h" // for Array_Get()
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallArrayFunction)

UK2Node_CallArrayFunction::UK2Node_CallArrayFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_CallArrayFunction::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* TargetArrayPin = GetTargetArrayPin();
	if (ensureMsgf(TargetArrayPin, TEXT("%s"), *GetFullName()))
	{
		TargetArrayPin->PinType.ContainerType = EPinContainerType::Array;
		TargetArrayPin->PinType.bIsReference = true;
		TargetArrayPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
		TargetArrayPin->PinType.PinSubCategory = NAME_None;
		TargetArrayPin->PinType.PinSubCategoryObject = nullptr;
	}

	TArray< FArrayPropertyPinCombo > ArrayPins;
	GetArrayPins(ArrayPins);
	for(auto Iter = ArrayPins.CreateConstIterator(); Iter; ++Iter)
	{
		if(Iter->ArrayPropPin)
		{
			Iter->ArrayPropPin->bHidden = true;
			Iter->ArrayPropPin->bNotConnectable = true;
			Iter->ArrayPropPin->bDefaultValueIsReadOnly = true;
		}
	}

	PropagateArrayTypeInfo(TargetArrayPin);
}

void UK2Node_CallArrayFunction::PostReconstructNode()
{
	// cannot use a ranged for here, as PinConnectionListChanged() might end up 
	// collapsing split pins (subtracting elements from Pins)
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		if (Pin->LinkedTo.Num() > 0)
		{
			PinConnectionListChanged(Pin);
		}
	}

	Super::PostReconstructNode();
}

void UK2Node_CallArrayFunction::NotifyPinConnectionListChanged(UEdGraphPin* ChangedPin)
{
	Super::NotifyPinConnectionListChanged(ChangedPin);

	TArray<UEdGraphPin*> PinsToCheck;
	GetArrayTypeDependentPins(PinsToCheck);

	for (int32 Index = 0; Index < PinsToCheck.Num(); ++Index)
	{
		UEdGraphPin* PinToCheck = PinsToCheck[Index];
		if (PinToCheck->SubPins.Num() > 0)
		{
			PinsToCheck.Append(PinToCheck->SubPins);
		}
	}

	UEdGraphPin* TargetArray = GetTargetArrayPin();

	if (PinsToCheck.Contains(ChangedPin))
	{
		bool bNeedToPropagate = false;

		if(ChangedPin->LinkedTo.Num() > 0)
		{
			UEdGraphPin* LinkedTo = ChangedPin->LinkedTo[0];
			check(LinkedTo);

			// If the pin being changed is a WildCard or the array input itself, then we must propagate changes
			if (ChangedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard || (ChangedPin == TargetArray && LinkedTo->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard))
			{
				ChangedPin->PinType.PinCategory = LinkedTo->PinType.PinCategory;
				ChangedPin->PinType.PinSubCategory = LinkedTo->PinType.PinSubCategory;
				ChangedPin->PinType.PinSubCategoryObject = LinkedTo->PinType.PinSubCategoryObject;

				bNeedToPropagate = true;
			}
		}
		else
		{
			bNeedToPropagate = true;

			for (UEdGraphPin* PinToCheck : PinsToCheck)
			{
				if (PinToCheck->LinkedTo.Num() > 0)
				{
					bNeedToPropagate = false;
					break;
				}
			}

			if (bNeedToPropagate)
			{
				ChangedPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				ChangedPin->PinType.PinSubCategory = NAME_None;
				ChangedPin->PinType.PinSubCategoryObject = nullptr;
			}
		}

		if (bNeedToPropagate)
		{
			PropagateArrayTypeInfo(ChangedPin);
			GetGraph()->NotifyGraphChanged();
		}
	}
}

void UK2Node_CallArrayFunction::ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges)
{
	if (GetLinkerCustomVersion(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::ArrayGetFuncsReplacedByCustomNode)
	{
		if (FunctionReference.GetMemberParentClass() == UKismetArrayLibrary::StaticClass() && 
			FunctionReference.GetMemberName() == GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get))
		{
			UBlueprintNodeSpawner::FCustomizeNodeDelegate CustomizeToReturnByVal = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(
				[](UEdGraphNode* NewNode, bool /*bIsTemplateNode*/)
				{
					UK2Node_GetArrayItem* ArrayGetNode = CastChecked<UK2Node_GetArrayItem>(NewNode);
					ArrayGetNode->SetDesiredReturnType(/*bAsReference =*/false);
				}
			);
			UBlueprintNodeSpawner* GetItemNodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_GetArrayItem::StaticClass(), /*Outer =*/nullptr, CustomizeToReturnByVal);

			FVector2D NodePos(NodePosX, NodePosY);
			IBlueprintNodeBinder::FBindingSet Bindings;
			UK2Node_GetArrayItem* GetItemNode = Cast<UK2Node_GetArrayItem>(GetItemNodeSpawner->Invoke(Graph, Bindings, NodePos));

			const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
			if (K2Schema && GetItemNode)
			{
				TMap<FName, FName> OldToNewPinMap;
				for (UEdGraphPin* Pin : Pins)
				{
					if (Pin->ParentPin != nullptr)
					{
						// ReplaceOldNodeWithNew() will take care of mapping split pins (as long as the parents are properly mapped)
						continue;
					}
					else if (Pin->PinName == UEdGraphSchema_K2::PN_Self)
					{
						// there's no analogous pin, signal that we're expecting this
						OldToNewPinMap.Add(Pin->PinName, NAME_None);
					}
					else if (Pin->PinType.IsArray())
					{
						OldToNewPinMap.Add(Pin->PinName, GetItemNode->GetTargetArrayPin()->PinName);
					}
					else if (Pin->Direction == EGPD_Output)
					{
						OldToNewPinMap.Add(Pin->PinName, GetItemNode->GetResultPin()->PinName);
					}
					else
					{
						OldToNewPinMap.Add(Pin->PinName, GetItemNode->GetIndexPin()->PinName);
					}
				}
				K2Schema->ReplaceOldNodeWithNew(this, GetItemNode, OldToNewPinMap);
			}
		}
	}
}

UEdGraphPin* UK2Node_CallArrayFunction::GetTargetArrayPin() const
{
	TArray< FArrayPropertyPinCombo > ArrayParmPins;

	GetArrayPins(ArrayParmPins);

	if(ArrayParmPins.Num())
	{
		return ArrayParmPins[0].ArrayPin;
	}
	return nullptr;
}

void UK2Node_CallArrayFunction::GetArrayPins(TArray< FArrayPropertyPinCombo >& OutArrayPinInfo ) const
{
	OutArrayPinInfo.Empty();

	UFunction* TargetFunction = GetTargetFunction();
	if (ensure(TargetFunction))
	{
		const FString& ArrayPointerMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_ArrayParam);
		TArray<FString> ArrayPinComboNames;
		ArrayPointerMetaData.ParseIntoArray(ArrayPinComboNames, TEXT(","), true);

		for (auto Iter = ArrayPinComboNames.CreateConstIterator(); Iter; ++Iter)
		{
			TArray<FString> ArrayPinNames;
			Iter->ParseIntoArray(ArrayPinNames, TEXT("|"), true);

			FArrayPropertyPinCombo ArrayInfo;
			ArrayInfo.ArrayPin = FindPin(ArrayPinNames[0]);
			if (ArrayPinNames.Num() > 1)
			{
				ArrayInfo.ArrayPropPin = FindPin(ArrayPinNames[1]);
			}

			if (ArrayInfo.ArrayPin)
			{
				OutArrayPinInfo.Add(ArrayInfo);
			}
		}
	}
}

bool UK2Node_CallArrayFunction::IsWildcardProperty(UFunction* InArrayFunction, const FProperty* InProperty)
{
	if(InArrayFunction && InProperty)
	{
		const FString& ArrayPointerMetaData = InArrayFunction->GetMetaData(FBlueprintMetadata::MD_ArrayParam);
		if (!ArrayPointerMetaData.IsEmpty())
		{
			TArray<FString> ArrayPinComboNames;
			ArrayPointerMetaData.ParseIntoArray(ArrayPinComboNames, TEXT(","), true);

			for (auto Iter = ArrayPinComboNames.CreateConstIterator(); Iter; ++Iter)
			{
				TArray<FString> ArrayPinNames;
				Iter->ParseIntoArray(ArrayPinNames, TEXT("|"), true);

				if (ArrayPinNames[0] == InProperty->GetName())
				{
					return true;
				}
			}
		}
	}
	return false;
}

void UK2Node_CallArrayFunction::GetArrayTypeDependentPins(TArray<UEdGraphPin*>& OutPins) const
{
	OutPins.Empty();

	UFunction* TargetFunction = GetTargetFunction();
	if (ensure(TargetFunction))
	{
		const FString& DependentPinMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_ArrayDependentParam);
		TArray<FString> TypeDependentPinNameStrs;
		DependentPinMetaData.ParseIntoArray(TypeDependentPinNameStrs, TEXT(","), true);

		TArray<FName> TypeDependentPinNames;
		Algo::Transform(TypeDependentPinNameStrs, TypeDependentPinNames, [](const FString& PinNameStr) { return FName(*PinNameStr); });

		for (TArray<UEdGraphPin*>::TConstIterator it(Pins); it; ++it)
		{
			UEdGraphPin* CurrentPin = *it;
			int32 ItemIndex = 0;
			if (CurrentPin && TypeDependentPinNames.Find(CurrentPin->PinName, ItemIndex))
			{
				OutPins.Add(CurrentPin);
			}
		}
	}

	// Add the target array pin to make sure we get everything that depends on this type
	if(UEdGraphPin* TargetPin = GetTargetArrayPin())
	{
		OutPins.Add(TargetPin);
	}	
}

void UK2Node_CallArrayFunction::PropagateArrayTypeInfo(const UEdGraphPin* SourcePin)
{
	if( SourcePin )
	{
		const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
		const FEdGraphPinType& SourcePinType = SourcePin->PinType;

		TArray<UEdGraphPin*> DependentPins;
		GetArrayTypeDependentPins(DependentPins);
	
		// Propagate pin type info (except for array info!) to pins with dependent types
		for (UEdGraphPin* CurrentPin : DependentPins)
		{
			if (CurrentPin && CurrentPin != SourcePin)
			{
				FEdGraphPinType& CurrentPinType = CurrentPin->PinType;

				bool const bHasTypeMismatch = (CurrentPinType.PinCategory != SourcePinType.PinCategory) ||
					(CurrentPinType.PinSubCategory != SourcePinType.PinSubCategory) ||
					(CurrentPinType.PinSubCategoryObject != SourcePinType.PinSubCategoryObject);

				if (!bHasTypeMismatch)
				{
					continue;
				}

				if (CurrentPin->SubPins.Num() > 0)
				{
					Schema->RecombinePin(CurrentPin->SubPins[0]);
				}

				CurrentPinType.PinCategory          = SourcePinType.PinCategory;
				CurrentPinType.PinSubCategory       = SourcePinType.PinSubCategory;
				CurrentPinType.PinSubCategoryObject = SourcePinType.PinSubCategoryObject;

				// Reset default values
				if (!Schema->IsPinDefaultValid(CurrentPin, CurrentPin->DefaultValue, CurrentPin->DefaultObject, CurrentPin->DefaultTextValue).IsEmpty())
				{
					Schema->ResetPinToAutogeneratedDefaultValue(CurrentPin);
				}
			}
		}
	}
}
