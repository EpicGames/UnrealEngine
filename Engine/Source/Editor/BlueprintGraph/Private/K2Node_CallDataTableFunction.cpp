// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CallDataTableFunction.h"

#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallDataTableFunction)


UK2Node_CallDataTableFunction::UK2Node_CallDataTableFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_CallDataTableFunction::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	const FString& DataTablePinName = GetTargetFunction()->GetMetaData(FBlueprintMetadata::MD_DataTablePin);
	if (Pin->PinName.ToString() == DataTablePinName)
	{
		// When the DataTable pin gets a new value assigned, we need to update the Slate UI so that the RowName drop down gets updated
		GetGraph()->NotifyGraphChanged();
	}
}

void UK2Node_CallDataTableFunction::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	const FString& DataTablePinName = GetTargetFunction()->GetMetaData(FBlueprintMetadata::MD_DataTablePin);
	if (Pin->PinName.ToString() == DataTablePinName)
	{
		// When the DataTable pin gets a new connection assigned, we need to update the Slate UI so that the RowName drop down gets updated
		GetGraph()->NotifyGraphChanged();
	}
}

