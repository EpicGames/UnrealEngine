// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CallFunction.h"
#include "BlueprintCompilationManager.h"
#include "BlueprintEditorSettings.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/Interface.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GraphEditorSettings.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "FindInBlueprints.h"
#include "K2Node_Event.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_TemporaryVariable.h"
#include "K2Node_ExecutionSequence.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/WildcardNodeUtils.h"
#include "Settings/EditorStyleSettings.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "Preferences/UnrealEdOptions.h"
#include "UnrealEdGlobals.h"
#include "EdGraphUtilities.h"

#include "KismetCompiler.h"
#include "CallFunctionHandler.h"
#include "K2Node_SwitchEnum.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetArrayLibrary.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "K2Node_PureAssignmentStatement.h"
#include "BlueprintActionFilter.h"
#include "FindInBlueprintManager.h"
#include "ScopedTransaction.h"
#include "ObjectEditorUtils.h"
#include "SPinTypeSelector.h"
#include "SourceCodeNavigation.h"
#include "HAL/FileManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "BlueprintNodeStatics.h"
#include "ObjectTools.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "ToolMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallFunction)

#define LOCTEXT_NAMESPACE "K2Node"

namespace UE::K2NodeCallFunction::Private
{
	UEdGraphPin* FindBoolParamPin(const UK2Node_CallFunction& Node, FName ParameterName)
	{
		auto FindPin = [ParameterName](const UEdGraphPin* InPin)
		{
			check(InPin);
			const bool bPinMatches =
				(InPin->PinName == ParameterName) &&
				(InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
			return bPinMatches;
		};

		return Node.FindPinByPredicate(FindPin);
	}

	UEdGraphPin* FindEnumParamPin(const UK2Node_CallFunction& Node, FName ParameterName)
	{
		auto FindPin = [ParameterName](const UEdGraphPin* InPin)
		{
			check(InPin);
			const bool bPinMatches =
				(InPin->PinName == ParameterName) &&
				(InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte) &&
				(Cast<UEnum>(InPin->PinType.PinSubCategoryObject) != nullptr);
			return bPinMatches;
		};

		return Node.FindPinByPredicate(FindPin);
	}
}

/*******************************************************************************
 *  FCustomStructureParamHelper
 ******************************************************************************/

struct FCustomStructureParamHelper
{
	static void FillCustomStructureParameterNames(const UFunction* Function, TArray<FString>& OutNames)
	{
		OutNames.Reset();
		if (Function)
		{
			const FString& MetaDataValue = Function->GetMetaData(FBlueprintMetadata::MD_CustomStructureParam);
			if (!MetaDataValue.IsEmpty())
			{
				MetaDataValue.ParseIntoArray(OutNames, TEXT(","), true);
			}
		}
	}

	static void HandleSinglePin(UEdGraphPin* Pin)
	{
		if (Pin)
		{
			if (Pin->LinkedTo.Num() > 0)
			{
				UEdGraphPin* LinkedTo = Pin->LinkedTo[0];
				check(LinkedTo);

				if (UK2Node* Node = Cast<UK2Node>(Pin->GetOwningNode()))
				{
					ensure(
						!LinkedTo->PinType.IsContainer() ||
						Node->DoesWildcardPinAcceptContainer(Pin)
					);
				}
				else
				{
					ensure( !LinkedTo->PinType.IsContainer() );
				}

				Pin->PinType = LinkedTo->PinType;
			}
			else
			{
				// constness and refness are controlled by our declaration
				// but everything else needs to be reset to default wildcard:
				const bool bWasRef = Pin->PinType.bIsReference;
				const bool bWasConst = Pin->PinType.bIsConst;

				Pin->PinType = FEdGraphPinType();
				Pin->PinType.bIsReference = bWasRef;
				Pin->PinType.bIsConst = bWasConst;
				Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				Pin->PinType.PinSubCategory = NAME_None;
				Pin->PinType.PinSubCategoryObject = nullptr;
			}
		}
	}

	static void UpdateCustomStructurePins(const UFunction* Function, UK2Node* Node, UEdGraphPin* SinglePin = nullptr)
	{
		if (Function && Node)
		{
			TArray<FString> Names;
			FCustomStructureParamHelper::FillCustomStructureParameterNames(Function, Names);
			if (SinglePin)
			{
				if (Names.Contains(SinglePin->PinName.ToString()))
				{
					HandleSinglePin(SinglePin);
				}
			}
			else
			{
				for (const FString& Name : Names)
				{
					if (UEdGraphPin* Pin = Node->FindPin(Name))
					{
						HandleSinglePin(Pin);
					}
				}
			}
		}
	}
};

/*******************************************************************************
 *  FDynamicOutputUtils
 ******************************************************************************/

struct FDynamicOutputHelper
{
public:
	FDynamicOutputHelper(UEdGraphPin* InAlteredPin)
		: MutatingPin(InAlteredPin)
	{}

	/**
	 * Attempts to change the output pin's type so that it reflects the class 
	 * specified by the input class pin.
	 */
	void ConformOutputType() const;

	/**
	 * Retrieves the class pin that is used to determine the function's output type.
	 * 
	 * @return Null if the "DeterminesOutputType" metadata targets an invalid 
	 *         param (or if the metadata isn't present), otherwise a class picker pin.
	 */
	static UEdGraphPin* GetTypePickerPin(const UK2Node_CallFunction* FuncNode);

	/**
	 * Attempts to pull out class info from the supplied pin. Starts with the 
	 * pin's default, and then falls back onto the pin's native type. Will poll
	 * any connections that the pin may have.
	 * 
	 * @param  Pin	The pin you want a class from.
	 * @return A class that the pin represents (could be null if the pin isn't a class pin).
	 */
	static UClass* GetPinClass(UEdGraphPin* Pin);

	/**
	 * Intended to be used by ValidateNodeDuringCompilation(). Will check to 
	 * make sure the dynamic output's connections are still valid (they could
	 * become invalid as the the pin's type changes).
	 * 
	 * @param  FuncNode		The node you wish to validate.
	 * @param  MessageLog	The log to post errors/warnings to.
	 */
	static void VerifyNode(const UK2Node_CallFunction* FuncNode, FCompilerResultsLog& MessageLog);

private:
	/**
	 * 
	 * 
	 * @return 
	 */
	UK2Node_CallFunction* GetFunctionNode() const;

	/**
	 * 
	 * 
	 * @return 
	 */
	UFunction* GetTargetFunction() const;

	/**
	 * Checks if the supplied pin is the class picker that governs the 
	 * function's output type.
	 * 
	 * @param  Pin	The pin to test.
	 * @return True if the pin corresponds to the param that was flagged by the "DeterminesOutputType" metadata.
	 */
	bool IsTypePickerPin(UEdGraphPin* Pin) const;

	/**
	* Retrieves the object output pins that are altered as the class input is
	 * changed (favors params flagged by "DynamicOutputParam" metadata).
	 * 
	* @param FuncNode 		The function node in question
	* @param OutPins		Out array of pins that are flagged with "DynamicOutputParam" metadata
	 */
	static void GetDynamicOutPins(const UK2Node_CallFunction* FuncNode, TArray<UEdGraphPin*>& OutPins);

	/**
	 * Checks if the specified type is an object type that reflects the picker 
	 * pin's class.
	 * 
	 * @param  TypeToTest	The type you want to check.
	 * @return True if the type is likely the output governed by the class picker pin, otherwise false.
	 */
	static bool CanConformPinType(const UK2Node_CallFunction* FuncNode, const FEdGraphPinType& TypeToTest);

private:
	UEdGraphPin* MutatingPin;
};

void FDynamicOutputHelper::ConformOutputType() const
{
	if (IsTypePickerPin(MutatingPin))
	{
		UClass* PickedClass = GetPinClass(MutatingPin);
		UK2Node_CallFunction* FuncNode = GetFunctionNode();

		// See if there is any dynamic output pins
		TArray<UEdGraphPin*> DynamicPins;
		GetDynamicOutPins(FuncNode, DynamicPins);
		
		// Set the pins class
		for (UEdGraphPin* Pin : DynamicPins)
		{
			if (ensure(Pin != nullptr))
		{
				Pin->PinType.PinSubCategoryObject = PickedClass;
			}
		}
	}
}

UEdGraphPin* FDynamicOutputHelper::GetTypePickerPin(const UK2Node_CallFunction* FuncNode)
{
	UEdGraphPin* TypePickerPin = nullptr;

	if (UFunction* Function = FuncNode->GetTargetFunction())
	{
		const FString& TypeDeterminingPinName = Function->GetMetaData(FBlueprintMetadata::MD_DynamicOutputType);
		if (!TypeDeterminingPinName.IsEmpty())
		{
			TypePickerPin = FuncNode->FindPin(TypeDeterminingPinName);
		}
	}
	return TypePickerPin;
}

UClass* FDynamicOutputHelper::GetPinClass(UEdGraphPin* Pin)
{
	UClass* PinClass = UObject::StaticClass();

	bool const bIsClassOrObjectPin = (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object);
	if (bIsClassOrObjectPin)
	{
		if (UClass* DefaultClass = Cast<UClass>(Pin->DefaultObject))
		{
			PinClass = DefaultClass;
		}
		else if (UClass* BaseClass = Cast<UClass>(Pin->PinType.PinSubCategoryObject.Get()))
		{
			PinClass = BaseClass;
		}

		if (Pin->LinkedTo.Num() > 0)
		{
			UClass* CommonInputClass = nullptr;
			for (UEdGraphPin* LinkedPin : Pin->LinkedTo)
			{
				const FEdGraphPinType& LinkedPinType = LinkedPin->PinType;

				UClass* LinkClass = Cast<UClass>(LinkedPinType.PinSubCategoryObject.Get());
				if (LinkClass == nullptr && LinkedPinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self)
				{
					if (UK2Node* K2Node = Cast<UK2Node>(LinkedPin->GetOwningNode()))
					{
						LinkClass = K2Node->GetBlueprint()->GeneratedClass;
					}
				}

				if (LinkClass != nullptr)
				{
					if (CommonInputClass != nullptr)
					{
						while (!LinkClass->IsChildOf(CommonInputClass))
						{
							CommonInputClass = CommonInputClass->GetSuperClass();
						}
					}
					else
					{
						CommonInputClass = LinkClass;
					}
				}
			}

			PinClass = CommonInputClass;
		}
	}
	return PinClass;
}

void FDynamicOutputHelper::VerifyNode(const UK2Node_CallFunction* FuncNode, FCompilerResultsLog& MessageLog)
{
	TArray<UEdGraphPin*> DynamicPins;
	GetDynamicOutPins(FuncNode, DynamicPins);
	
	for (UEdGraphPin* DynamicOutPin : DynamicPins)
	{
		if (ensure(DynamicOutPin != nullptr))
	{
		const UEdGraphSchema* Schema = FuncNode->GetSchema();
		for (UEdGraphPin* Link : DynamicOutPin->LinkedTo)
		{
			if (Schema->CanCreateConnection(DynamicOutPin, Link).Response == CONNECT_RESPONSE_DISALLOW)
			{
				FText const ErrorFormat = LOCTEXT("BadConnection", "Invalid pin connection from '@@' to '@@'. You may have changed the type after the connections were made.");
				MessageLog.Error(*ErrorFormat.ToString(), DynamicOutPin, Link);
			}
		}
	}
	}
}

UK2Node_CallFunction* FDynamicOutputHelper::GetFunctionNode() const
{
	return CastChecked<UK2Node_CallFunction>(MutatingPin->GetOwningNode());
}

UFunction* FDynamicOutputHelper::GetTargetFunction() const
{
	return GetFunctionNode()->GetTargetFunction();
}

bool FDynamicOutputHelper::IsTypePickerPin(UEdGraphPin* Pin) const
{
	bool bIsTypeDeterminingPin = false;

	if (UFunction* Function = GetTargetFunction())
	{
		const FString& TypeDeterminingPinName = Function->GetMetaData(FBlueprintMetadata::MD_DynamicOutputType);
		if (!TypeDeterminingPinName.IsEmpty())
		{
			bIsTypeDeterminingPin = (Pin->PinName.ToString() == TypeDeterminingPinName);
		}
	}

	bool const bPinIsClassPicker = (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass);
	bool const bPinIsObjectPicker = (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject);

	return bIsTypeDeterminingPin && (bPinIsClassPicker || bPinIsObjectPicker) && (Pin->Direction == EGPD_Input);
}

void FDynamicOutputHelper::GetDynamicOutPins(const UK2Node_CallFunction* FuncNode, TArray<UEdGraphPin*>& OutPins)
{
	if (UFunction* Function = FuncNode->GetTargetFunction())
	{
		const FString& OutputPinNames = Function->GetMetaData(FBlueprintMetadata::MD_DynamicOutputParam);
		
		// There could be more than one dynamic output, so split by comma
		TArray<FString> UserDefinedDynamicProprties;
		OutputPinNames.ParseIntoArray(UserDefinedDynamicProprties, TEXT(","), true);

		// trim up the whitespace that a user may have added
		for (FString& Name : UserDefinedDynamicProprties)
		{
			Name = Name.TrimStartAndEnd();
		}

		// Get the schema so we can verify the pin if we find it
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// Lambda to add a pin to the out pins if it is valid
		auto AddPinToOutputLambda = [FuncNode, K2Schema](FProperty* TaggedOutputParam, TArray<UEdGraphPin*>& OutPins)
		{
			// Ensure that this is a valid pin to make dynamic
			FEdGraphPinType PropertyPinType;

			if (TaggedOutputParam && K2Schema->ConvertPropertyToPinType(TaggedOutputParam, /*out*/PropertyPinType) && CanConformPinType(FuncNode, PropertyPinType))
			{
				UEdGraphPin* DynamicOutPin = FuncNode->FindPin(TaggedOutputParam->GetFName());
				if (DynamicOutPin && (DynamicOutPin->Direction == EGPD_Output))
				{
					OutPins.Add(DynamicOutPin);
				}
			}
		};

		// we sort through properties, instead of pins, because the pin's type 
		// could already be modified to some other class (for when we check CanConformPinType)
		for (TFieldIterator<FProperty> ParamIt(Function); ParamIt && (ParamIt->PropertyFlags & CPF_Parm); ++ParamIt)
		{
			// If the user defined pins are 0 then assume we are just setting the type of a single output
			if (UserDefinedDynamicProprties.Num() == 0 && ParamIt->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				AddPinToOutputLambda(*ParamIt, OutPins);
			}
			else
			{
				// Check against each property that the user has specified
				for (const FString& OutputPinName : UserDefinedDynamicProprties)
				{
					// If this is the return parameter of this function or the pin name matches that which the user has specified
					if (OutputPinName == ParamIt->GetName())
					{
						AddPinToOutputLambda(*ParamIt, OutPins);
						break;
					}
				}
			}
		}
	}
}

bool FDynamicOutputHelper::CanConformPinType(const UK2Node_CallFunction* FuncNode, const FEdGraphPinType& TypeToTest)
{
	bool bIsProperType = false;
	if (UEdGraphPin* TypePickerPin = GetTypePickerPin(FuncNode))
	{
		UClass* BasePickerClass = CastChecked<UClass>(TypePickerPin->PinType.PinSubCategoryObject.Get());

		const FName PinCategory = TypeToTest.PinCategory;
		if ((PinCategory == UEdGraphSchema_K2::PC_Object) ||
			(PinCategory == UEdGraphSchema_K2::PC_Interface) ||
			(PinCategory == UEdGraphSchema_K2::PC_Class) ||
			(PinCategory == UEdGraphSchema_K2::PC_SoftObject) || 
			(PinCategory == UEdGraphSchema_K2::PC_SoftClass))
		{
			if (UClass* TypeClass = Cast<UClass>(TypeToTest.PinSubCategoryObject.Get()))
			{
				bIsProperType = BasePickerClass->IsChildOf(TypeClass);
			}
		}
	}
	return bIsProperType;
}

static bool WantsExecPinsForParams(const UFunction* TargetFunction)
{
	check(TargetFunction);
	return TargetFunction->HasMetaData(FBlueprintMetadata::MD_ExpandEnumAsExecs) ||
		TargetFunction->HasMetaData(FBlueprintMetadata::MD_ExpandBoolAsExecs);
}

static FString GetAllExecParams(const UFunction* TargetFunction)
{
	check(TargetFunction);
	FString Ret = TargetFunction->GetMetaData(FBlueprintMetadata::MD_ExpandEnumAsExecs);
	const FString& ExpandBools = TargetFunction->GetMetaData(FBlueprintMetadata::MD_ExpandBoolAsExecs);
	if (!ExpandBools.IsEmpty())
	{
		if(!Ret.IsEmpty())
		{
			Ret += ", " + ExpandBools;
		}
		else
		{
			Ret = ExpandBools;
		}
	}
	return Ret;
}

/*******************************************************************************
 *  UK2Node_CallFunction
 ******************************************************************************/

UK2Node_CallFunction::UK2Node_CallFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bPinTooltipsValid(false)
{
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAll;
}


bool UK2Node_CallFunction::HasDeprecatedReference() const
{
	if (UFunction* Function = GetTargetFunction())
	{
		bool bDeprecated = Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction);

		if (bDeprecated)
		{
			const UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
			const FString PathName = Function->GetPathName();

			if (BlueprintEditorProjectSettings->SuppressedDeprecationMessages.Contains(PathName))
			{
				bDeprecated = false;
			}
		}

		return bDeprecated;
	}

	return false;
}

FEdGraphNodeDeprecationResponse UK2Node_CallFunction::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		UFunction* Function = GetTargetFunction();
		if (ensureMsgf(Function != nullptr, TEXT("This node should not be able to report having a deprecated reference if the target function cannot be resolved.")))
		{
			// Check the deprecation type to override the severity
			FString MessageType = Function->GetMetaData(FBlueprintMetadata::MD_DeprecatedFunction);
			Response.MessageType = FBlueprintEditorUtils::GetDeprecatedMessageType(MessageType);

			FString DetailedMessage = Function->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
			Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(ObjectTools::GetUserFacingFunctionName(Function), FText::FromString(DetailedMessage));
		}
	}
	
	return Response;
}

FText UK2Node_CallFunction::GetFunctionContextString() const
{
	FText ContextString;

	// Don't show 'target is' if no target pin!
	UEdGraphPin* SelfPin = GetDefault<UEdGraphSchema_K2>()->FindSelfPin(*this, EGPD_Input);
	if(SelfPin != NULL && !SelfPin->bHidden)
	{
		const UFunction* Function = GetTargetFunction();
		UClass* CurrentSelfClass = (Function != NULL) ? Function->GetOwnerClass() : NULL;
		UClass const* TrueSelfClass = CurrentSelfClass;
		if (CurrentSelfClass && CurrentSelfClass->ClassGeneratedBy)
		{
			TrueSelfClass = CurrentSelfClass->GetAuthoritativeClass();
		}

		const FText TargetText = FBlueprintEditorUtils::GetFriendlyClassDisplayName(TrueSelfClass);

		FFormatNamedArguments Args;
		Args.Add(TEXT("TargetName"), TargetText);
		ContextString = FText::Format(LOCTEXT("CallFunctionOnDifferentContext", "Target is {TargetName}"), Args);
	}

	return ContextString;
}


FText UK2Node_CallFunction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText FunctionName;
	FText ContextString;
	FText RPCString;

	if (UFunction* Function = GetTargetFunction())
	{
		const bool bAllowFriendlyNames = (TitleType == ENodeTitleType::FullTitle);
		RPCString = UK2Node_Event::GetLocalizedNetString(Function->FunctionFlags, true);
		FunctionName = ObjectTools::GetUserFacingFunctionName(Function, bAllowFriendlyNames);
		ContextString = GetFunctionContextString();
	}
	else
	{
		FunctionName = FText::FromName(FunctionReference.GetMemberName());
		if ((GEditor != NULL) && (GetDefault<UEditorStyleSettings>()->bShowFriendlyNames))
		{
			FunctionName = FText::FromString(FName::NameToDisplayString(FunctionName.ToString(), false));
		}
	}

	if(TitleType == ENodeTitleType::FullTitle)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("FunctionName"), FunctionName);
		Args.Add(TEXT("ContextString"), ContextString);
		Args.Add(TEXT("RPCString"), RPCString);

		if (ContextString.IsEmpty() && RPCString.IsEmpty())
		{
			return FText::Format(LOCTEXT("CallFunction_FullTitle", "{FunctionName}"), Args);
		}
		else if (ContextString.IsEmpty())
		{
			return FText::Format(LOCTEXT("CallFunction_FullTitle_WithRPCString", "{FunctionName}\n{RPCString}"), Args);
		}
		else if (RPCString.IsEmpty())
		{
			return FText::Format(LOCTEXT("CallFunction_FullTitle_WithContextString", "{FunctionName}\n{ContextString}"), Args);
		}
		else
		{
			return FText::Format(LOCTEXT("CallFunction_FullTitle_WithContextRPCString", "{FunctionName}\n{ContextString}\n{RPCString}"), Args);
		}
	}
	else
	{
		return FunctionName;
	}
}

void UK2Node_CallFunction::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	if (!bPinTooltipsValid)
	{
		for (UEdGraphPin* P : Pins)
		{
			if (!P->PinToolTip.IsEmpty() && ExpandAsEnumPins.Contains(P))
			{
				continue;
			}

			P->PinToolTip.Reset();
			GeneratePinTooltip(*P);
		}

		bPinTooltipsValid = true;
	}

	return UK2Node::GetPinHoverText(Pin, HoverTextOut);
}

void UK2Node_CallFunction::AllocateDefaultPins()
{
	InvalidatePinTooltips();

	UBlueprint* MyBlueprint = GetBlueprint();
	
	UFunction* Function = GetTargetFunction();
	// favor the skeleton function if possible (in case the signature has 
	// changed, and not yet compiled).
	if (!FunctionReference.IsSelfContext())
	{
		UClass* FunctionClass = FunctionReference.GetMemberParentClass(MyBlueprint->GeneratedClass);
		if (UBlueprintGeneratedClass* BpClassOwner = Cast<UBlueprintGeneratedClass>(FunctionClass))
		{
			// this function could currently only be a part of some skeleton 
			// class (the blueprint has not be compiled with it yet), so let's 
			// check the skeleton class as well, see if we can pull pin data 
			// from there...
			UBlueprint* FunctionBlueprint = CastChecked<UBlueprint>(BpClassOwner->ClassGeneratedBy.Get(), ECastCheckedType::NullAllowed);
			if (FunctionBlueprint)
			{
				if (UFunction* SkelFunction = FindUField<UFunction>(FunctionBlueprint->SkeletonGeneratedClass, FunctionReference.GetMemberName()))
				{
					Function = SkelFunction;
				}
			}
		}
	}

	// First try remap table
	if (Function == NULL)
	{
		UClass* ParentClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());

		if (ParentClass != NULL)
		{
			if (UFunction* NewFunction = FMemberReference::FindRemappedField<UFunction>(ParentClass, FunctionReference.GetMemberName()))
			{
				// Found a remapped property, update the node
				Function = NewFunction;
				SetFromFunction(NewFunction);
			}
		}
	}

	if (Function == NULL)
	{
		// The function no longer exists in the stored scope
		// Try searching inside all function libraries, in case the function got refactored into one of them.
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* TestClass = *ClassIt;
			if (TestClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()))
			{
				Function = FindUField<UFunction>(TestClass, FunctionReference.GetMemberName());
				if (Function != NULL)
				{
					UClass* OldClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
					Message_Note(
						FText::Format(LOCTEXT("FixedUpFunctionInLibraryFmt", "UK2Node_CallFunction: Fixed up function '{0}', originally in '{1}', now in library '{2}'."),
							FText::FromString(FunctionReference.GetMemberName().ToString()),
							(OldClass != NULL) ? FText::FromString(*OldClass->GetName()) : LOCTEXT("FixedUpFunctionInLibraryNull", "(null)"),
							FText::FromString(TestClass->GetName())
						).ToString()
					);
					SetFromFunction(Function);
					break;
				}
			}
		}
	}

	// Now create the pins if we ended up with a valid function to call
	if (Function != NULL)
	{
		CreatePinsForFunctionCall(Function);
	}

	FCustomStructureParamHelper::UpdateCustomStructurePins(Function, this);

	Super::AllocateDefaultPins();
}

/** Util to find self pin in an array */
UEdGraphPin* FindSelfPin(TArray<UEdGraphPin*>& Pins)
{
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		if(Pins[PinIdx]->PinName == UEdGraphSchema_K2::PN_Self)
		{
			return Pins[PinIdx];
		}
	}
	return NULL;
}

void UK2Node_CallFunction::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	// BEGIN TEMP
	// We had a bug where the class was being messed up by copy/paste, but the self pin class was still ok. This code fixes up those cases.
	UFunction* Function = GetTargetFunction();
	if (Function == NULL)
	{
		if (UEdGraphPin* SelfPin = FindSelfPin(OldPins))
		{
			if (UClass* SelfPinClass = Cast<UClass>(SelfPin->PinType.PinSubCategoryObject.Get()))
			{
				if (UFunction* NewFunction = FindUField<UFunction>(SelfPinClass, FunctionReference.GetMemberName()))
				{
					SetFromFunction(NewFunction);
				}
			}
		}
	}
	// END TEMP

	const UBlueprint* Blueprint = GetBlueprint();
	if (Blueprint && Blueprint->bIsRegeneratingOnLoad)
	{
		// Older nodes incorrectly used an interface context for the target pin for calls to locally-implemented interface
		// functions (due to an earlier regression). This was compounded by occasional confusion in the context menu, where
		// a user might have chosen the wrong calling context for an interface implementation and then linked a term with an
		// incompatible object type to the target input. At runtime this worked fine because both the calling context as well
		// as the linked object context both implemented the target interface, which allowed any external object context to
		// be linked to the target pin, so long as it also implemented the interface. After the target pin context was fixed
		// to match the function context, the target pin context no longer matched the linked pin's context, so the connection
		// would otherwise be orphaned and result in a Blueprint compiler error. So, rather than cause confusion about nodes
		// that were not previously broken prior to the fix, we change the context to an interface for backwards compatibility.
		const UEdGraphPin* OldSelfPin = FindSelfPin(OldPins);
		if (OldSelfPin
			&& OldSelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface
			&& OldSelfPin->LinkedTo.Num() > 0)
		{
			if (const UFunction* TargetFunction = GetTargetFunction())
			{
				// Get the function context. This should match the target pin context, but due to the regression that's noted above,
				// in older assets this may not match the self pin if the context represents an implementation of an interface function.
				const UClass* FunctionContext = TargetFunction->GetOwnerClass()->GetAuthoritativeClass();

				// Get the interface context from the old target pin (this should already be non-NULL, but we'll check that below).
				const UClass* InterfaceContext = Cast<UClass>(OldSelfPin->PinType.PinSubCategoryObject);

				// If we're not already using an interface context, but the function's outer class implements the old target pin's type...
				if (ensure(FunctionContext)
					&& !FunctionContext->IsChildOf<UInterface>()
					&& ensure(InterfaceContext)
					&& FunctionContext->ImplementsInterface(InterfaceContext))
				{
					// Check for any linked object pins that aren't compatible with the current function context.
					for (const UEdGraphPin* LinkedTo : OldSelfPin->LinkedTo)
					{
						if (ensure(LinkedTo)
							&& LinkedTo->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
						{
							// If any linked object pin is not compatible with the current function context, but implements the old target pin's interface,
							// reset the function context to reference the interface method instead. This will change the function call to an interface call,
							// which will be compatible with any object context that's being passed in. When we reallocate the pins below, we'll reconstruct
							// the node using the proper function context in which all linked pins will remain backwards-compatible with the existing target.
							const UClass* LinkedToPinContext = Cast<UClass>(LinkedTo->PinType.PinSubCategoryObject);
							if (LinkedToPinContext
								&& !LinkedToPinContext->IsChildOf(FunctionContext)
								&& LinkedToPinContext->ImplementsInterface(InterfaceContext))
							{
								if (UFunction* InterfaceFunction = FindUField<UFunction>(InterfaceContext, FunctionReference.GetMemberName()))
								{
									SetFromFunction(InterfaceFunction);
									break;
								}
							}
						}
					}
				}
			}
		}
	}

	Super::ReallocatePinsDuringReconstruction(OldPins);

	// Connect Execute and Then pins for functions, which became pure.
	ReconnectPureExecPins(OldPins);
}

bool UK2Node_CallFunction::IsNodePure() const
{
	return !AreExecPinsVisible();
}

UEdGraphPin* UK2Node_CallFunction::CreateSelfPin(const UFunction* Function)
{
	return FBlueprintNodeStatics::CreateSelfPin(this, Function);
}

void UK2Node_CallFunction::CreateExecPinsForFunctionCall(const UFunction* Function)
{
	bool bCreateSingleExecInputPin = true;
	bool bCreateThenPin = true;
	
	ExpandAsEnumPins.Reset();

	// If not pure, create exec pins
	if (!IsNodePure())
	{
		// If we want enum->exec expansion, and it is not disabled, do it now
		if (bWantsEnumToExecExpansion)
		{
			TArray<FName> EnumNames;
			GetExpandEnumPinNames(Function, EnumNames);

			FProperty* PreviousInput = nullptr;

			for (const FName& EnumParamName : EnumNames)
			{
				FProperty* Prop = nullptr;
				UEnum* Enum = nullptr;

				if (FByteProperty* ByteProp = FindFProperty<FByteProperty>(Function, EnumParamName))
				{
					Prop = ByteProp;
					Enum = ByteProp->Enum;
				}
				else if (FEnumProperty* EnumProp = FindFProperty<FEnumProperty>(Function, EnumParamName))
				{
					Prop = EnumProp;
					Enum = EnumProp->GetEnum();
				}
				else if (FBoolProperty* BoolProp = FindFProperty<FBoolProperty>(Function, EnumParamName))
				{
					Prop = BoolProp;
				}

				if (Prop != nullptr)
				{
					const bool bIsFunctionInput = !Prop->HasAnyPropertyFlags(CPF_ReturnParm) &&
						(!Prop->HasAnyPropertyFlags(CPF_OutParm) ||
						Prop->HasAnyPropertyFlags(CPF_ReferenceParm));
					const EEdGraphPinDirection Direction = bIsFunctionInput ? EGPD_Input : EGPD_Output;

					if (bIsFunctionInput)
					{
						if (PreviousInput)
						{
							bHasCompilerMessage = true;
							ErrorType = EMessageSeverity::Error;
							ErrorMsg = FString::Printf(TEXT("Parameter '%s' is listed as an ExpandEnumAsExecs input, but %s already was one. Only one is permitted."), *EnumParamName.ToString(), *PreviousInput->GetName());
							break;
						}
						PreviousInput = Prop;
					}

					if (Enum)
					{
						// yay, found it! Now create exec pin for each
						int32 NumExecs = (Enum->NumEnums() - 1);
						for (int32 ExecIdx = 0; ExecIdx < NumExecs; ExecIdx++)
						{
							bool const bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), ExecIdx) || Enum->HasMetaData(TEXT("Spacer"), ExecIdx);
							if (!bShouldBeHidden)
							{
								// Can't use Enum->GetNameByIndex here because it doesn't do namespace mangling
								const FString NameStr = Enum->GetNameStringByIndex(ExecIdx);

								UEdGraphPin* CreatedPin = nullptr;

								// todo: really only makes sense if there are multiple outputs
								if (bIsFunctionInput || EnumNames.Num() == 1)
								{
									CreatedPin = CreatePin(Direction, UEdGraphSchema_K2::PC_Exec, *NameStr);
								}
								else
								{
									CreatedPin = CreatePin(Direction, UEdGraphSchema_K2::PC_Exec, *NameStr);
									CreatedPin->PinFriendlyName = FText::AsCultureInvariant(FString::Printf(TEXT("(%s) %s"), *Prop->GetDisplayNameText().ToString(), *NameStr));
								}

								ExpandAsEnumPins.Add(CreatedPin);

								if (Enum->HasMetaData(TEXT("Tooltip"), ExecIdx))
								{
									FString EnumTooltip = Enum->GetMetaData(TEXT("Tooltip"), ExecIdx);

									if (const UEdGraphSchema_K2* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema()))
									{
										K2Schema->ConstructBasicPinTooltip(*CreatedPin, FText::FromString(EnumTooltip), CreatedPin->PinToolTip);
									}
									else
									{
										CreatedPin->PinToolTip = EnumTooltip;
									}
								}
							}
						}
					}
					else
					{
						check(Prop->IsA<FBoolProperty>());
						// Create a pin for true and false, note that the order here does not match the
						// numeric order of bool, but it is more natural to put true first (e.g. to match branch node):
						ExpandAsEnumPins.Add(CreatePin(Direction, UEdGraphSchema_K2::PC_Exec, TEXT("True")));
						ExpandAsEnumPins.Add(CreatePin(Direction, UEdGraphSchema_K2::PC_Exec, TEXT("False")));
					}

					if (bIsFunctionInput)
					{
						// If using ExpandEnumAsExec for input, don't want to add a input exec pin
						bCreateSingleExecInputPin = false;
					}
					else
					{
						// If using ExpandEnumAsExec for output, don't want to add a "then" pin
						bCreateThenPin = false;
					}
				}
			}
		}
		
		if (bCreateSingleExecInputPin)
		{
			// Single input exec pin
			CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
		}

		if (bCreateThenPin)
		{
			UEdGraphPin* OutputExecPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
			// Use 'completed' name for output pins on latent functions
			if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
			{
				OutputExecPin->PinFriendlyName = FText::FromName(UEdGraphSchema_K2::PN_Completed);
			}
		}
	}
}

FName UK2Node_CallFunction::GetFunctionName() const
{
	return FunctionReference.GetMemberName();
}

void UK2Node_CallFunction::DetermineWantsEnumToExecExpansion(const UFunction* Function)
{
	bWantsEnumToExecExpansion = false;

	if (WantsExecPinsForParams(Function))
	{
		TArray<FName> EnumNamesToCheck;
		GetExpandEnumPinNames(Function, EnumNamesToCheck);

		for (int32 i = EnumNamesToCheck.Num() - 1; i >= 0; --i)
		{
			const FName& EnumParamName = EnumNamesToCheck[i];

			FByteProperty* EnumProp = FindFProperty<FByteProperty>(Function, EnumParamName);
			if ((EnumProp != NULL && EnumProp->Enum != NULL) || FindFProperty<FEnumProperty>(Function, EnumParamName))
			{
				bWantsEnumToExecExpansion = true;
				EnumNamesToCheck.RemoveAt(i);
			}
			else
			{
				FBoolProperty* BoolProp = FindFProperty<FBoolProperty>(Function, EnumParamName);
				if (BoolProp)
				{
					bWantsEnumToExecExpansion = true;
					EnumNamesToCheck.RemoveAt(i);
				}
			}
		}
		
		if (bWantsEnumToExecExpansion && EnumNamesToCheck.Num() > 0 && !bHasCompilerMessage)
		{
			bHasCompilerMessage = true;
			ErrorType = EMessageSeverity::Warning;

			if (EnumNamesToCheck.Num() == 1)
			{
				ErrorMsg = FText::Format(LOCTEXT("EnumToExecExpansionFailedFmt", "Unable to find enum parameter with name '{0}' to expand for @@"), FText::FromName(EnumNamesToCheck[0])).ToString();
			}
			else
			{
				FString ParamNames;

				for (const FName& Name : EnumNamesToCheck)
				{
					if (!ParamNames.IsEmpty())
					{
						ParamNames += TEXT(", ");
					}

					ParamNames += Name.ToString();
				}

				ErrorMsg = FText::Format(LOCTEXT("EnumToExecExpansionFailedMultipleFmt", "Unable to find enum parameters for names:\n '{{0}}' \nto expand for @@"), FText::FromString(ParamNames)).ToString();
			}
		}
	}
}

void UK2Node_CallFunction::GetExpandEnumPinNames(const UFunction* Function, TArray<FName>& EnumNamesToCheck)
{ 
	EnumNamesToCheck.Reset();

	// todo: use metadatacache if/when that's accepted.
	const FString EnumParamString = GetAllExecParams(Function);

	TArray<FString> RawGroupings;
	EnumParamString.ParseIntoArray(RawGroupings, TEXT(","), false);
	for (FString& RawGroup : RawGroupings)
	{
		RawGroup.TrimStartAndEndInline();

		TArray<FString> IndividualEntries;
		RawGroup.ParseIntoArray(IndividualEntries, TEXT("|"));

		for (const FString& Entry : IndividualEntries)
		{
			if (Entry.IsEmpty())
			{
				continue;
			}

			EnumNamesToCheck.Add(*Entry);
		}
	}
}

void UK2Node_CallFunction::GeneratePinTooltip(UEdGraphPin& Pin) const
{
	ensure(Pin.GetOwningNode() == this);

	UEdGraphSchema const* Schema = GetSchema();
	check(Schema != NULL);
	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(Schema);

	if (K2Schema == NULL)
	{
		Schema->ConstructBasicPinTooltip(Pin, FText::GetEmpty(), Pin.PinToolTip);
		return;
	}
	
	// get the class function object associated with this node
	UFunction* Function = GetTargetFunction();
	if (Function == NULL)
	{
		Schema->ConstructBasicPinTooltip(Pin, FText::GetEmpty(), Pin.PinToolTip);
		return;
	}


	GeneratePinTooltipFromFunction(Pin, Function);
}

bool UK2Node_CallFunction::CreatePinsForFunctionCall(const UFunction* Function)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UClass* FunctionOwnerClass = Function->GetOuterUClass();

	bDefaultsToPureFunc = Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
	DetermineWantsEnumToExecExpansion(Function);

	// Create input pins
	CreateExecPinsForFunctionCall(Function);

	UEdGraphPin* SelfPin = CreateSelfPin(Function);

	// Renamed self pin to target
	SelfPin->PinFriendlyName = LOCTEXT("Target", "Target");

	const bool bIsProtectedFunc = Function->GetBoolMetaData(FBlueprintMetadata::MD_Protected);
	const bool bIsStaticFunc = Function->HasAllFunctionFlags(FUNC_Static);

	const UEdGraph* const Graph = GetGraph();
	const UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);
	ensure(BP);
	if (BP != nullptr)
	{
		const bool bIsFunctionCompatibleWithSelf = BP->SkeletonGeneratedClass && BP->SkeletonGeneratedClass->IsChildOf(FunctionOwnerClass);

		if (bIsStaticFunc)
		{
			// For static methods, wire up the self to the CDO of the class if it's not us
			if (!bIsFunctionCompatibleWithSelf)
			{
				UClass* AuthoritativeClass = FunctionOwnerClass->GetAuthoritativeClass();
				SelfPin->DefaultObject = AuthoritativeClass->GetDefaultObject();
			}

			// Purity doesn't matter with a static function, we can always hide the self pin since we know how to call the method
			SelfPin->bHidden = true;
		}
		else
		{
			if (Function->GetBoolMetaData(FBlueprintMetadata::MD_HideSelfPin))
			{
				SelfPin->bHidden = true;
				SelfPin->bNotConnectable = true;
			}
			else
			{
				const bool bLocalIsConstFunc = (Function->HasAnyFunctionFlags(FUNC_Const) != false);

				// Non-const pure functions are generally compact looking, so we take advantage of that by hiding the self pin if we can.
				// In this case, if the function belongs to our current class, then "self" is implied.
				SelfPin->bHidden = (bIsFunctionCompatibleWithSelf && IsNodePure() && !bLocalIsConstFunc);
			}
		}
	}

	// Build a list of the pins that should be hidden for this function (ones that are automagically filled in by the K2 compiler)
	TSet<FName> PinsToHide;
	TSet<FName> InternalPins;
	FBlueprintEditorUtils::GetHiddenPinsForFunction(Graph, Function, PinsToHide, &InternalPins);

	const bool bShowWorldContextPin = ((PinsToHide.Num() > 0) && BP && BP->ParentClass && BP->ParentClass->HasMetaDataHierarchical(FBlueprintMetadata::MD_ShowWorldContextPin));

	// Create the inputs and outputs
	bool bAllPinsGood = true;
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Param = *PropIt;
		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_ReturnParm) && (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm));
		const bool bIsRefParam = Param->HasAnyPropertyFlags(CPF_ReferenceParm) && bIsFunctionInput;

		const EEdGraphPinDirection Direction = bIsFunctionInput ? EGPD_Input : EGPD_Output;

		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.bIsReference = bIsRefParam;
		UEdGraphPin* Pin = CreatePin(Direction, NAME_None, Param->GetFName(), PinParams);
		const bool bPinGood = (Pin && K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType));

		if (bPinGood)
		{
			// Check for a display name override
			const FString& PinDisplayName = Param->GetMetaData(FBlueprintMetadata::MD_DisplayName);
			if (!PinDisplayName.IsEmpty())
			{
				Pin->PinFriendlyName = FText::AsCultureInvariant(PinDisplayName);
			}
			else if (Function->GetReturnProperty() == Param && Function->HasMetaData(FBlueprintMetadata::MD_ReturnDisplayName))
			{
				Pin->PinFriendlyName = Function->GetMetaDataText(FBlueprintMetadata::MD_ReturnDisplayName);
			}

			//Flag pin as read only for const reference property
			Pin->bDefaultValueIsIgnored = Param->HasAllPropertyFlags(CPF_ConstParm | CPF_ReferenceParm) && (!Function->HasMetaData(FBlueprintMetadata::MD_AutoCreateRefTerm) || Pin->PinType.IsContainer());

			const bool bIsRequiredParam = Param->HasAnyPropertyFlags(CPF_RequiredParm);
			// Don't let the user edit the default value if the parameter is required to be explicit.
			Pin->bDefaultValueIsIgnored |= bIsRequiredParam;

			const bool bAdvancedPin = Param->HasAllPropertyFlags(CPF_AdvancedDisplay);
			Pin->bAdvancedView = bAdvancedPin;
			if(bAdvancedPin && (ENodeAdvancedPins::NoPins == AdvancedPinDisplay))
			{
				AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
			}

			FString ParamValue;
			if (K2Schema->FindFunctionParameterDefaultValue(Function, Param, ParamValue))
			{
				K2Schema->SetPinAutogeneratedDefaultValue(Pin, ParamValue);
			}
			else
			{
				K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
			}
			
			if (PinsToHide.Contains(Pin->PinName))
			{
				const FString PinNameStr = Pin->PinName.ToString();
				const FString& DefaultToSelfMetaValue = Function->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
				const FString& WorldContextMetaValue  = Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);
				bool bIsSelfPin = ((PinNameStr == DefaultToSelfMetaValue) || (PinNameStr == WorldContextMetaValue));

				if (!bShowWorldContextPin || !bIsSelfPin)
				{
					Pin->bHidden = true;
					Pin->bNotConnectable = InternalPins.Contains(Pin->PinName);
				}
			}

			PostParameterPinCreated(Pin);
		}

		bAllPinsGood = bAllPinsGood && bPinGood;
	}

	// If we have 'enum to exec' parameters, set their default value to something valid so we don't get warnings
	if (bWantsEnumToExecExpansion)
	{
		using namespace UE::K2NodeCallFunction::Private;

		TArray<FName> EnumNamesToCheck;
		GetExpandEnumPinNames(Function, EnumNamesToCheck);

		for (const FName& Name : EnumNamesToCheck)
		{
			UEdGraphPin* EnumParamPin = FindEnumParamPin(*this, Name);
			if (UEnum* PinEnum = (EnumParamPin ? Cast<UEnum>(EnumParamPin->PinType.PinSubCategoryObject.Get()) : nullptr))
			{
				EnumParamPin->DefaultValue = PinEnum->GetNameStringByIndex(0);
			}
		}
	}

	return bAllPinsGood;
}

void UK2Node_CallFunction::PostReconstructNode()
{
	Super::PostReconstructNode();
	InvalidatePinTooltips();

	// conform pins that are marked as SetParam:
	ConformContainerPins();

	FCustomStructureParamHelper::UpdateCustomStructurePins(GetTargetFunction(), this);

	// Fixup self node, may have been overridden from old self node
	UFunction* Function = GetTargetFunction();
	const bool bIsStaticFunc = Function ? Function->HasAllFunctionFlags(FUNC_Static) : false;

	UEdGraphPin* SelfPin = FindPin(UEdGraphSchema_K2::PN_Self);
	if (bIsStaticFunc && SelfPin)
	{
		// Wire up the self to the CDO of the class if it's not us
		if (UBlueprint* BP = GetBlueprint())
		{
			UClass* FunctionOwnerClass = Function->GetOuterUClass();
			if (!BP->SkeletonGeneratedClass || !BP->SkeletonGeneratedClass->IsChildOf(FunctionOwnerClass))
			{
				SelfPin->DefaultObject = FunctionOwnerClass->GetAuthoritativeClass()->GetDefaultObject();
			}
			else
			{
				// In case a non-NULL reference was previously serialized on load, ensure that it's set to NULL here to match what a new node's self pin would be initialized as (see CreatePinsForFunctionCall).
				SelfPin->DefaultObject = nullptr;
			}
		}
	}

	if (UEdGraphPin* TypePickerPin = FDynamicOutputHelper::GetTypePickerPin(this))
	{
		FDynamicOutputHelper(TypePickerPin).ConformOutputType();
	}

	if (IsNodePure())
	{
		// Remove the breakpoint
		FKismetDebugUtilities::RemoveBreakpointFromNode(this, GetBlueprint());
	}

	// If the user marked the override as pure, but the underlying function changed to have no outputs,
	// then we need to reset the override. Nodes can only be pure if they have outputs.
	const bool bNeedsPurityOverrideReset =
		(NodePurityOverride == ENodePurityOverride::Pure) &&
		!FunctionHasOutputs()
	;
	
	if (bNeedsPurityOverrideReset)
	{
		NodePurityOverride = ENodePurityOverride::Unset;
	}
}

void UK2Node_CallFunction::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	// conform pins that are marked as SetParam:
	ConformContainerPins();

	if (!ensure(Pin))
	{
		return;
	}

	FCustomStructureParamHelper::UpdateCustomStructurePins(GetTargetFunction(), this, Pin);

	// Refresh the node to hide internal-only pins once the [invalid] connection has been broken
	// If the pin was a container then it needs to be refreshed to get the correct pin literal text boxes
	// for its default value
	if (Pin->PinType.IsContainer() || (Pin->bHidden && Pin->bNotConnectable && Pin->LinkedTo.Num() == 0))
	{
		GetGraph()->NotifyGraphChanged();
	}

	InvalidatePinTooltips();
	if(!Pin->IsPendingKill())
	{
		FDynamicOutputHelper(Pin).ConformOutputType();
	}
}

void UK2Node_CallFunction::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);
	InvalidatePinTooltips();
	FDynamicOutputHelper(Pin).ConformOutputType();
}

UFunction* UK2Node_CallFunction::GetTargetFunction() const
{
	if(!FBlueprintCompilationManager::IsGeneratedClassLayoutReady())
	{
		// first look in the skeleton class:
		if(UFunction* SkeletonFn = GetTargetFunctionFromSkeletonClass())
		{
			return SkeletonFn;
		}
	}

	UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	return Function;
}

UFunction* UK2Node_CallFunction::GetTargetFunctionFromSkeletonClass() const
{
	UFunction* TargetFunction = nullptr;
	UClass* ParentClass = FunctionReference.GetMemberParentClass( GetBlueprintClassFromNode() );
	UBlueprint* OwningBP = ParentClass ? Cast<UBlueprint>( ParentClass->ClassGeneratedBy ) : nullptr;
	if( UClass* SkeletonClass = OwningBP ? *OwningBP->SkeletonGeneratedClass : nullptr )
	{
		TargetFunction = SkeletonClass->FindFunctionByName( FunctionReference.GetMemberName() );
	}
	return TargetFunction;
}

UEdGraphPin* UK2Node_CallFunction::GetReturnValuePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin == nullptr || Pin->Direction == EGPD_Output); // If pin exists, it must be output
	return Pin;
}

bool UK2Node_CallFunction::IsLatentFunction() const
{
	if (UFunction* Function = GetTargetFunction())
	{
		if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
		{
			return true;
		}
	}

	return false;
}

bool UK2Node_CallFunction::AllowMultipleSelfs(bool bInputAsArray) const
{
	if (UFunction* Function = GetTargetFunction())
	{
		return CanFunctionSupportMultipleTargets(Function);
	}

	return Super::AllowMultipleSelfs(bInputAsArray);
}

bool UK2Node_CallFunction::CanFunctionSupportMultipleTargets(UFunction const* Function)
{
	bool const bIsImpure = !Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
	bool const bIsLatent = Function->HasMetaData(FBlueprintMetadata::MD_Latent);
	bool const bHasReturnParam = (Function->GetReturnProperty() != nullptr);

	return !bHasReturnParam && bIsImpure && !bIsLatent;
}

bool UK2Node_CallFunction::CanEditorOnlyFunctionBeCalled(const UFunction* InFunction, const UObject* InObject)
{
	if (InFunction && InObject &&
		(IsEditorOnlyObject(InFunction) || InFunction->HasAnyFunctionFlags(FUNC_EditorOnly)))
	{
		if (!IsEditorOnlyObject(InObject))
		{
			// InObject isn't editor-only, but it's still possible that it's a blueprint derived from an editor-only class, so let's check for that case
			const UBlueprint* InObjectAsBP = Cast<const UBlueprint>(InObject->GetOuter());
			return (InObjectAsBP && InObjectAsBP->ParentClass && IsEditorOnlyObject(InObjectAsBP->ParentClass));
		}
	}

	return true;
}

bool UK2Node_CallFunction::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// If a BP class, prefer using the skeleton class.
	// This ensures that there's agreement with the target graph's class when we call CanFunctionBeUsedInGraph.
	const UClass* ParentClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
	const UBlueprint* OwningBP = ParentClass ? Cast<UBlueprint>(ParentClass->ClassGeneratedBy) : nullptr;
	if (OwningBP)
	{
		ParentClass = OwningBP->SkeletonGeneratedClass;
	}
	const UFunction* TargetFunction = ParentClass ? ParentClass->FindFunctionByName(FunctionReference.GetMemberName()) : nullptr;

	// Basic check for graph compatibility, etc.
	bool bCanPaste = Super::CanPasteHere(TargetGraph);

	// Cannot paste editor only functions into runtime graphs
	if (bCanPaste)
	{
		bCanPaste = CanEditorOnlyFunctionBeCalled(TargetFunction, TargetGraph);
	}

	// We check function context for placability only in the base class case; derived classes are typically bound to
	// specific functions that should always be placeable, but may not always be explicitly callable (e.g. InternalUseOnly).
	if (bCanPaste && GetClass() == StaticClass())
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		uint32 AllowedFunctionTypes = UEdGraphSchema_K2::EFunctionType::FT_Pure | UEdGraphSchema_K2::EFunctionType::FT_Const | UEdGraphSchema_K2::EFunctionType::FT_Protected;
		if (K2Schema->DoesGraphSupportImpureFunctions(TargetGraph))
		{
			AllowedFunctionTypes |= UEdGraphSchema_K2::EFunctionType::FT_Imperative;
		}

		if (TargetFunction)
		{
			bCanPaste = K2Schema->CanFunctionBeUsedInGraph(FBlueprintEditorUtils::FindBlueprintForGraphChecked(TargetGraph)->SkeletonGeneratedClass, TargetFunction, TargetGraph, AllowedFunctionTypes, false);
		}
		else
		{
			// If the function doesn't exist and it is from self context, then it could be created from a CustomEvent node, that was also pasted (but wasn't compiled yet).
			bCanPaste = FunctionReference.IsSelfContext();
		}
	}
	
	return bCanPaste;
}

bool UK2Node_CallFunction::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	for(UEdGraph* TargetGraph : Filter.Context.Graphs)
	{
		bIsFilteredOut |= !CanPasteHere(TargetGraph);
	}

	if(const UFunction* TargetFunction = GetTargetFunction())
	{
		const bool bIsProtected = (TargetFunction->FunctionFlags & FUNC_Protected) != 0;
		const bool bIsPrivate = (TargetFunction->FunctionFlags & FUNC_Private) != 0;
		const UClass* OwningClass = TargetFunction->GetOwnerClass();
		if( (bIsProtected || bIsPrivate) && !FBlueprintEditorUtils::IsNativeSignature(TargetFunction) && OwningClass)
		{
			OwningClass = OwningClass->GetAuthoritativeClass();
			// we can filter private and protected blueprints that are unrelated:
			bool bAccessibleInAll = true;
			for (const UBlueprint* Blueprint : Filter.Context.Blueprints)
			{
				UClass* AuthoritativeClass = Blueprint->GeneratedClass;
				if(!AuthoritativeClass)
				{
					continue;
				}

				if(bIsPrivate)
				{
					bAccessibleInAll = bAccessibleInAll && AuthoritativeClass == OwningClass;
				}
				else if(bIsProtected)
				{
					bAccessibleInAll = bAccessibleInAll && AuthoritativeClass->IsChildOf(OwningClass);
				}
			}

			if(!bAccessibleInAll)
			{
				bIsFilteredOut = true;
			}
		}
	}

	return bIsFilteredOut;
}

static FLinearColor GetPalletteIconColor(UFunction const* Function)
{
	bool const bIsPure = (Function != nullptr) && Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
	if (bIsPure)
	{
		return GetDefault<UGraphEditorSettings>()->PureFunctionCallNodeTitleColor;
	}
	return GetDefault<UGraphEditorSettings>()->FunctionCallNodeTitleColor;
}

FSlateIcon UK2Node_CallFunction::GetPaletteIconForFunction(UFunction const* Function, FLinearColor& OutColor)
{
	static const FName NativeMakeFunc(TEXT("NativeMakeFunc"));
	static const FName NativeBrakeFunc(TEXT("NativeBreakFunc"));

	if (Function && Function->HasMetaData(NativeMakeFunc))
	{
		static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeStruct_16x");
		return Icon;
	}
	else if (Function && Function->HasMetaData(NativeBrakeFunc))
	{
		static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.BreakStruct_16x");
		return Icon;
	}
	// Check to see if the function is calling an function that could be an event, display the event icon instead.
	else if (Function && UEdGraphSchema_K2::FunctionCanBePlacedAsEvent(Function))
	{
		static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
		return Icon;
	}
	else
	{
		OutColor = GetPalletteIconColor(Function);

		static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
		return Icon;
	}
}

FLinearColor UK2Node_CallFunction::GetNodeTitleColor() const
{
	return GetPalletteIconColor(GetTargetFunction());
}

FString UK2Node_CallFunction::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	if (EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::UseSearchSyntax))
	{
		// Searching by class member: try to resolve function and construct precise query
		if (const UFunction* Function = GetTargetFunction())
		{
			FString SearchTerm;
			if (FindInBlueprintsHelpers::ConstructSearchTermFromFunction(Function, SearchTerm))
			{
				return SearchTerm;
			}
		}
	}

	// Searching by function name: try to resolve function to return native name
	if (const UFunction* Function = GetTargetFunction())
	{
		const FString NativeName = Function->GetName();
		return FString::Printf(TEXT("\"%s\""), *NativeName);
	}

	// Fallback behavior: search by node title
	return Super::GetFindReferenceSearchString_Impl(InFlags);
}

FText UK2Node_CallFunction::GetTooltipText() const
{
	FText Tooltip;

	UFunction* Function = GetTargetFunction();
	if (Function == nullptr)
	{
		// try to see where this function is meant to come from:
		if (UClass* FuncOwnerClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode()))
		{
			return FText::Format(LOCTEXT("CallUnknownFunctionKnownOuter", "Call unknown function {0} - missing from {1}"), FText::FromName(FunctionReference.GetMemberName()), FText::FromName(FuncOwnerClass->GetFName()));
		}
		else
		{
			return FText::Format(LOCTEXT("CallUnknownFunction", "Call unknown function {0}"), FText::FromName(FunctionReference.GetMemberName()));
		}
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		FText BaseTooltip = FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function));

		FFormatNamedArguments Args;
		Args.Add(TEXT("DefaultTooltip"), BaseTooltip);

		if (Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
		{
			Args.Add(
				TEXT("Subtitle"),
				NSLOCTEXT("K2Node", "ServerFunction", "Authority Only. This function will only execute on the server.")
			);
			// FText::Format() is slow, so we cache this to save on performance
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("CallFunction_SubtitledTooltip", "{DefaultTooltip}\n\n{Subtitle}"), Args), this);
		}
		else if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic))
		{
			Args.Add(
				TEXT("Subtitle"),
				NSLOCTEXT("K2Node", "ClientFunction", "Cosmetic. This event is only for cosmetic, non-gameplay actions.")
			);
			// FText::Format() is slow, so we cache this to save on performance
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("CallFunction_SubtitledTooltip", "{DefaultTooltip}\n\n{Subtitle}"), Args), this);
		} 
		else if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
		{
			Args.Add(
				TEXT("Subtitle"),
				NSLOCTEXT("K2Node", "LatentFunction", "Latent. This node will complete at a later time. Latent nodes can only be placed in event graphs.")
			);
			// FText::Format() is slow, so we cache this to save on performance
			CachedTooltip.SetCachedText(FText::Format(LOCTEXT("CallFunction_SubtitledTooltip", "{DefaultTooltip}\n\n{Subtitle}"), Args), this);
		}
		else
		{
			CachedTooltip.SetCachedText(BaseTooltip, this);
		}
	}
	return CachedTooltip;
}

void UK2Node_CallFunction::GeneratePinTooltipFromFunction(UEdGraphPin& Pin, const UFunction* Function)
{
	if (Pin.bWasTrashed)
	{
		return;
	}

	const bool bReturnPin = Pin.PinName == UEdGraphSchema_K2::PN_ReturnValue;
	const FString ParamTooltip = bReturnPin ? ObjectTools::GetDefaultTooltipForFunctionReturn(Function) : ObjectTools::GetDefaultTooltipForFunctionParam(Function, Pin.PinName.ToString());
	Pin.PinToolTip += ParamTooltip;

	GetDefault<UEdGraphSchema_K2>()->ConstructBasicPinTooltip(Pin, FText::FromString(Pin.PinToolTip), Pin.PinToolTip);
}

FText UK2Node_CallFunction::GetUserFacingFunctionName(const UFunction* Function, ENodeTitleType::Type NodeTitleType /*= ENodeTitleType::EditableTitle*/)
{
	const bool bAllowFriendlyNames = NodeTitleType == ENodeTitleType::FullTitle;
	return ObjectTools::GetUserFacingFunctionName(Function, bAllowFriendlyNames);
}

FString UK2Node_CallFunction::GetDefaultTooltipForFunction(const UFunction* Function)
{
	return ObjectTools::GetDefaultTooltipForFunction(Function);
}

FText UK2Node_CallFunction::GetDefaultCategoryForFunction(const UFunction* Function, const FText& BaseCategory)
{
	FText NodeCategory = BaseCategory;
	if( Function->HasMetaData(FBlueprintMetadata::MD_FunctionCategory) )
	{
		FText FuncCategory;
		// If we are not showing friendly names, return the metadata stored, without localization
		if( GEditor && !GetDefault<UEditorStyleSettings>()->bShowFriendlyNames )
		{
			FuncCategory = FText::FromString(Function->GetMetaData(FBlueprintMetadata::MD_FunctionCategory));
		}
		else
		{
			// Look for localized metadata
			FuncCategory = FObjectEditorUtils::GetCategoryText(Function);

			// If the result is culture invariant, force it into a display string
			if (FuncCategory.IsCultureInvariant())
			{
				FuncCategory = FText::FromString(FName::NameToDisplayString(FuncCategory.ToString(), false));
			}
		}

		// Combine with the BaseCategory to form the full category, delimited by "|"
		if (!FuncCategory.IsEmpty() && !NodeCategory.IsEmpty())
		{
			NodeCategory = FText::Format(FText::FromString(TEXT("{0}|{1}")), NodeCategory, FuncCategory);
		}
		else if (NodeCategory.IsEmpty())
		{
			NodeCategory = FuncCategory;
		}
	}
	return NodeCategory;
}


FText UK2Node_CallFunction::GetKeywordsForFunction(const UFunction* Function)
{
	// Always add the real function name as the first keyword, even if it matches the display name we don't want to penalize one word function names in later searches
	FString Keywords = Function->GetName();

	if (ShouldDrawCompact(Function))
	{
		Keywords.AppendChar(TEXT(' '));
		Keywords += GetCompactNodeTitle(Function);
	}

	FText MetadataKeywords = Function->GetMetaDataText(FBlueprintMetadata::MD_FunctionKeywords, TEXT("UObjectKeywords"), Function->GetFullGroupName(false));
	FText ResultKeywords;

	if (!MetadataKeywords.IsEmpty())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Name"), FText::FromString(Keywords));
		Args.Add(TEXT("MetadataKeywords"), MetadataKeywords);
		ResultKeywords = FText::Format(FText::FromString("{Name} {MetadataKeywords}"), Args);
	}
	else
	{
		ResultKeywords = FText::FromString(Keywords);
	}

	return ResultKeywords;
}

void UK2Node_CallFunction::SetFromFunction(const UFunction* Function)
{
	if (Function != NULL)
	{
		bDefaultsToPureFunc = Function->HasAnyFunctionFlags(FUNC_BlueprintPure);
		DetermineWantsEnumToExecExpansion(Function);

		FunctionReference.SetFromField<UFunction>(Function, GetBlueprintClassFromNode());
	}
}

FString UK2Node_CallFunction::GetDocumentationLink() const
{
	UClass* ParentClass = NULL;
	if (FunctionReference.IsSelfContext())
	{
		if (HasValidBlueprint())
		{
			UFunction* Function = FindUField<UFunction>(GetBlueprint()->GeneratedClass, FunctionReference.GetMemberName());
			if (Function != NULL)
			{
				ParentClass = Function->GetOwnerClass();
			}
		}		
	}
	else 
	{
		ParentClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
	}
	
	if (ParentClass != NULL)
	{
		return FString::Printf(TEXT("Shared/GraphNodes/Blueprint/%s%s"), ParentClass->GetPrefixCPP(), *ParentClass->GetName());
	}

	return FString("Shared/GraphNodes/Blueprint/UK2Node_CallFunction");
}

FString UK2Node_CallFunction::GetDocumentationExcerptName() const
{
	return FunctionReference.GetMemberName().ToString();
}

FString UK2Node_CallFunction::GetDescriptiveCompiledName() const
{
	return FString(TEXT("CallFunc_")) + FunctionReference.GetMemberName().ToString();
}

bool UK2Node_CallFunction::ShouldDrawCompact(const UFunction* Function)
{
	return (Function != NULL) && Function->HasMetaData(FBlueprintMetadata::MD_CompactNodeTitle);
}

bool UK2Node_CallFunction::ShouldDrawCompact() const
{
	UFunction* Function = GetTargetFunction();

	return ShouldDrawCompact(Function);
}

bool UK2Node_CallFunction::ShouldShowNodeProperties() const
{
	// Show node properties if this corresponds to a function graph
	if (FunctionReference.GetMemberName() != NAME_None && HasValidBlueprint())
	{
		return FindObject<UEdGraph>(GetBlueprint(), *(FunctionReference.GetMemberName().ToString())) != NULL;
	}
	return false;
}

FString UK2Node_CallFunction::GetCompactNodeTitle(const UFunction* Function)
{
	static const FString ProgrammerMultiplicationSymbol = TEXT("*");
	static const FString CommonMultiplicationSymbol = TEXT("\xD7");

	static const FString ProgrammerDivisionSymbol = TEXT("/");
	static const FString CommonDivisionSymbol = TEXT("\xF7");

	static const FString ProgrammerConversionSymbol = TEXT("->");
	static const FString CommonConversionSymbol = TEXT("\x2022");

	const FString& OperatorTitle = Function->GetMetaData(FBlueprintMetadata::MD_CompactNodeTitle);
	if (!OperatorTitle.IsEmpty())
	{
		if (OperatorTitle == ProgrammerMultiplicationSymbol)
		{
			return CommonMultiplicationSymbol;
		}
		else if (OperatorTitle == ProgrammerDivisionSymbol)
		{
			return CommonDivisionSymbol;
		}
		else if (OperatorTitle == ProgrammerConversionSymbol)
		{
			return CommonConversionSymbol;
		}
		else
		{
			return OperatorTitle;
		}
	}
	
	return Function->GetName();
}

FText UK2Node_CallFunction::GetCompactNodeTitle() const
{
	UFunction* Function = GetTargetFunction();
	if (Function != NULL)
	{
		return FText::FromString(GetCompactNodeTitle(Function));
	}
	else
	{
		return Super::GetCompactNodeTitle();
	}
}

void UK2Node_CallFunction::GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const
{
	Super::GetRedirectPinNames(Pin, RedirectPinNames);

	if (RedirectPinNames.Num() > 0)
	{
		const FString OldPinName = RedirectPinNames[0];

		// first add functionname.param
		RedirectPinNames.Add(FString::Printf(TEXT("%s.%s"), *FunctionReference.GetMemberName().ToString(), *OldPinName));

		// if there is class, also add an option for class.functionname.param
		UClass* FunctionClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
		while (FunctionClass)
		{
			RedirectPinNames.Add(FString::Printf(TEXT("%s.%s.%s"), *FunctionClass->GetName(), *FunctionReference.GetMemberName().ToString(), *OldPinName));
			FunctionClass = FunctionClass->GetSuperClass();
		}
	}
}

void UK2Node_CallFunction::FixupSelfMemberContext()
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
	auto IsBlueprintOfType = [Blueprint](UClass* ClassType)->bool
	{
		bool bIsChildOf  = Blueprint && (Blueprint->GeneratedClass != nullptr) && Blueprint->GeneratedClass->IsChildOf(ClassType);
		if (!bIsChildOf && Blueprint && (Blueprint->SkeletonGeneratedClass))
		{
			bIsChildOf = Blueprint->SkeletonGeneratedClass->IsChildOf(ClassType);
		}
		return bIsChildOf;
	};

	UClass* MemberClass = FunctionReference.GetMemberParentClass();
	if (FunctionReference.IsSelfContext())
	{
		// if there is a function that matches the reference in the new context
		// and there are no connections to the self pin, we just want to call
		// that function
		UEdGraphPin* SelfPin = GetDefault<UEdGraphSchema_K2>()->FindSelfPin(*this, EGPD_Input);
		if (!FunctionReference.ResolveMember<UFunction>(Blueprint) || (SelfPin && SelfPin->HasAnyConnections()))
		{
			if (MemberClass == nullptr)
			{
				// the self pin may have type information stored on it
				if (SelfPin)
				{
					MemberClass = Cast<UClass>(SelfPin->PinType.PinSubCategoryObject.Get());
				}
			}
			// if we happened to retain the ParentClass for a self reference 
			// (unlikely), then we know where this node came from... let's keep it
			// referencing that function
			if (MemberClass != nullptr)
			{
				if (!IsBlueprintOfType(MemberClass))
				{
					FunctionReference.SetExternalMember(FunctionReference.GetMemberName(), MemberClass);
				}
			}
			// else, there is nothing we can do... the node will produce an error later during compilation
		}
	}
	else if (MemberClass != nullptr)
	{
		if (IsBlueprintOfType(MemberClass))
		{
			FunctionReference.SetSelfMember(FunctionReference.GetMemberName());
		}
	}
}

bool UK2Node_CallFunction::CanToggleNodePurity() const
{
	// If the underlying function defaults to pure, then we can always toggle it.
	// If not, then we only allow it if enabled in the project settings.
	// Additionally, only functions with ouputs can be toggled.
	// Otherwise, you can end up with a pure node with no outputs that's never evaluated.

	const bool bFunctionHasOutputs = FunctionHasOutputs();

	if (bDefaultsToPureFunc)
	{
		return bFunctionHasOutputs;
	}
	else
	{
		const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
		check(EditorProjectSettings);

		return bFunctionHasOutputs && EditorProjectSettings->bAllowImpureToPureNodeConversion;
	}
}

void UK2Node_CallFunction::SuppressDeprecationWarning() const
{
	if (UFunction* Function = GetTargetFunction())
	{
		FString PathName = Function->GetPathName();
		UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetMutableDefault<UBlueprintEditorProjectSettings>();
		BlueprintEditorProjectSettings->SuppressedDeprecationMessages.Add(MoveTemp(PathName));
		BlueprintEditorProjectSettings->SaveConfig();
		BlueprintEditorProjectSettings->TryUpdateDefaultConfigFile("", false);
	}
}

TSet<FName> UK2Node_CallFunction::GetRequiredParamNames(const UFunction* ForFunction)
{
	TSet<FName> Result;

	for (TFieldIterator<FProperty> PropIt(ForFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Param = *PropIt;
		const bool bIsRequiredParam = Param->HasAnyPropertyFlags(CPF_RequiredParm);
		if (bIsRequiredParam)
		{
			Result.Add(Param->GetFName());
		}
	}
	return Result;
}

void UK2Node_CallFunction::ValidateRequiredPins(const UFunction* Function, FCompilerResultsLog& MessageLog) const
{
	TSet<FName> RequiredPinNames = GetRequiredParamNames(Function);

	if(RequiredPinNames.Num() == 0)
	{
		return;
	}

	for (const UEdGraphPin* Pin : Pins)
	{
		if (Pin != nullptr)
		{
			const bool bIsRequired = RequiredPinNames.Contains(Pin->GetFName());
			const bool bIsNotLinked = Pin->LinkedTo.Num() == 0;
			if (bIsRequired && bIsNotLinked)
			{
				MessageLog.Error(*LOCTEXT("MissingRequiredPin", "Pin @@ must be linked to another node (in @@)").ToString(), Pin, this);
			}
		}
	}
}

void UK2Node_CallFunction::PostPasteNode()
{
	Super::PostPasteNode();
	FixupSelfMemberContext();

	if (UFunction* Function = GetTargetFunction())
	{
		if (Pins.Num() > 0)
		{
			// After pasting we need to go through and ensure the hidden the self pins is correct in case the source blueprint had different metadata
			TSet<FName> PinsToHide;
			FBlueprintEditorUtils::GetHiddenPinsForFunction(GetGraph(), Function, PinsToHide);

			const bool bShowWorldContextPin = ((PinsToHide.Num() > 0) && GetBlueprint()->ParentClass->HasMetaDataHierarchical(FBlueprintMetadata::MD_ShowWorldContextPin));

			const FString& DefaultToSelfMetaValue = Function->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
			const FString& WorldContextMetaValue = Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);

			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* Pin = Pins[PinIndex];
					const FString PinNameStr = Pin->PinName.ToString();

					const bool bIsSelfPin = ((PinNameStr == DefaultToSelfMetaValue) || (PinNameStr == WorldContextMetaValue));
					const bool bPinShouldBeHidden = ((Pin->SubPins.Num() > 0) || (PinsToHide.Contains(Pin->PinName) && (!bShowWorldContextPin || !bIsSelfPin)));

				if (bPinShouldBeHidden && !Pin->bHidden)
				{
					Pin->BreakAllPinLinks();
					K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
				}
				Pin->bHidden = bPinShouldBeHidden;
			}
		}
	}
}

bool UK2Node_CallFunction::CanSplitPin(const UEdGraphPin* Pin) const
{
	TSet<FName> RequiredPins;
	if (UFunction* Function = GetTargetFunction())
	{
		RequiredPins = GetRequiredParamNames(Function);
	}
	return Super::CanSplitPin(Pin) && !RequiredPins.Contains(Pin->GetFName());
}

void UK2Node_CallFunction::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE && (!this->HasAnyFlags(RF_Transient)))
	{
		FixupSelfMemberContext();
	}
}

void UK2Node_CallFunction::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UBlueprint* Blueprint = GetBlueprint();
	UFunction *Function = GetTargetFunction();
	if (Function == nullptr)
	{
		FString OwnerName;

		if (Blueprint != nullptr)
		{
			OwnerName = Blueprint->GetName();
			if (UClass* FuncOwnerClass = FunctionReference.GetMemberParentClass(Blueprint->GeneratedClass))
			{
				OwnerName = FuncOwnerClass->GetName();
			}
		}
		FString const FunctName = FunctionReference.GetMemberName().ToString();

		FText const WarningFormat = LOCTEXT("FunctionNotFoundFmt", "Could not find a function named \"{0}\" in '{1}'.\nMake sure '{2}' has been compiled for @@");
		MessageLog.Error(*FText::Format(WarningFormat, FText::FromString(FunctName), FText::FromString(OwnerName), FText::FromString(OwnerName)).ToString(), this);
	}
	else if (WantsExecPinsForParams(Function) && bWantsEnumToExecExpansion == false)
	{
		// will technically not have a properly formatted output for multiple params... but /shrug. 
		const FString EnumParamName = GetAllExecParams(Function);
		MessageLog.Warning(*FText::Format(LOCTEXT("EnumToExecExpansionFailedFmt", "Unable to find enum parameter with name '{0}' to expand for @@"), FText::FromString(EnumParamName)).ToString(), this);
	}

	const UClass* BlueprintClass = Blueprint ? *Blueprint->ParentClass : nullptr;
	const bool bIsEditorOnlyBlueprintBaseClass = !BlueprintClass || IsEditorOnlyObject(BlueprintClass);
	static bool bAllowUnsafeBlueprintCalls = FParse::Param(FCommandLine::Get(), TEXT("AllowUnsafeBlueprintCalls"));

	if (!bAllowUnsafeBlueprintCalls)
	{
		// This error is disabled while we figure out how we can identify uncooked only
		// blueprints that want to make use of uncooked only APIs:
		#if 0
		const bool bIsUncookedOnlyFunction = Function && Function->GetOutermost()->HasAllPackagesFlags(PKG_UncookedOnly);
		if (bIsUncookedOnlyFunction &&
			// Only allow calls to uncooked only functions from editor only/uncooked only
			// contexts:
			!(GetOutermost()->HasAnyPackageFlags(PKG_UncookedOnly | PKG_EditorOnly) ||
				bIsEditorOnlyBlueprintBaseClass))
		{
			MessageLog.Error(*LOCTEXT("UncookedOnlyError", "Attempting to call uncooked only function @@ in runtime blueprint").ToString(), this);
		}
		#endif	// 0

		// Ensure that editor module BP exposed UFunctions can only be called in blueprints for which the base class is also part of an editor module
		// Also check for functions wrapped in WITH_EDITOR 
		if (Function && Blueprint &&
			(IsEditorOnlyObject(Function) || Function->HasAnyFunctionFlags(FUNC_EditorOnly)))
		{
			if (!bIsEditorOnlyBlueprintBaseClass)
			{
				FString const FunctName = Function->GetName();
				FText const WarningFormat = LOCTEXT("EditorFunctionFmt", "Cannot use the editor function \"{0}\" in this runtime Blueprint. Only for use in Editor Utility Blueprints and Blutilities.");
				MessageLog.Error(*FText::Format(WarningFormat, FText::FromString(FunctName)).ToString(), this);
			}
		}
	}

	if (Function)
	{
		ValidateRequiredPins(Function, MessageLog);

		// enforce UnsafeDuringActorConstruction keyword
		if (Function->HasMetaData(FBlueprintMetadata::MD_UnsafeForConstructionScripts))
		{
			// emit warning if we are in a construction script
			UEdGraph const* const Graph = GetGraph();
			bool bNodeIsInConstructionScript = UEdGraphSchema_K2::IsConstructionScript(Graph);

			if (bNodeIsInConstructionScript == false)
			{
				// IsConstructionScript() can return false if graph was cloned from the construction script
				// in that case, check the function entry
				TArray<const UK2Node_FunctionEntry*> EntryPoints;
				Graph->GetNodesOfClass(EntryPoints);

				if (EntryPoints.Num() == 1)
				{
					UK2Node_FunctionEntry const* const Node = EntryPoints[0];
					if (Node)
					{
						UFunction* const SignatureFunction = Node->FunctionReference.ResolveMember<UFunction>(Node->GetBlueprintClassFromNode());
						bNodeIsInConstructionScript = SignatureFunction && (SignatureFunction->GetFName() == UEdGraphSchema_K2::FN_UserConstructionScript);
					}
				}
			}

			if (bNodeIsInConstructionScript)
			{
				MessageLog.Warning(*LOCTEXT("FunctionUnsafeDuringConstruction", "Function '@@' is unsafe to call in a construction script.").ToString(), this);
			}
		}

		// enforce WorldContext restrictions
		const bool bInsideBpFuncLibrary = Blueprint && (BPTYPE_FunctionLibrary == Blueprint->BlueprintType);
		
		if (!bInsideBpFuncLibrary &&
			Function->HasMetaData(FBlueprintMetadata::MD_WorldContext) && 
			(!Function->HasMetaData(FBlueprintMetadata::MD_CallableWithoutWorldContext)))
		{
			check(Blueprint);
			UClass* ParentClass = Blueprint->ParentClass;
			check(ParentClass);

			// go through all of the pins and verify if we have a connected world context or not.
			const FName ContextParam = FName(*Function->GetMetaData(FBlueprintMetadata::MD_WorldContext));
			bool bConnectedWorldContext = false;
			for (const UEdGraphPin* Pin : Pins)
			{
				if (Pin->PinName.IsEqual(ContextParam) && Pin->LinkedTo.Num() > 0)
				{
					bConnectedWorldContext = true;
					break;
				}
			}

			if (!bConnectedWorldContext && ParentClass && !FBlueprintEditorUtils::ImplementsGetWorld(Blueprint) && !ParentClass->HasMetaDataHierarchical(FBlueprintMetadata::MD_ShowWorldContextPin))
			{
				MessageLog.Warning(*LOCTEXT("FunctionUnsafeInContext", "Function '@@' is unsafe to call from blueprints of class '@@' without connecting a World Context Object.").ToString(), this, ParentClass);
			}
		}

		if (Blueprint && !FBlueprintEditorUtils::IsNativeSignature(Function))
		{
			// enforce protected function restriction
			const bool bCanTreatAsError = Blueprint->GetLinkerCustomVersion(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::EnforceBlueprintFunctionVisibility;

			const bool bIsProtected = (Function->FunctionFlags & FUNC_Protected) != 0;
			const bool bFuncBelongsToSubClass = Blueprint->SkeletonGeneratedClass && Blueprint->SkeletonGeneratedClass->IsChildOf(Function->GetOuterUClass());
			if (bIsProtected && !bFuncBelongsToSubClass)
			{
				if(bCanTreatAsError)
				{
					MessageLog.Error(*LOCTEXT("FunctionProtectedAccessed", "Function '@@' is protected and can't be accessed outside of its hierarchy.").ToString(), this);
				}
				else
				{
					MessageLog.Note(*LOCTEXT("FunctionProtectedAccessedNote", "Function '@@' is protected and can't be accessed outside of its hierarchy - this will be an error if the asset is resaved.").ToString(), this);
				}
			}

			// enforce private function restriction
			const bool bIsPrivate = (Function->FunctionFlags & FUNC_Private) != 0;
			const bool bFuncBelongsToClass = bFuncBelongsToSubClass && (Blueprint->SkeletonGeneratedClass == Function->GetOuterUClass());
			if (bIsPrivate && !bFuncBelongsToClass)
			{
				if (bCanTreatAsError)
				{
					MessageLog.Error(*LOCTEXT("FunctionPrivateAccessed", "Function '@@' is private and can't be accessed outside of its defined class '@@'.").ToString(), this, Function->GetOuterUClass());
				}
				else
				{
					MessageLog.Note(*LOCTEXT("FunctionPrivateAccessedNote", "Function '@@' is private and can't be accessed outside of its defined class '@@' - this will be an error if the asset is resaved.").ToString(), this, Function->GetOuterUClass());
				}
			}
		}

		const UBlueprintEditorProjectSettings* EditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
		check(EditorProjectSettings);

		const bool bHasIllegalPureOverride =
			!bDefaultsToPureFunc &&
			(NodePurityOverride == ENodePurityOverride::Pure) &&
			(!EditorProjectSettings->bAllowImpureToPureNodeConversion)
		;

		if (bHasIllegalPureOverride)
		{
			MessageLog.Warning(*LOCTEXT("IllegalPureNodeOverride", "Function '@@' is not pure and conversions to pure nodes are illegal in this project. Either replace the node or enable pure node conversions in the project settings.").ToString(), this);
		}
	}

	FDynamicOutputHelper::VerifyNode(this, MessageLog);

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->PinType.bIsWeakPointer && !Pin->PinType.IsContainer())
		{
			const FString ErrorString = FText::Format(
				LOCTEXT("WeakPtrNotSupportedErrorFmt", "Weak pointers are not supported as function parameters. Pin '{0}' @@"),
				FText::FromString(Pin->GetName())
			).ToString();
			MessageLog.Error(*ErrorString, this);
		}
		else if (Pin && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object && Pin->PinName == UEdGraphSchema_K2::PN_Self)
		{
			const UEdGraphPin* SelfPin = MessageLog.FindSourcePin(Pin);
			for (const UEdGraphPin* LinkedTo : SelfPin->LinkedTo)
			{
				if (ensure(LinkedTo) && LinkedTo->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
				{
					if (LinkedTo->PinType.IsArray())
					{
						MessageLog.Note(*FText::Format(LOCTEXT("InterfaceArrayTargetConnectionNote", "@@: An array of interface types can no longer be directly connected to '{0}'. Each entry must first be cast to the object type. However, the existing connection to '{1}' will continue to work for backwards-compatibility."), FText::FromString(SelfPin->GetName()), FText::FromString(LinkedTo->GetName())).ToString(), this);
					}
					else
					{
						MessageLog.Note(*FText::Format(LOCTEXT("InterfaceTargetConnectionNote", "@@: An interface type can no longer be directly connected to '{0}'. It must first be cast to the object type. However, the existing connection to '{1}' will continue to work for backwards-compatibility."), FText::FromString(SelfPin->GetName()), FText::FromString(LinkedTo->GetName())).ToString(), this);
					}
				}
			}
		}
	}
}

void UK2Node_CallFunction::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.UEVer() < VER_UE4_SWITCH_CALL_NODE_TO_USE_MEMBER_REFERENCE)
		{
			UFunction* Function = FindUField<UFunction>(CallFunctionClass_DEPRECATED, CallFunctionName_DEPRECATED);
			const bool bProbablySelfCall = (CallFunctionClass_DEPRECATED == NULL) || ((Function != NULL) && (Function->GetOuterUClass()->ClassGeneratedBy == GetBlueprint()));

			FunctionReference.SetDirect(CallFunctionName_DEPRECATED, FGuid(), CallFunctionClass_DEPRECATED, bProbablySelfCall);
		}

		if(Ar.UEVer() < VER_UE4_K2NODE_REFERENCEGUIDS)
		{
			FGuid FunctionGuid;

			if (UBlueprint::GetGuidFromClassByFieldName<UFunction>(GetBlueprint()->GeneratedClass, FunctionReference.GetMemberName(), FunctionGuid))
			{
				const bool bSelf = FunctionReference.IsSelfContext();
				FunctionReference.SetDirect(FunctionReference.GetMemberName(), FunctionGuid, (bSelf ? NULL : FunctionReference.GetMemberParentClass((UClass*)NULL)), bSelf);
			}
		}

		// Consider the 'CPF_UObjectWrapper' flag on native function call parameters and return values.
		if (Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::PinTypeIncludesUObjectWrapperFlag)
		{
			if (UFunction* TargetFunction = GetTargetFunction())
			{
				if (TargetFunction->IsNative())
				{
					for (TFieldIterator<FProperty> PropIt(TargetFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
					{
						if (UEdGraphPin* Pin = FindPin(PropIt->GetFName()))
						{
							if (const FMapProperty* MapProperty = CastField<FMapProperty>(*PropIt))
							{
								if (MapProperty->KeyProp && MapProperty->KeyProp->HasAllPropertyFlags(CPF_UObjectWrapper))
								{
									Pin->PinType.bIsUObjectWrapper = 1;
								}

								if (MapProperty->ValueProp && MapProperty->ValueProp->HasAllPropertyFlags(CPF_UObjectWrapper))
								{
									Pin->PinType.PinValueType.bTerminalIsUObjectWrapper = true;
								}
							}
							else if (const FSetProperty* SetProperty = CastField<FSetProperty>(*PropIt))
							{
								if (SetProperty->ElementProp && SetProperty->ElementProp->HasAllPropertyFlags(CPF_UObjectWrapper))
								{
									Pin->PinType.PinValueType.bTerminalIsUObjectWrapper = true;
								}
							}
							else if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*PropIt))
							{
								if(ArrayProperty->Inner && ArrayProperty->Inner->HasAllPropertyFlags(CPF_UObjectWrapper))
								{
									Pin->PinType.PinValueType.bTerminalIsUObjectWrapper = true;
								}
							}
							else if (PropIt->HasAllPropertyFlags(CPF_UObjectWrapper))
							{
								Pin->PinType.bIsUObjectWrapper = 1;
							}
						}
					}
				}
			}
		}

		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			check(Pin);

			bool bNeedsSubCategoryObjectRepair =
				(Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) &&
				(Pin->PinType.PinSubCategory != UEdGraphSchema_K2::PSC_Self) &&
				(Pin->PinType.PinSubCategoryObject == nullptr) &&
				(Ar.CustomVer(FReleaseObjectVersion::GUID) < FUE5MainStreamObjectVersion::NullPinSubCategoryObjectFix);

			// Prior to NullPinSubCategoryObjectFix, some object pins were serialized with a null PinSubCategoryObject.
			// Going forward, this will be an error, so we'll attempt to repair the pin by assigning a class.
			if (bNeedsSubCategoryObjectRepair)
			{
				Pin->PinType.PinSubCategoryObject = FunctionReference.GetMemberParentClass();
			}
		}

		if (!Ar.IsObjectReferenceCollector())
		{
			// Don't validate the enabled state if the user has explicitly set it. Also skip validation if we're just duplicating this node.
			const bool bIsDuplicating = (Ar.GetPortFlags() & PPF_Duplicate) != 0;
			if (!bIsDuplicating && !HasUserSetTheEnabledState())
			{
				if (const UFunction* Function = GetTargetFunction())
				{
					// Enable as development-only if specified in metadata. This way existing functions that have the metadata added to them will get their enabled state fixed up on load.
					if (GetDesiredEnabledState() == ENodeEnabledState::Enabled && Function->HasMetaData(FBlueprintMetadata::MD_DevelopmentOnly))
					{
						SetEnabledState(ENodeEnabledState::DevelopmentOnly, /*bUserAction=*/ false);
					}
					// Ensure that if the metadata is removed, we also fix up the enabled state to avoid leaving it set as development-only in that case.
					else if (GetDesiredEnabledState() == ENodeEnabledState::DevelopmentOnly && !Function->HasMetaData(FBlueprintMetadata::MD_DevelopmentOnly))
					{
						SetEnabledState(ENodeEnabledState::Enabled, /*bUserAction=*/ false);
					}
				}
			}
		}
	}
}

void UK2Node_CallFunction::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// Try re-setting the function given our new parent scope, in case it turns an external to an internal, or vis versa
	FunctionReference.RefreshGivenNewSelfScope<UFunction>(GetBlueprintClassFromNode());

	// Set the node to development only if the function specifies that
	check(!HasUserSetTheEnabledState());
	if (const UFunction* Function = GetTargetFunction())
	{
		if (Function->HasMetaData(FBlueprintMetadata::MD_DevelopmentOnly))
		{
			SetEnabledState(ENodeEnabledState::DevelopmentOnly, /*bUserAction=*/ false);
		}
	}
}

FNodeHandlingFunctor* UK2Node_CallFunction::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_CallFunction(CompilerContext);
}

void UK2Node_CallFunction::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	UFunction* Function = GetTargetFunction();

	// connect DefaultToSelf and WorldContext inside static functions to proper 'self'  
	if (SourceGraph && Schema->IsStaticFunctionGraph(SourceGraph) && Function)
	{
		TArray<UK2Node_FunctionEntry*> EntryPoints;
		SourceGraph->GetNodesOfClass(EntryPoints);
		if (1 != EntryPoints.Num())
		{
			CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("WrongEntryPointsNumFmt", "{0} entry points found while expanding node @@"), EntryPoints.Num()).ToString(), this);
		}
		else if (UEdGraphPin* BetterSelfPin = EntryPoints[0]->GetAutoWorldContextPin())
		{
			const FString& DefaultToSelfMetaValue = Function->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
			const FString& WorldContextMetaValue = Function->GetMetaData(FBlueprintMetadata::MD_WorldContext);

			struct FStructConnectHelper
			{
				static void Connect(const FString& PinName, UK2Node* Node, UEdGraphPin* BetterSelf, const UEdGraphSchema_K2* InSchema, FCompilerResultsLog& MessageLog)
				{
					UEdGraphPin* Pin = Node->FindPin(PinName);
					if (!PinName.IsEmpty() && Pin && !Pin->LinkedTo.Num())
					{
						const bool bConnected = InSchema->TryCreateConnection(Pin, BetterSelf);
						if (!bConnected)
						{
							MessageLog.Warning(*LOCTEXT("DefaultToSelfNotConnected", "DefaultToSelf pin @@ from node @@ cannot be connected to @@").ToString(), Pin, Node, BetterSelf);
						}
					}
				}
			};
			FStructConnectHelper::Connect(DefaultToSelfMetaValue, this, BetterSelfPin, Schema, CompilerContext.MessageLog);

			// Attempt to connect, even if it's optional because of MD_CallableWithoutWorldContext
			if (!WorldContextMetaValue.IsEmpty())
			{
				FStructConnectHelper::Connect(WorldContextMetaValue, this, BetterSelfPin, Schema, CompilerContext.MessageLog);
			}
		}
	}

	// If we have an enum param that is expanded, we handle that first
	if (bWantsEnumToExecExpansion)
	{
		if(Function)
		{
			using namespace UE::K2NodeCallFunction::Private;

			TArray<FName> EnumNamesToCheck;
			GetExpandEnumPinNames(Function, EnumNamesToCheck);

			bool bAlreadyHandleInput = false;

			UEdGraphPin* OutMainExecutePin = nullptr;
			UK2Node_ExecutionSequence* SpawnedSequenceNode = nullptr;
			int32 OutSequenceIndex = 0;

			const auto LinkIntoOutputChain = [&OutMainExecutePin, &SpawnedSequenceNode, &OutSequenceIndex, &CompilerContext, this, SourceGraph, Schema](UK2Node* Node)
			{
				if (!OutMainExecutePin)
				{
					// Create normal exec output -- only once though.
					OutMainExecutePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
				}
				else
				{
					// set up a sequence so we can call one after another.
					if (!SpawnedSequenceNode)
					{
						SpawnedSequenceNode = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
						SpawnedSequenceNode->AllocateDefaultPins();
						CompilerContext.MovePinLinksToIntermediate(*OutMainExecutePin, *SpawnedSequenceNode->GetThenPinGivenIndex(OutSequenceIndex++));
						Schema->TryCreateConnection(OutMainExecutePin, SpawnedSequenceNode->Pins[0]);
					}
				}

				// Hook up execution to the branch node
				if (!SpawnedSequenceNode)
				{
					Schema->TryCreateConnection(OutMainExecutePin, Node->GetExecPin());
				}
				else
				{
					UEdGraphPin* SequenceOutput = SpawnedSequenceNode->GetThenPinGivenIndex(OutSequenceIndex);

					if (!SequenceOutput)
					{
						SpawnedSequenceNode->AddInputPin();
						SequenceOutput = SpawnedSequenceNode->GetThenPinGivenIndex(OutSequenceIndex);
					}

					Schema->TryCreateConnection(SequenceOutput, Node->GetExecPin());
					OutSequenceIndex++;
				}
			};

			for (const FName& ParamName : EnumNamesToCheck)
			{
				UEnum* Enum = nullptr;
				UEdGraphPin* EnumParamPin = nullptr;

				if (FBoolProperty* BoolProp = FindFProperty<FBoolProperty>(Function, ParamName))
				{
					EnumParamPin = FindBoolParamPin(*this, ParamName);
				}
				else if (FByteProperty* ByteProp = FindFProperty<FByteProperty>(Function, ParamName))
				{
					Enum = ByteProp->Enum;
					EnumParamPin = FindEnumParamPin(*this, ParamName);
				}
				else if (FEnumProperty* EnumProp = FindFProperty<FEnumProperty>(Function, ParamName))
				{
					Enum = EnumProp->GetEnum();
					EnumParamPin = FindEnumParamPin(*this, ParamName);
				}

				if (Enum && EnumParamPin)
				{
					// Expanded as input execs pins
					if (EnumParamPin->Direction == EGPD_Input)
					{
						if (bAlreadyHandleInput)
						{
							CompilerContext.MessageLog.Error(TEXT("@@ Already provided an input enum parameter for ExpandEnumAsExecs. Only one is permitted."), this);
							return;
						}

						bAlreadyHandleInput = true;

						// Create normal exec input
						UEdGraphPin* ExecutePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

						// Create temp enum variable
						UK2Node_TemporaryVariable* TempEnumVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
						TempEnumVarNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Byte;
						TempEnumVarNode->VariableType.PinSubCategoryObject = Enum;
						TempEnumVarNode->AllocateDefaultPins();
						// Get the output pin
						UEdGraphPin* TempEnumVarOutput = TempEnumVarNode->GetVariablePin();

						// Connect temp enum variable to (hidden) enum pin
						Schema->TryCreateConnection(TempEnumVarOutput, EnumParamPin);

						// Now we want to iterate over other exec inputs...
						for (int32 PinIdx = Pins.Num() - 1; PinIdx >= 0; PinIdx--)
						{
							UEdGraphPin* Pin = Pins[PinIdx];
							if (Pin != NULL &&
								Pin != ExecutePin &&
								Pin->Direction == EGPD_Input &&
								Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
							{
								// Create node to set the temp enum var
								UK2Node_AssignmentStatement* AssignNode = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
								AssignNode->AllocateDefaultPins();

								// Move connections from fake 'enum exec' pint to this assignment node
								CompilerContext.MovePinLinksToIntermediate(*Pin, *AssignNode->GetExecPin());

								// Connect this to out temp enum var
								Schema->TryCreateConnection(AssignNode->GetVariablePin(), TempEnumVarOutput);

								// Connect exec output to 'real' exec pin
								Schema->TryCreateConnection(AssignNode->GetThenPin(), ExecutePin);

								// set the literal enum value to set to
								AssignNode->GetValuePin()->DefaultValue = Pin->PinName.ToString();

								// Finally remove this 'cosmetic' exec pin
								Pins[PinIdx]->MarkAsGarbage();
								Pins.RemoveAt(PinIdx);
							}
						}
					}
					// Expanded as output execs pins
					else if (EnumParamPin->Direction == EGPD_Output)
					{
						// Create a SwitchEnum node to switch on the output enum
						UK2Node_SwitchEnum* SwitchEnumNode = CompilerContext.SpawnIntermediateNode<UK2Node_SwitchEnum>(this, SourceGraph);
						UEnum* EnumObject = Cast<UEnum>(EnumParamPin->PinType.PinSubCategoryObject.Get());
						SwitchEnumNode->SetEnum(EnumObject);
						SwitchEnumNode->AllocateDefaultPins();

						LinkIntoOutputChain(SwitchEnumNode);

						// Connect (hidden) enum pin to switch node's selection pin
						Schema->TryCreateConnection(EnumParamPin, SwitchEnumNode->GetSelectionPin());

						// Now we want to iterate over other exec outputs corresponding to the enum.
						// the first pins created are the ExpandEnumAsExecs pins, and they're all made at the same time.
						for (int32 PinIdx = Enum->NumEnums() - 2; PinIdx >= 0; PinIdx--)
						{
							UEdGraphPin* Pin = Pins[PinIdx];

							if (Pin &&
								Pin != OutMainExecutePin &&
								Pin->Direction == EGPD_Output &&
								Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
							{
								if (UEdGraphPin* FoundPin = SwitchEnumNode->FindPin(Pin->PinName))
								{
									if (!FoundPin->LinkedTo.Contains(Pin))
									{
										// Move connections from fake 'enum exec' pin to this switch node
										CompilerContext.MovePinLinksToIntermediate(*Pin, *FoundPin);

										// Finally remove this 'cosmetic' exec pin
										Pins[PinIdx]->MarkAsGarbage();
										Pins.RemoveAt(PinIdx);
									}
								}
								// Have passed the relevant entries... no more work to do here.
								else
								{
									break;
								}
							}
						}
					}
				}
				else if(EnumParamPin && !EnumParamPin->PinType.IsContainer() && EnumParamPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
				{
					if (EnumParamPin->Direction == EGPD_Input)
					{
						// Create normal exec input
						UEdGraphPin* ExecutePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
						// Create temp bool variable
						UK2Node_TemporaryVariable* TempBoolVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
						TempBoolVarNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
						TempBoolVarNode->AllocateDefaultPins();
						// Get the output pin
						UEdGraphPin* TempBoolVarOutput = TempBoolVarNode->GetVariablePin();
						// Connect temp enum variable to (hidden) bool pin
						Schema->TryCreateConnection(TempBoolVarOutput, EnumParamPin);
						
						// create a true entry and a false:
						const auto CreateAssignNode = [Schema, &CompilerContext, this, SourceGraph, TempBoolVarOutput, ExecutePin](UEdGraphPin* FakePin, const TCHAR* DefaultValue)
						{
							UK2Node_AssignmentStatement* AssignNode = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
							AssignNode->AllocateDefaultPins();

							// Move connections from fake 'enum exec' pint to this assignment node
							CompilerContext.MovePinLinksToIntermediate(*FakePin, *AssignNode->GetExecPin());

							// Connect this to out temp enum var
							Schema->TryCreateConnection(AssignNode->GetVariablePin(), TempBoolVarOutput);

							// Connect exec output to 'real' exec pin
							Schema->TryCreateConnection(AssignNode->GetThenPin(), ExecutePin);

							// set the literal enum value to set to
							AssignNode->GetValuePin()->DefaultValue = DefaultValue;
						};

						UEdGraphPin* TruePin = FindPinChecked(TEXT("True"), EEdGraphPinDirection::EGPD_Input);
						UEdGraphPin* FalsePin = FindPinChecked(TEXT("False"), EEdGraphPinDirection::EGPD_Input);

						CreateAssignNode(TruePin, TEXT("True"));
						CreateAssignNode(FalsePin, TEXT("False"));

						// remove fake false/true nodes:
						RemovePin(TruePin);
						RemovePin(FalsePin);
					}
					else if (EnumParamPin->Direction == EGPD_Output)
					{
						// Create a Branch node to switch on the output bool:
						UK2Node_IfThenElse* IfElseNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
						IfElseNode->AllocateDefaultPins();

						LinkIntoOutputChain(IfElseNode);

						// Connect (hidden) bool pin to branch node
						Schema->TryCreateConnection(EnumParamPin, IfElseNode->GetConditionPin());

						UEdGraphPin* TruePin = FindPinChecked(TEXT("True"), EEdGraphPinDirection::EGPD_Output);
						UEdGraphPin* FalsePin = FindPinChecked(TEXT("False"), EEdGraphPinDirection::EGPD_Output);

						// move true connection to branch node:
						CompilerContext.MovePinLinksToIntermediate(*TruePin, *IfElseNode->GetThenPin());
						// move false connection to branch node:
						CompilerContext.MovePinLinksToIntermediate(*FalsePin, *IfElseNode->GetElsePin());

						// remove fake false/true nodes:
						RemovePin(TruePin);
						RemovePin(FalsePin);
					}
				}
			}
		}
	}

	// AUTO CREATED REFS
	{
		if ( Function )
		{
			TArray<FString> AutoCreateRefTermPinNames;
			CompilerContext.GetSchema()->GetAutoEmitTermParameters(Function, AutoCreateRefTermPinNames);
			const bool bHasAutoCreateRefTerms = AutoCreateRefTermPinNames.Num() != 0;

			for (UEdGraphPin* Pin : Pins)
			{
				const bool bIsRefInputParam = Pin && Pin->PinType.bIsReference && (Pin->Direction == EGPD_Input) && !CompilerContext.GetSchema()->IsMetaPin(*Pin);
				if (!bIsRefInputParam)
				{
					continue;
				}

				const bool bHasConnections = Pin->LinkedTo.Num() > 0;
				const bool bCreateDefaultValRefTerm = bHasAutoCreateRefTerms && 
					!bHasConnections && AutoCreateRefTermPinNames.Contains(Pin->PinName.ToString());

				if (bCreateDefaultValRefTerm)
				{
					const bool bHasDefaultValue = !Pin->DefaultValue.IsEmpty() || Pin->DefaultObject || !Pin->DefaultTextValue.IsEmpty();

					// copy defaults as default values can be reset when the pin is connected
					const FString DefaultValue = Pin->DefaultValue;
					UObject* DefaultObject = Pin->DefaultObject;
					const FText DefaultTextValue = Pin->DefaultTextValue;
					bool bMatchesDefaults = Pin->DoesDefaultValueMatchAutogenerated();

					UEdGraphPin* ValuePin = InnerHandleAutoCreateRef(this, Pin, CompilerContext, SourceGraph, bHasDefaultValue);
					if ( ValuePin )
					{
						if (bMatchesDefaults)
						{
							// Use the latest code to set default value
							Schema->SetPinAutogeneratedDefaultValueBasedOnType(ValuePin);
						}
						else
						{
							ValuePin->DefaultValue = DefaultValue;
							ValuePin->DefaultObject = DefaultObject;
							ValuePin->DefaultTextValue = DefaultTextValue;
						}
					}
				}
				// since EX_Self does not produce an addressable (referenceable) FProperty, we need to shim
				// in a "auto-ref" term in its place (this emulates how UHT generates a local value for 
				// native functions; hence the IsNative() check)
				else if (bHasConnections && Pin->LinkedTo[0]->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self && Pin->PinType.bIsConst && !Function->IsNative())
				{
					InnerHandleAutoCreateRef(this, Pin, CompilerContext, SourceGraph, /*bForceAssignment =*/true);
				}
			}
		}
	}

	// Older assets may have interface pins wired directly to the target pin in the source graph (due to an earlier regression).
	UEdGraphPin* SelfPin = Schema->FindSelfPin(*this, EEdGraphPinDirection::EGPD_Input);
	if (SelfPin && SelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
	{
		for (UEdGraphPin* PinLinkedToSelfPin : SelfPin->LinkedTo)
		{
			if (PinLinkedToSelfPin && PinLinkedToSelfPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface && !PinLinkedToSelfPin->PinType.IsContainer())
			{
				PinLinkedToSelfPin->BreakLinkTo(SelfPin);
				if (!Schema->TryCreateConnection(PinLinkedToSelfPin, SelfPin))
				{
					PinLinkedToSelfPin->MakeLinkTo(SelfPin);
				}
			}
		}
	}

	// Then we go through and expand out array iteration if necessary
	const bool bAllowMultipleSelfs = AllowMultipleSelfs(true);
	if (bAllowMultipleSelfs && SelfPin && !SelfPin->PinType.IsArray())
	{
		const bool bProperInputToExpandForEach =
			(1 == SelfPin->LinkedTo.Num()) &&
			(nullptr != SelfPin->LinkedTo[0]) &&
			(SelfPin->LinkedTo[0]->PinType.IsArray());
		if (bProperInputToExpandForEach)
		{
			CallForEachElementInArrayExpansion(this, SelfPin, CompilerContext, SourceGraph);
		}
	}
}

UEdGraphPin* UK2Node_CallFunction::InnerHandleAutoCreateRef(UK2Node* Node, UEdGraphPin* Pin, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, bool bForceAssignment)
{
	const bool bAddAssigment = !Pin->PinType.IsContainer() && bForceAssignment;

	// ADD LOCAL VARIABLE
	UK2Node_TemporaryVariable* LocalVariable = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(Node, SourceGraph);
	LocalVariable->VariableType = Pin->PinType;
	LocalVariable->VariableType.bIsReference = false;
	LocalVariable->AllocateDefaultPins();
	if (!bAddAssigment)
	{
		if (!CompilerContext.GetSchema()->TryCreateConnection(LocalVariable->GetVariablePin(), Pin))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("AutoCreateRefTermPin_NotConnected", "AutoCreateRefTerm Expansion: Pin @@ cannot be connected to @@").ToString(), LocalVariable->GetVariablePin(), Pin);
			return nullptr;
		}
	}
	// ADD ASSIGMENT
	else
	{
		// TODO connect to dest..
		UK2Node_PureAssignmentStatement* AssignDefaultValue = CompilerContext.SpawnIntermediateNode<UK2Node_PureAssignmentStatement>(Node, SourceGraph);
		AssignDefaultValue->AllocateDefaultPins();
		const bool bVariableConnected = CompilerContext.GetSchema()->TryCreateConnection(AssignDefaultValue->GetVariablePin(), LocalVariable->GetVariablePin());
		UEdGraphPin* AssignInputPit = AssignDefaultValue->GetValuePin();
		const bool bPreviousInputSaved = AssignInputPit && CompilerContext.MovePinLinksToIntermediate(*Pin, *AssignInputPit).CanSafeConnect();
		const bool bOutputConnected = CompilerContext.GetSchema()->TryCreateConnection(AssignDefaultValue->GetOutputPin(), Pin);
		if (!bVariableConnected || !bOutputConnected || !bPreviousInputSaved)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("AutoCreateRefTermPin_AssignmentError", "AutoCreateRefTerm Expansion: Assignment Error @@").ToString(), AssignDefaultValue);
			return nullptr;
		}
		CompilerContext.GetSchema()->SetPinAutogeneratedDefaultValueBasedOnType(AssignDefaultValue->GetValuePin());
		return AssignInputPit;
	}
	return nullptr;
}

void UK2Node_CallFunction::CallForEachElementInArrayExpansion(UK2Node* Node, UEdGraphPin* MultiSelf, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(Node && MultiSelf && SourceGraph && Schema);
	const bool bProperInputToExpandForEach = 
		(1 == MultiSelf->LinkedTo.Num()) && 
		(NULL != MultiSelf->LinkedTo[0]) && 
		(MultiSelf->LinkedTo[0]->PinType.IsArray());
	ensure(bProperInputToExpandForEach);

	UEdGraphPin* ThenPin = Node->FindPinChecked(UEdGraphSchema_K2::PN_Then);

	// Create int Iterator
	UK2Node_TemporaryVariable* IteratorVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(Node, SourceGraph);
	IteratorVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
	IteratorVar->AllocateDefaultPins();

	// Initialize iterator
	UK2Node_AssignmentStatement* InteratorInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
	InteratorInitialize->AllocateDefaultPins();
	InteratorInitialize->GetValuePin()->DefaultValue = TEXT("0");
	Schema->TryCreateConnection(IteratorVar->GetVariablePin(), InteratorInitialize->GetVariablePin());
	CompilerContext.MovePinLinksToIntermediate(*Node->GetExecPin(), *InteratorInitialize->GetExecPin());

	// Do loop branch
	UK2Node_IfThenElse* Branch = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(Node, SourceGraph);
	Branch->AllocateDefaultPins();
	Schema->TryCreateConnection(InteratorInitialize->GetThenPin(), Branch->GetExecPin());
	CompilerContext.MovePinLinksToIntermediate(*ThenPin, *Branch->GetElsePin());

	// Do loop condition
	UK2Node_CallFunction* Condition = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Node, SourceGraph); 
	Condition->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Less_IntInt)));
	Condition->AllocateDefaultPins();
	Schema->TryCreateConnection(Condition->GetReturnValuePin(), Branch->GetConditionPin());
	Schema->TryCreateConnection(Condition->FindPinChecked(TEXT("A")), IteratorVar->GetVariablePin());

	// Array size
	UK2Node_CallArrayFunction* ArrayLength = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(Node, SourceGraph); 
	ArrayLength->SetFromFunction(UKismetArrayLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Length)));
	ArrayLength->AllocateDefaultPins();
	CompilerContext.CopyPinLinksToIntermediate(*MultiSelf, *ArrayLength->GetTargetArrayPin());
	ArrayLength->PinConnectionListChanged(ArrayLength->GetTargetArrayPin());
	Schema->TryCreateConnection(Condition->FindPinChecked(TEXT("B")), ArrayLength->GetReturnValuePin());

	// Get Element
	UK2Node_CallArrayFunction* GetElement = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(Node, SourceGraph); 
	GetElement->SetFromFunction(UKismetArrayLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get)));
	GetElement->AllocateDefaultPins();
	CompilerContext.CopyPinLinksToIntermediate(*MultiSelf, *GetElement->GetTargetArrayPin());
	GetElement->PinConnectionListChanged(GetElement->GetTargetArrayPin());
	Schema->TryCreateConnection(GetElement->FindPinChecked(TEXT("Index")), IteratorVar->GetVariablePin());

	// Iterator increment
	UK2Node_CallFunction* Increment = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Node, SourceGraph); 
	Increment->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Add_IntInt)));
	Increment->AllocateDefaultPins();
	Schema->TryCreateConnection(Increment->FindPinChecked(TEXT("A")), IteratorVar->GetVariablePin());
	Increment->FindPinChecked(TEXT("B"))->DefaultValue = TEXT("1");

	// Iterator assigned
	UK2Node_AssignmentStatement* IteratorAssign = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
	IteratorAssign->AllocateDefaultPins();
	Schema->TryCreateConnection(IteratorAssign->GetVariablePin(), IteratorVar->GetVariablePin());
	Schema->TryCreateConnection(IteratorAssign->GetValuePin(), Increment->GetReturnValuePin());
	Schema->TryCreateConnection(IteratorAssign->GetThenPin(), Branch->GetExecPin());

	// Connect pins from intermediate nodes back in to the original node
	Schema->TryCreateConnection(Branch->GetThenPin(), Node->GetExecPin());
	Schema->TryCreateConnection(ThenPin, IteratorAssign->GetExecPin());
	Schema->TryCreateConnection(GetElement->FindPinChecked(TEXT("Item")), MultiSelf);
}

FName UK2Node_CallFunction::GetCornerIcon() const
{
	if (const UFunction* Function = GetTargetFunction())
	{
		if (Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
		{
			return TEXT("Graph.Replication.AuthorityOnly");		
		}
		else if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic))
		{
			return TEXT("Graph.Replication.ClientEvent");
		}
		else if(Function->HasMetaData(FBlueprintMetadata::MD_Latent))
		{
			return TEXT("Graph.Latent.LatentIcon");
		}
	}
	return Super::GetCornerIcon();
}

FSlateIcon UK2Node_CallFunction::GetIconAndTint(FLinearColor& OutColor) const
{
	return GetPaletteIconForFunction(GetTargetFunction(), OutColor);
}

bool UK2Node_CallFunction::ReconnectPureExecPins(TArray<UEdGraphPin*>& OldPins)
{
	if (IsNodePure())
	{
		// look for an old exec pin
		UEdGraphPin* PinExec = nullptr;
		for (int32 PinIdx = 0; PinIdx < OldPins.Num(); PinIdx++)
		{
			if (OldPins[PinIdx]->PinName == UEdGraphSchema_K2::PN_Execute)
			{
				PinExec = OldPins[PinIdx];
				break;
			}
		}
		if (PinExec)
		{
			PinExec->SetSavePinIfOrphaned(false); 

			// look for old then pin
			UEdGraphPin* PinThen = nullptr;
			for (int32 PinIdx = 0; PinIdx < OldPins.Num(); PinIdx++)
			{
				if (OldPins[PinIdx]->PinName == UEdGraphSchema_K2::PN_Then)
				{
					PinThen = OldPins[PinIdx];
					break;
				}
			}
			if (PinThen)
			{
				PinThen->SetSavePinIfOrphaned(false);

				// reconnect all incoming links to old exec pin to the far end of the old then pin.
				if (PinThen->LinkedTo.Num() > 0)
				{
					UEdGraphPin* PinThenLinked = PinThen->LinkedTo[0];
					while (PinExec->LinkedTo.Num() > 0)
					{
						UEdGraphPin* PinExecLinked = PinExec->LinkedTo[0];
						PinExecLinked->BreakLinkTo(PinExec);
						PinExecLinked->MakeLinkTo(PinThenLinked);
					}
					return true;
				}
			}
		}
	}
	return false;
}

void UK2Node_CallFunction::InvalidatePinTooltips()
{
	bPinTooltipsValid = false;
}

void UK2Node_CallFunction::ConformContainerPins()
{
	// helper functions for type propagation:
	const auto TryReadTypeToPropagate = [](UEdGraphPin* Pin, bool& bOutPropagated, FEdGraphTerminalType& TypeToPropagete)
	{
		if (Pin && !bOutPropagated)
		{
			if (Pin->HasAnyConnections() || !Pin->DoesDefaultValueMatchAutogenerated() )
			{
				FEdGraphTerminalType TypeToPotentiallyPropagate;
				if (Pin->LinkedTo.Num() != 0)
				{
					TypeToPotentiallyPropagate = Pin->LinkedTo[0]->GetPrimaryTerminalType();
				}
				else
				{
					TypeToPotentiallyPropagate = Pin->GetPrimaryTerminalType();
				}
				
				if (!FWildcardNodeUtils::IsWildcardPin(TypeToPotentiallyPropagate))
				{
					bOutPropagated = true;
					TypeToPropagete = TypeToPotentiallyPropagate;
				}
			}
		}
	};

	const auto TryReadValueTypeToPropagate = [](UEdGraphPin* Pin, bool& bOutPropagated, FEdGraphTerminalType& TypeToPropagete)
	{
		if (Pin && !bOutPropagated)
		{
			if (Pin->LinkedTo.Num() != 0 || !Pin->DoesDefaultValueMatchAutogenerated())
			{
				const FEdGraphTerminalType* ValueType = &Pin->PinType.PinValueType;
				if (Pin->LinkedTo.Num() != 0)
				{
					ValueType = &(Pin->LinkedTo[0]->PinType.PinValueType);
				}

				if (!FWildcardNodeUtils::IsWildcardPin(*ValueType))
				{
					bOutPropagated = true;
					TypeToPropagete = *ValueType;
				}
			}
		}
	};
	
	const UEdGraphSchema_K2* Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	const auto TryPropagateType = [Schema](UEdGraphPin* Pin, const FEdGraphTerminalType& TerminalType, bool bTypeIsAvailable)
	{
		if(Pin)
		{
			if(bTypeIsAvailable)
			{
				const FEdGraphTerminalType PrimaryType = Pin->GetPrimaryTerminalType();
				if( PrimaryType.TerminalCategory != TerminalType.TerminalCategory ||
					PrimaryType.TerminalSubCategory != TerminalType.TerminalSubCategory ||
					PrimaryType.TerminalSubCategoryObject != TerminalType.TerminalSubCategoryObject)
				{
					// terminal type changed:
					if (Pin->SubPins.Num() > 0 && Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard)
					{
						Schema->RecombinePin(Pin->SubPins[0]);
					}

					Pin->PinType.PinCategory = TerminalType.TerminalCategory;
					Pin->PinType.PinSubCategory = TerminalType.TerminalSubCategory;
					Pin->PinType.PinSubCategoryObject = TerminalType.TerminalSubCategoryObject;

					// Also propagate the CPF_UObjectWrapper flag, which will be set for "wrapped" object ptr types (e.g. TSubclassOf).
					Pin->PinType.bIsUObjectWrapper = TerminalType.bTerminalIsUObjectWrapper;
				
					// Reset default values
					if (!Schema->IsPinDefaultValid(Pin, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue).IsEmpty())
					{
						Schema->ResetPinToAutogeneratedDefaultValue(Pin, false);
					}
				}
			}
			else
			{
				// reset to wildcard:
				if (Pin->SubPins.Num() > 0)
				{
					Schema->RecombinePin(Pin->SubPins[0]);
				}

				Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				Pin->PinType.PinSubCategory = NAME_None;
				Pin->PinType.PinSubCategoryObject = nullptr;
				Pin->PinType.bIsUObjectWrapper = false;
				Schema->ResetPinToAutogeneratedDefaultValue(Pin, false);
			}
		}
	};

	const auto TryPropagateValueType = [](UEdGraphPin* Pin, const FEdGraphTerminalType& TerminalType, bool bTypeIsAvailable)
	{
		if (Pin)
		{
			if (bTypeIsAvailable)
			{
				Pin->PinType.PinValueType.TerminalCategory = TerminalType.TerminalCategory;
				Pin->PinType.PinValueType.TerminalSubCategory = TerminalType.TerminalSubCategory;
				Pin->PinType.PinValueType.TerminalSubCategoryObject = TerminalType.TerminalSubCategoryObject;
			}
			else
			{
				Pin->PinType.PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;
				Pin->PinType.PinValueType.TerminalSubCategory = NAME_None;
				Pin->PinType.PinValueType.TerminalSubCategoryObject = nullptr;
			}
		}
	};
		
	const UFunction* TargetFunction = GetTargetFunction();
	if (TargetFunction == nullptr)
	{
		return;
	}

	// find any pins marked as SetParam
	const FString& SetPinMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_SetParam);

	// useless copies/allocates in this code, could be an optimization target...
	TArray<FString> SetParamPinGroups;
	{
		SetPinMetaData.ParseIntoArray(SetParamPinGroups, TEXT(","), true);
	}

	for (FString& Entry : SetParamPinGroups)
	{
		// split the group:
		TArray<FString> GroupEntries;
		Entry.ParseIntoArray(GroupEntries, TEXT("|"), true);
		// resolve pins
		TArray<UEdGraphPin*> ResolvedPins;
		for(UEdGraphPin* Pin : Pins)
		{
			if (GroupEntries.Contains(Pin->GetName()))
			{
				ResolvedPins.Add(Pin);
			}
		}

		// if nothing is connected (or non-default), reset to wildcard
		// else, find the first type and propagate to everyone else::
		bool bReadyToPropagatSetType = false;
		FEdGraphTerminalType TypeToPropagate;
		for (UEdGraphPin* Pin : ResolvedPins)
		{
			TryReadTypeToPropagate(Pin, bReadyToPropagatSetType, TypeToPropagate);
			if(bReadyToPropagatSetType)
			{
				break;
			}
		}

		for (UEdGraphPin* Pin : ResolvedPins)
		{
			TryPropagateType( Pin, TypeToPropagate, bReadyToPropagatSetType );
		}
	}

	const FString& MapPinMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_MapParam);
	const FString& MapKeyPinMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_MapKeyParam);
	const FString& MapValuePinMetaData = TargetFunction->GetMetaData(FBlueprintMetadata::MD_MapValueParam);

	if (!MapPinMetaData.IsEmpty() || !MapKeyPinMetaData.IsEmpty() || !MapValuePinMetaData.IsEmpty())
	{
		// if the map pin has a connection infer from that, otherwise use the information on the key param and value param:
		bool bReadyToPropagateKeyType = false;
		FEdGraphTerminalType KeyTypeToPropagate;
		bool bReadyToPropagateValueType = false;
		FEdGraphTerminalType ValueTypeToPropagate;

		UEdGraphPin* MapPin = MapPinMetaData.IsEmpty() ? nullptr : FindPin(MapPinMetaData);
		UEdGraphPin* MapKeyPin = MapKeyPinMetaData.IsEmpty() ? nullptr : FindPin(MapKeyPinMetaData);
		UEdGraphPin* MapValuePin = MapValuePinMetaData.IsEmpty() ? nullptr : FindPin(MapValuePinMetaData);

		TryReadTypeToPropagate(MapPin, bReadyToPropagateKeyType, KeyTypeToPropagate);
		TryReadValueTypeToPropagate(MapPin, bReadyToPropagateValueType, ValueTypeToPropagate);
		TryReadTypeToPropagate(MapKeyPin, bReadyToPropagateKeyType, KeyTypeToPropagate);
		TryReadTypeToPropagate(MapValuePin, bReadyToPropagateValueType, ValueTypeToPropagate);

		TryPropagateType(MapPin, KeyTypeToPropagate, bReadyToPropagateKeyType);
		TryPropagateType(MapKeyPin, KeyTypeToPropagate, bReadyToPropagateKeyType);

		TryPropagateValueType(MapPin, ValueTypeToPropagate, bReadyToPropagateValueType);
		TryPropagateType(MapValuePin, ValueTypeToPropagate, bReadyToPropagateValueType);
	}
}

bool UK2Node_CallFunction::AreExecPinsVisible() const
{
	const bool bAreExecPinsVisible =
		(NodePurityOverride == ENodePurityOverride::Impure) ||
		(!bDefaultsToPureFunc && NodePurityOverride != ENodePurityOverride::Pure)
	;

	return bAreExecPinsVisible;
}

bool UK2Node_CallFunction::FunctionHasOutputs() const
{
	bool bCanToggle = false;

	const UFunction* Function = GetTargetFunction();
	if (ensure(Function))
	{
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			bCanToggle =
				PropIt->HasAnyPropertyFlags(CPF_ReturnParm) ||
				PropIt->HasAnyPropertyFlags(CPF_OutParm)
				;

			if (bCanToggle)
			{
				break;
			}
		}
	}

	return bCanToggle;
}

void UK2Node_CallFunction::ToggleNodePurityOverride()
{
	const FText TransactionTitle = AreExecPinsVisible() ? LOCTEXT("HideExecPins", "Hide Exec pins") : LOCTEXT("ShowExecPins", "Show Exec pins");
	
	FScopedTransaction Transaction(TransactionTitle);
	Modify();

	switch (NodePurityOverride)
	{
	case ENodePurityOverride::Unset:
		NodePurityOverride = bDefaultsToPureFunc ? ENodePurityOverride::Impure : ENodePurityOverride::Pure;
		break;

	case ENodePurityOverride::Pure:
		NodePurityOverride = ENodePurityOverride::Impure;
		break;

	case ENodePurityOverride::Impure:
		NodePurityOverride = ENodePurityOverride::Pure;
		break;

	default:
		checkNoEntry();
	}
	
	ReconstructNode();
}

FText UK2Node_CallFunction::GetToolTipHeading() const
{
	FText Heading = Super::GetToolTipHeading();

	struct FHeadingBuilder
	{
		FHeadingBuilder(FText InitialHeading) : ConstructedHeading(InitialHeading) {}

		void Append(FText HeadingAddOn)
		{
			if (ConstructedHeading.IsEmpty())
			{
				ConstructedHeading = HeadingAddOn;
			}
			else 
			{
				ConstructedHeading = FText::Format(FText::FromString("{0}\n{1}"), HeadingAddOn, ConstructedHeading);
			}
		}

		FText ConstructedHeading;
	};
	FHeadingBuilder HeadingBuilder(Super::GetToolTipHeading());

	if (const UFunction* Function = GetTargetFunction())
	{
		if (Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
		{
			HeadingBuilder.Append(LOCTEXT("ServerOnlyFunc", "Server Only"));	
		}
		if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic))
		{
			HeadingBuilder.Append(LOCTEXT("ClientOnlyFunc", "Client Only"));
		}
		if(Function->HasMetaData(FBlueprintMetadata::MD_Latent))
		{
			HeadingBuilder.Append(LOCTEXT("LatentFunc", "Latent"));
		}
	}

	return HeadingBuilder.ConstructedHeading;
}

void UK2Node_CallFunction::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	UFunction* TargetFunction = GetTargetFunction();
	const FString TargetFunctionName = TargetFunction ? TargetFunction->GetName() : TEXT( "InvalidFunction" );
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "Function" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), TargetFunctionName ));
}

FText UK2Node_CallFunction::GetMenuCategory() const
{
	UFunction* TargetFunction = GetTargetFunction();
	if (TargetFunction != nullptr)
	{
		return GetDefaultCategoryForFunction(TargetFunction, FText::GetEmpty());
	}
	return FText::GetEmpty();
}

bool UK2Node_CallFunction::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UFunction* Function = GetTargetFunction();
	const UClass* SourceClass = Function ? Function->GetOwnerClass() : nullptr;
	const UBlueprint* SourceBlueprint = GetBlueprint();
	bool bResult = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(Function);
	}

	// All structures, that are required for the BP compilation, should be gathered
	for (UEdGraphPin* Pin : Pins)
	{
		UStruct* DepStruct = Pin ? Cast<UStruct>(Pin->PinType.PinSubCategoryObject.Get()) : nullptr;

		UClass* DepClass = Cast<UClass>(DepStruct);
		if (DepClass && (DepClass->ClassGeneratedBy.Get() == SourceBlueprint))
		{
			//Don't include self
			continue;
		}

		if (DepStruct && !DepStruct->IsNative())
		{
			if (OptionalOutput)
			{
				OptionalOutput->AddUnique(DepStruct);
			}
			bResult = true;
		}
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

bool UK2Node_CallFunction::IsLatentForMacros() const
{
	UFunction* TargetFunction = GetTargetFunction();
	return (TargetFunction && TargetFunction->HasMetaData(FBlueprintMetadata::MD_Latent));
}

UEdGraph* UK2Node_CallFunction::GetFunctionGraph(const UEdGraphNode*& OutGraphNode) const
{
	OutGraphNode = nullptr;

	// Search for the Blueprint owner of the function graph, climbing up through the Blueprint hierarchy
	UClass* MemberParentClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
	if(MemberParentClass != nullptr)
	{
		UBlueprintGeneratedClass* ParentClass = Cast<UBlueprintGeneratedClass>(MemberParentClass);
		if(ParentClass != nullptr && ParentClass->ClassGeneratedBy != nullptr)
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(ParentClass->ClassGeneratedBy);
			while(Blueprint != nullptr)
			{
				UEdGraph* TargetGraph = nullptr;
				const FName FunctionName = FunctionReference.GetMemberName();
				for (UEdGraph* const Graph : Blueprint->FunctionGraphs) 
				{
					if (Graph->GetFName() == FunctionName)
					{
						TargetGraph = Graph;
						break;
					}
				}

				if (!TargetGraph)
				{
					for (const FBPInterfaceDescription& Interface : Blueprint->ImplementedInterfaces)
					{
						for (UEdGraph* const Graph : Interface.Graphs)
						{
							if (Graph->GetFName() == FunctionName)
							{
								TargetGraph = Graph;
								break;
							}
						}

						if (TargetGraph)
						{
							break;
						}
					}
				}

				if((TargetGraph != nullptr) && !TargetGraph->HasAnyFlags(RF_Transient))
				{
					// Found the function graph in a Blueprint, return that graph
					return TargetGraph;
				}
				else
				{
					// Did not find the function call as a graph, it may be a custom event
					UK2Node_CustomEvent* CustomEventNode = nullptr;

					TArray<UK2Node_CustomEvent*> CustomEventNodes;
					FBlueprintEditorUtils::GetAllNodesOfClass(Blueprint, CustomEventNodes);

					for (UK2Node_CustomEvent* const CustomEvent : CustomEventNodes)
					{
						if(CustomEvent->CustomFunctionName == FunctionReference.GetMemberName())
						{
							OutGraphNode = CustomEvent;
							return CustomEvent->GetGraph();
						}
					}
				}

				ParentClass = Cast<UBlueprintGeneratedClass>(Blueprint->ParentClass);
				Blueprint = ParentClass != nullptr ? Cast<UBlueprint>(ParentClass->ClassGeneratedBy) : nullptr;
			}
		}
	}
	return nullptr;
}

bool UK2Node_CallFunction::IsStructureWildcardProperty(const UFunction* Function, const FName PropertyName)
{
	if (Function && !PropertyName.IsNone())
	{
		TArray<FString> Names;
		FCustomStructureParamHelper::FillCustomStructureParameterNames(Function, Names);
		if (Names.Contains(PropertyName.ToString()))
		{
			return true;
		}
	}
	return false;
}

bool UK2Node_CallFunction::IsWildcardProperty(const UFunction* InFunction, const FProperty* InProperty)
{
	if (InProperty)
	{
		return FEdGraphUtilities::IsSetParam(InFunction, InProperty->GetFName()) || FEdGraphUtilities::IsMapParam(InFunction, InProperty->GetFName());
	}
	return false;
}

void UK2Node_CallFunction::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	if (UFunction* TargetFunction = GetTargetFunction())
	{
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(TargetFunction->GetName())));
	}
}

void UK2Node_CallFunction::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);

	// Blueprint graphs that call a function declared in the same blueprint don't store a target type, but rather PinSubCategory == Self.
	// When this is the case, we will still explicitly index the ObjectClass for the target pin, so that it can be treated the same as 
	// any other call function nodes.
	if (Pin->PinName == UEdGraphSchema_K2::PN_Self && Pin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self && !Pin->PinType.PinSubCategoryObject.IsValid())
	{
		// Get the parent or interface class that originally defined this function
		if (const UClass* FuncOriginClass = FindInBlueprintsHelpers::GetFunctionOriginClass(GetTargetFunction()))
		{
			const FString FuncOriginClassName = FuncOriginClass->GetPathName();
			OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_ObjectClass, FText::FromString(FuncOriginClassName)));
		}
	}
}

TSharedPtr<SWidget> UK2Node_CallFunction::CreateNodeImage() const
{
	// For set, map and array functions we have a cool icon. This helps users quickly
	// identify container types:
	if (UFunction* TargetFunction = GetTargetFunction())
	{
		UEdGraphPin* NodeImagePin = FEdGraphUtilities::FindArrayParamPin(TargetFunction, this);
		NodeImagePin = NodeImagePin ? NodeImagePin : FEdGraphUtilities::FindSetParamPin(TargetFunction, this);
		NodeImagePin = NodeImagePin ? NodeImagePin : FEdGraphUtilities::FindMapParamPin(TargetFunction, this);
		if(NodeImagePin)
		{
			// Find the first array param pin and bind that to our array image:
			return SPinTypeSelector::ConstructPinTypeImage(NodeImagePin);
		}
	}

	return TSharedPtr<SWidget>();
}

UObject* UK2Node_CallFunction::GetJumpTargetForDoubleClick() const
{
	// If there is an event node, jump to it, otherwise jump to the function graph
	const UEdGraphNode* ResultEventNode = nullptr;
	UEdGraph* FunctionGraph = GetFunctionGraph(/*out*/ ResultEventNode);
	if (ResultEventNode != nullptr)
	{
		return const_cast<UEdGraphNode*>(ResultEventNode);
	}
	else
	{
		return FunctionGraph;
	}
}

bool UK2Node_CallFunction::CanJumpToDefinition() const
{
	const UFunction* TargetFunction = GetTargetFunction();
	const bool bNativeFunction = (TargetFunction != nullptr) && (TargetFunction->IsNative());
	const bool bCanJumpToNativeFunction = bNativeFunction && ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed();
	return bCanJumpToNativeFunction || (GetJumpTargetForDoubleClick() != nullptr);
}

void UK2Node_CallFunction::JumpToDefinition() const
{
	if (ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		// For native functions, try going to the function definition in C++ if available
		if (UFunction* TargetFunction = GetTargetFunction())
		{
			if (TargetFunction->IsNative())
			{
				// First try the nice way that will get to the right line number
				bool bSucceeded = false;
				const bool bNavigateToNativeFunctions = GetDefault<UBlueprintEditorSettings>()->bNavigateToNativeFunctionsFromCallNodes;

				if (bNavigateToNativeFunctions)
				{
					if (FSourceCodeNavigation::CanNavigateToFunction(TargetFunction))
					{
						bSucceeded = FSourceCodeNavigation::NavigateToFunction(TargetFunction);
					}

					// Failing that, fall back to the older method which will still get the file open assuming it exists
					if (!bSucceeded)
					{
						FString NativeParentClassHeaderPath;
						const bool bFileFound = FSourceCodeNavigation::FindClassHeaderPath(TargetFunction, NativeParentClassHeaderPath) && (IFileManager::Get().FileSize(*NativeParentClassHeaderPath) != INDEX_NONE);
						if (bFileFound)
						{
							const FString AbsNativeParentClassHeaderPath = FPaths::ConvertRelativePathToFull(NativeParentClassHeaderPath);
							FSourceCodeNavigation::OpenSourceFileAsync(AbsNativeParentClassHeaderPath);
						}
					}
				}
				else
				{
					// Inform user that the function is native, give them opportunity to enable navigation to native
					// functions:
					FNotificationInfo Info(LOCTEXT("NavigateToNativeDisabled", "Navigation to Native (c++) Functions Disabled"));
					Info.ExpireDuration = 10.0f;
					Info.CheckBoxState = bNavigateToNativeFunctions ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;

					Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateStatic(
						[](ECheckBoxState NewState)
						{
							const FScopedTransaction Transaction(LOCTEXT("ChangeNavigateToNativeFunctionsFromCallNodes", "Change Navigate to Native Functions from Call Nodes Setting"));

							UBlueprintEditorSettings* MutableEditorSetings = GetMutableDefault<UBlueprintEditorSettings>();
							MutableEditorSetings->Modify();
							MutableEditorSetings->bNavigateToNativeFunctionsFromCallNodes = (NewState == ECheckBoxState::Checked) ? true : false;
							MutableEditorSetings->SaveConfig();
						}
					);
					Info.CheckBoxText = LOCTEXT("EnableNavigationToNative", "Navigate to Native Functions from Blueprint Call Nodes?");

					FSlateNotificationManager::Get().AddNotification(Info);
				}

				return;
			}
		}
	}

	// Otherwise, fall back to the inherited behavior which should go to the function entry node
	Super::JumpToDefinition();
}

void UK2Node_CallFunction::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	FToolMenuSection& Section = Menu->AddSection("K2NodeCallFunction", LOCTEXT("FunctionHeader", "Function"));

	if (HasDeprecatedReference())
	{
		const FText MenuEntryTitle = LOCTEXT("SuppressFunctionDeprecationWarningTitle", "Suppress Deprecation Warning");
		const FText MenuEntryTooltip = LOCTEXT("SuppressFunctionDeprecationWarningTooltip", "Adds this function to the suppressed deprecation warnings list in the Bluperint Editor Project Settings for this project.");

		Section.AddMenuEntry(
			"SuppressDeprecationWarning",
			MenuEntryTitle,
			MenuEntryTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(this, &UK2Node_CallFunction::SuppressDeprecationWarning),
				FCanExecuteAction::CreateUObject(this, &UK2Node_CallFunction::HasDeprecatedReference),
				FIsActionChecked()
			)
		);
	}

	if (CanToggleNodePurity() && !Context->bIsDebugging)
	{
		const FText HideExecPinsTitle = LOCTEXT("HideExecPins", "Hide Exec pins");
		const FText ShowExecPinsTitle = LOCTEXT("ShowExecPins", "Show Exec pins");
		const FText HideExecPinsTooltip = LOCTEXT("HideExecPinsTooltip", "Hide the node's Exec pins. The node will be Pure and will execute once for each node connected to an output pin.");
		const FText ShowExecPinsTooltip = LOCTEXT("ShowExecPinsTooltip", "Show the node's Exec pins. The node will not be Pure and will have execution pins that must be connected to the graph. The outputs may also be cached for later use.");

		const bool bAreExecPinsVisible = AreExecPinsVisible();
		const FText& MenuEntryTitle = bAreExecPinsVisible ? HideExecPinsTitle : ShowExecPinsTitle;
		const FText& MenuEntryTooltip = bAreExecPinsVisible ? HideExecPinsTooltip : ShowExecPinsTooltip;

		Section.AddMenuEntry(
			TEXT("ToggleExecPins"),
			MenuEntryTitle,
			MenuEntryTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_CallFunction*>(this), &UK2Node_CallFunction::ToggleNodePurityOverride),
				FCanExecuteAction(),
				FIsActionChecked()
			)
		);
	}
}

FString UK2Node_CallFunction::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData = Super::GetPinMetaData(InPinName, InKey);

	// If there's no metadata directly on the pin then check for metadata on the function
	if (MetaData.IsEmpty())
	{
		if (UFunction* Function = GetTargetFunction())
		{
			// Find the corresponding property for the pin and search that first
			if (FProperty* Property = Function->FindPropertyByName(InPinName))
			{
				MetaData = Property->GetMetaData(InKey);
			}

			// Also look for metadata like DefaultToSelf on the function itself
			if (MetaData.IsEmpty())
			{
				MetaData = Function->GetMetaData(InKey);
				if (MetaData != InPinName.ToString())
				{
					// Only return if the value matches the pin name as we don't want general function metadata
					MetaData.Empty();
				}
			}
		}
	}

	return MetaData;
}

bool UK2Node_CallFunction::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	bool bIsDisallowed = Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
	if (!bIsDisallowed && MyPin != nullptr)
	{
		if (MyPin->bNotConnectable)
		{
			bIsDisallowed = true;
			OutReason = LOCTEXT("PinConnectionDisallowed", "This parameter is for internal use only.").ToString();
		}
		else if (UFunction* TargetFunction = GetTargetFunction())
		{
			const bool bIsObjectType = (MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
				MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject) &&
				(OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
				OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject);

			if (// Strictly speaking this first check is not needed, but by not disabling the connection here we get a better reason later:
				(	OtherPin->PinType.IsContainer() 
					// make sure we don't allow connections of mismatched container types (e.g. maps to arrays)
					&& (OtherPin->PinType.ContainerType != MyPin->PinType.ContainerType)
					&& (
						(FEdGraphUtilities::IsSetParam(TargetFunction, MyPin->PinName) && !MyPin->PinType.IsSet()) ||
						(FEdGraphUtilities::IsMapParam(TargetFunction, MyPin->PinName) && !MyPin->PinType.IsMap()) ||
						(FEdGraphUtilities::IsArrayDependentParam(TargetFunction, MyPin->PinName) && !MyPin->PinType.IsArray())
					)
				)
			)
			{
				bIsDisallowed = true;
				OutReason = LOCTEXT("PinSetConnectionDisallowed", "Containers of containers are not supported - consider wrapping a container in a Structure object").ToString();
			}
			// Do not allow exec pins to be connected to a wildcard if this is a container function
			else if(MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard && OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
			{
				bIsDisallowed = true;
				OutReason = LOCTEXT("PinExecConnectionDisallowed", "Cannot create a container of Exec pins.").ToString();
			}
			else if (bIsObjectType && MyPin->Direction == EGPD_Input && MyPin->PinType.IsContainer() && OtherPin->PinType.IsContainer())
			{
				// Check that we can actually connect the dependent pins to this new array
				const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(GetSchema());

				// Gather all pins that would be dependent on on the container type
				TArray<UEdGraphPin*> DependentPins;
				{
					for (UEdGraphPin* Pin : Pins)
					{
						if (Pin->Direction == EGPD_Input && Pin != MyPin && FEdGraphUtilities::IsDynamicContainerParam(TargetFunction, Pin->PinName))
						{
							DependentPins.Add(Pin);
						}
					}
				}

				for (UEdGraphPin* Pin : DependentPins)
				{
					// If the pins are both containers, then ArePinTypesCompatible will fail incorrectly.
					if (OtherPin->PinType.ContainerType != Pin->PinType.ContainerType)
					{
						continue;
					}

					UClass* Context = nullptr;
					UBlueprint* Blueprint = GetBlueprint();
					if (Blueprint)
					{
						Context = Blueprint->GeneratedClass;
					}

					const bool ConnectResponse = K2Schema->ArePinTypesCompatible(Pin->PinType, OtherPin->PinType, Context, /* bIgnoreArray = */ true);

					if (!ConnectResponse)
					{
						// For sets, we have to check if the other pin is a valid child that can actually 
						// be connected in cases like the "Union" function
						UStruct const* OutputObject = (OtherPin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self) ? Context : Cast<UStruct>(OtherPin->PinType.PinSubCategoryObject.Get());
						UStruct const* InputObject = (Pin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self) ? Context : Cast<UStruct>(Pin->PinType.PinSubCategoryObject.Get());

						if (OtherPin->PinType.IsSet() && OutputObject && InputObject && OutputObject->IsChildOf(InputObject))
						{
							bIsDisallowed = false;
						}
						else
						{
							// Display the necessary tooltip on the pin hover, and log it if we are compiling
							FFormatNamedArguments MessageArgs;
							MessageArgs.Add(TEXT("PinAType"), UEdGraphSchema_K2::TypeToText(Pin->PinType));
							MessageArgs.Add(TEXT("PinBType"), UEdGraphSchema_K2::TypeToText(OtherPin->PinType));
							UBlueprint* BP = GetBlueprint();
							UEdGraph* OwningGraph = GetGraph();

							OutReason = FText::Format(LOCTEXT("DefaultPinIncompatibilityMessage", "{PinAType} is not compatible with {PinBType}."), MessageArgs).ToString();
							return true;
						}
					}
				}
			}
		}
	}

	return bIsDisallowed;
}

#undef LOCTEXT_NAMESPACE
