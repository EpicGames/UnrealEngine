// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_CallFunctionOnMember.h"

#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallFunctionOnMember)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_CallFunctionOnMember::UK2Node_CallFunctionOnMember(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

UEdGraphPin* UK2Node_CallFunctionOnMember::CreateSelfPin(const UFunction* Function) 
{
	UEdGraphPin* SelfPin = nullptr;
	if (MemberVariableToCallOn.IsSelfContext())
	{
		// This means the function is defined within the blueprint, so the pin should be a true "self" pin
		SelfPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UEdGraphSchema_K2::PSC_Self, UEdGraphSchema_K2::PN_Self);
	}
	else
	{
		// This means that the function is declared in an external class, and should reference that class
		SelfPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, MemberVariableToCallOn.GetMemberParentClass(GetBlueprintClassFromNode()), UEdGraphSchema_K2::PN_Self);
	}
	check(SelfPin);

	return SelfPin;
}

FText UK2Node_CallFunctionOnMember::GetFunctionContextString() const
{
	UClass* MemberVarClass = MemberVariableToCallOn.GetMemberParentClass(GetBlueprintClassFromNode());
	FText CallFunctionClassName = (MemberVarClass != NULL) ? MemberVarClass->GetDisplayNameText() : FText::GetEmpty();
	FFormatNamedArguments Args;
	Args.Add(TEXT("TargetName"), CallFunctionClassName);
	Args.Add(TEXT("MemberVariableName"), FText::FromName(MemberVariableToCallOn.GetMemberName()));
	return FText::Format(LOCTEXT("CallFunctionOnMemberDifferentContext", "Target is {TargetName} ({MemberVariableName})"), Args);
}

FNodeHandlingFunctor* UK2Node_CallFunctionOnMember::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_CallFunctionOnMember::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// This skips UK2Node_CallFunction::ExpandNode.  Instead it spawns a new CallFunction node and does hookup that this is interested in,
	// and then that CallFunction node will get its own Expansion to handle the parent portions
	UK2Node::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	UFunction* Function = GetTargetFunction();

	// Create real 'call function' node.
	UK2Node_CallFunction* CallFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallFuncNode->SetFromFunction(Function);
	CallFuncNode->AllocateDefaultPins();
	UEdGraphPin* CallFuncSelfPin = Schema->FindSelfPin(*CallFuncNode, EGPD_Input);

	// Now because you can wire multiple variables to a self pin, need to iterate over each one and create a 'get var' node for each
	UEdGraphPin* SelfPin = Schema->FindSelfPin(*this, EGPD_Input);
	if(SelfPin != NULL)
	{
		if (SelfPin->LinkedTo.Num() == 0)
		{
			UK2Node_VariableGet* GetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, SourceGraph);
			GetVarNode->VariableReference.SetSelfMember(MemberVariableToCallOn.GetMemberName());
			GetVarNode->AllocateDefaultPins();

			if (UEdGraphPin* ValuePin = GetVarNode->GetValuePin())
			{
				ValuePin->MakeLinkTo(CallFuncSelfPin);
			}
		}
		else
		{
			for (int32 TargetIdx = 0; TargetIdx < SelfPin->LinkedTo.Num(); TargetIdx++)
			{
				UEdGraphPin* SourcePin = SelfPin->LinkedTo[TargetIdx];
				if (SourcePin != NULL)
				{
					// Create 'get var' node to get the member
					UK2Node_VariableGet* GetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, SourceGraph);
					GetVarNode->VariableReference = MemberVariableToCallOn;
					GetVarNode->AllocateDefaultPins();

					UEdGraphPin* VarNodeSelfPin = Schema->FindSelfPin(*GetVarNode, EGPD_Input);
					if (VarNodeSelfPin != NULL)
					{
						VarNodeSelfPin->MakeLinkTo(SourcePin);

						UEdGraphPin* ValuePin = GetVarNode->GetValuePin();
						ValuePin->MakeLinkTo(CallFuncSelfPin);
					}
					else
					{
						// Failed to find the member to call on for this expansion, so warn about it
						CompilerContext.MessageLog.Warning(*LOCTEXT("CallFunctionOnInvalidMember_Warning", "Function node @@ called on invalid target member.").ToString(), this);
					}
				}
			}
		}			
	}

	// Now move the rest of the connections (including exec connections...)
	for(int32 SrcPinIdx=0; SrcPinIdx<Pins.Num(); SrcPinIdx++)
	{
		UEdGraphPin* SrcPin = Pins[SrcPinIdx];
		if(SrcPin != NULL && SrcPin != SelfPin) // check its not the self pin
		{
			UEdGraphPin* DestPin = CallFuncNode->FindPin(SrcPin->PinName);
			if(DestPin != NULL)
			{
				CompilerContext.MovePinLinksToIntermediate(*SrcPin, *DestPin); // Source node is assumed to be owner...
			}
		}
	}

	// Finally, break any remaining links on the 'call func on member' node
	BreakAllNodeLinks();
}

bool UK2Node_CallFunctionOnMember::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	auto VarProperty = MemberVariableToCallOn.ResolveMember<FProperty>(GetBlueprintClassFromNode());
	UClass* SourceClass = VarProperty ? VarProperty->GetOwnerClass() : nullptr;
	const bool bResult = (SourceClass != NULL) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

#undef LOCTEXT_NAMESPACE
