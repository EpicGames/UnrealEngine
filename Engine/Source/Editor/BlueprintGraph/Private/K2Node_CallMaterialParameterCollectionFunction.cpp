// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CallMaterialParameterCollectionFunction.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/Platform.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Materials/MaterialParameterCollection.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallMaterialParameterCollectionFunction)

UK2Node_CallMaterialParameterCollectionFunction::UK2Node_CallMaterialParameterCollectionFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_CallMaterialParameterCollectionFunction::PinDefaultValueChanged(UEdGraphPin* Pin) 
{
	Super::PinDefaultValueChanged(Pin);

	if (Pin->PinName == TEXT("Collection") )
	{
		// When the Collection pin gets a new value assigned, we need to update the Slate UI so that SGraphNodeCallParameterCollectionFunction will update the ParameterName drop down
		GetGraph()->NotifyGraphChanged();
	}
}

void UK2Node_CallMaterialParameterCollectionFunction::PreloadRequiredAssets()
{
	if (UEdGraphPin* CollectionPin = FindPin(TEXT("Collection")))
	{
		if (UMaterialParameterCollection* Collection = Cast<UMaterialParameterCollection>(CollectionPin->DefaultObject))
		{
			PreloadObject(Collection);

			// When on async loading thread, postload happens later on GT unless it's possible
			// to do it right now safely.
			if (IsInGameThread() || Collection->IsPostLoadThreadSafe())
			{
				Collection->ConditionalPostLoad();
			}
		}
	}

	Super::PreloadRequiredAssets();
}

void UK2Node_CallMaterialParameterCollectionFunction::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (UEdGraphPin* ParameterNamePin = FindPin(TEXT("ParameterName")))
	{
		if (ParameterNamePin->LinkedTo.Num() == 0)
		{
			if (ParameterNamePin->DefaultValue == TEXT(""))
			{
				MessageLog.Error(TEXT("@@ is invalid, @@ needs to be set to a parameter."), this, ParameterNamePin);
			}
			else
			{
				if (UEdGraphPin* CollectionPin = FindPin(TEXT("Collection")))
				{
					if (UMaterialParameterCollection* Collection = Cast<UMaterialParameterCollection>(CollectionPin->DefaultObject))
					{
						const FGuid ParameterNameGuid = Collection->GetParameterId(*ParameterNamePin->DefaultValue);
						if (!ParameterNameGuid.IsValid())
						{
							MessageLog.Error(TEXT("@@ is invalid, @@ needs to be set to a parameter."), this, ParameterNamePin);
						}
					}
				}
			}
		}
	}
}
