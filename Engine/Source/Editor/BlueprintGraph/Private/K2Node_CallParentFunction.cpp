// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CallParentFunction.h"

#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GraphEditorSettings.h"
#include "FindInBlueprintManager.h"
#include "FindInBlueprints.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Misc/Guid.h"
#include "ObjectTools.h"
#include "Settings/EditorStyleSettings.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Script.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CallParentFunction)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_CallParentFunction::UK2Node_CallParentFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UK2Node_CallParentFunction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	FText FunctionName;

	if (Function)
	{
		FunctionName = ObjectTools::GetUserFacingFunctionName( Function );
	}
	else if ( GEditor && GetDefault<UEditorStyleSettings>()->bShowFriendlyNames )
	{
		FunctionName = FText::FromString(FName::NameToDisplayString(FunctionReference.GetMemberName().ToString(), false));
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("FunctionName"), FunctionName);
	return FText::Format( LOCTEXT( "CallSuperFunction", "Parent: {FunctionName}" ), Args);
}

FLinearColor UK2Node_CallParentFunction::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ParentFunctionCallNodeTitleColor;
}

void UK2Node_CallParentFunction::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* SelfPin = Schema->FindSelfPin(*this, EGPD_Input);
	if( SelfPin )
	{
		SelfPin->bHidden = true;
	}
}

void UK2Node_CallParentFunction::SetFromFunction(const UFunction* Function)
{
	if (Function != nullptr)
	{
		bDefaultsToPureFunc = Function->HasAnyFunctionFlags(FUNC_BlueprintPure);

		UClass* OwnerClass = Function->GetOwnerClass();

		FGuid FunctionGuid;
		if (OwnerClass != nullptr)
		{
			OwnerClass = OwnerClass->GetAuthoritativeClass();
			UBlueprint::GetGuidFromClassByFieldName<UFunction>(OwnerClass, Function->GetFName(), FunctionGuid);
		}

		const bool bIsConsideredSelfContext = false;
		FunctionReference.SetDirect(Function->GetFName(), FunctionGuid, OwnerClass, bIsConsideredSelfContext);
	}
}

void UK2Node_CallParentFunction::FixupSelfMemberContext()
{
	// Do nothing. We want the context to continue to be our parent class.
}

void UK2Node_CallParentFunction::PostPlacedNewNode()
{
	// We don't want to check if our function exists in the current scope

	UK2Node::PostPlacedNewNode();
}

void UK2Node_CallParentFunction::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	// CallParentFunction nodes will always target the direct parent class, unlike CallFunction nodes.
	// However, we still want the function to be searchable using the ancestor class or interface that declares it.
	// Therefore, index a FuncOriginClass metadata value and don't rely on the target pin's type.
	if (const UClass* FuncOriginClass = FindInBlueprintsHelpers::GetFunctionOriginClass(GetTargetFunction()))
	{
		const FString FuncOriginClassName = FuncOriginClass->GetPathName();
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_FuncOriginClass, FText::FromString(FuncOriginClassName)));
	}
}

#undef LOCTEXT_NAMESPACE
