// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_CastByteToEnum.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFieldNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetNodeHelperLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Field.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CastByteToEnum)

class FString;
class UBlueprintNodeSpawner;
struct FBPTerminal;
struct FLinearColor;

const FName UK2Node_CastByteToEnum::ByteInputPinName(TEXT("Byte"));

UK2Node_CastByteToEnum::UK2Node_CastByteToEnum(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_CastByteToEnum::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
	if (!Enum)
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "CastByteToNullEnumError", "Undefined Enum in @@").ToString(), this);
	}
}

void UK2Node_CastByteToEnum::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, ByteInputPinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Byte, Enum, UEdGraphSchema_K2::PN_ReturnValue);
}

FText UK2Node_CastByteToEnum::GetTooltipText() const
{
	if (Enum == nullptr)
	{
		return NSLOCTEXT("K2Node", "CastByteToEnum_NullTooltip", "Byte to Enum (bad enum)");
	}
	else if(CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(
			NSLOCTEXT("K2Node", "CastByteToEnum_Tooltip", "Byte to Enum {0}"),
			FText::FromName(Enum->GetFName())
		), this);
	}
	return CachedTooltip;
}

FText UK2Node_CastByteToEnum::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetTooltipText();
}

FSlateIcon UK2Node_CastByteToEnum::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x");
	return Icon;
}

FText UK2Node_CastByteToEnum::GetCompactNodeTitle() const
{
	return NSLOCTEXT("K2Node", "CastSymbol", "\u2022");
}

FName UK2Node_CastByteToEnum::GetFunctionName() const
{
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, GetValidValue);
	return FunctionName;
}

void UK2Node_CastByteToEnum::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (bSafe && Enum)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

		// FUNCTION NODE
		const FName FunctionName = GetFunctionName();
		const UFunction* Function = UKismetNodeHelperLibrary::StaticClass()->FindFunctionByName(FunctionName);
		check(NULL != Function);
		UK2Node_CallFunction* CallValidation = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph); 
		CallValidation->SetFromFunction(Function);
		CallValidation->AllocateDefaultPins();
		check(CallValidation->IsNodePure());

		// FUNCTION ENUM PIN
		UEdGraphPin* FunctionEnumPin = CallValidation->FindPinChecked(TEXT("Enum"));
		Schema->TrySetDefaultObject(*FunctionEnumPin, Enum);
		check(FunctionEnumPin->DefaultObject == Enum);

		// FUNCTION INPUT BYTE PIN
		UEdGraphPin* OrgInputPin = FindPinChecked(ByteInputPinName);
		UEdGraphPin* FunctionIndexPin = CallValidation->FindPinChecked(TEXT("EnumeratorValue"));
		check(EGPD_Input == FunctionIndexPin->Direction && UEdGraphSchema_K2::PC_Byte == FunctionIndexPin->PinType.PinCategory);
		CompilerContext.MovePinLinksToIntermediate(*OrgInputPin, *FunctionIndexPin);

		// UNSAFE CAST NODE
		UK2Node_CastByteToEnum* UsafeCast = CompilerContext.SpawnIntermediateNode<UK2Node_CastByteToEnum>(this, SourceGraph); 
		UsafeCast->Enum = Enum;
		UsafeCast->bSafe = false;
		UsafeCast->AllocateDefaultPins();

		// UNSAFE CAST INPUT
		UEdGraphPin* CastInputPin = UsafeCast->FindPinChecked(ByteInputPinName);
		UEdGraphPin* FunctionReturnPin = CallValidation->GetReturnValuePin();
		check(NULL != FunctionReturnPin);
		Schema->TryCreateConnection(CastInputPin, FunctionReturnPin);

		// OPUTPUT PIN
		UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		UEdGraphPin* NewReturnPin = UsafeCast->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *NewReturnPin);

		BreakAllNodeLinks();
	}
}

class FKCHandler_CastByteToEnum : public FNodeHandlingFunctor
{
public:
	FKCHandler_CastByteToEnum(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		FNodeHandlingFunctor::RegisterNets(Context, Node); //handle literals

		UEdGraphPin* InPin = Node->FindPinChecked(UK2Node_CastByteToEnum::ByteInputPinName);
		UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(InPin);
		if (Context.NetMap.Find(Net) == NULL)
		{
			FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
			Context.NetMap.Add(Net, Term);
		}

		FBPTerminal** ValueSource = Context.NetMap.Find(Net);
		check(ValueSource && *ValueSource);
		UEdGraphPin* OutPin = Node->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		if (ensure(Context.NetMap.Find(OutPin) == nullptr))
		{
			// We need to copy here to avoid passing in a reference to an element inside the map. The array
			// that owns the map members could be reallocated, causing the reference to become stale.
			FBPTerminal* ValueSourceCopy = *ValueSource;
			Context.NetMap.Add(OutPin, ValueSourceCopy);
		}
	}
};

FNodeHandlingFunctor* UK2Node_CastByteToEnum::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	if (!bSafe)
	{
		return new FKCHandler_CastByteToEnum(CompilerContext);
	}
	return new FNodeHandlingFunctor(CompilerContext);;
}

bool UK2Node_CastByteToEnum::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	const UEnum* SubCategoryObject = Cast<UEnum>(OtherPin->PinType.PinSubCategoryObject.Get());
	if (SubCategoryObject)
	{
		if (SubCategoryObject != Enum)
		{
			return true;
		}
	}
	return false;
}

void UK2Node_CastByteToEnum::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, FFieldVariant /*EnumField*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UK2Node_CastByteToEnum* EnumNode = CastChecked<UK2Node_CastByteToEnum>(NewNode);
			EnumNode->Enum  = NonConstEnumPtr.Get();
			EnumNode->bSafe = true;
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* InEnum)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UEnum*>(InEnum));
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(InEnum));
		NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

FText UK2Node_CastByteToEnum::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Enum);
}

void UK2Node_CastByteToEnum::ReloadEnum(class UEnum* InEnum)
{
	Enum = InEnum;
	CachedTooltip.MarkDirty();
}
