// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ClassDynamicCast.h"

#include "BlueprintCompiledStatement.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "DynamicCastHandler.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "GraphEditorSettings.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ClassDynamicCast)

class FKismetCompilerContext;

#define LOCTEXT_NAMESPACE "K2Node_ClassDynamicCast"

struct FClassDynamicCastHelper
{
	static const FName CastSuccessPinName;
	static const FName ClassToCastName;;
};

const FName FClassDynamicCastHelper::CastSuccessPinName(TEXT("bSuccess"));
const FName FClassDynamicCastHelper::ClassToCastName(TEXT("Class"));


UK2Node_ClassDynamicCast::UK2Node_ClassDynamicCast(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_ClassDynamicCast::AllocateDefaultPins()
{
	// Check to track down possible BP comms corruption
	//@TODO: Move this somewhere more sensible
	ensure((TargetType == nullptr) || (!TargetType->HasAnyClassFlags(CLASS_NewerVersionExists)));

	// Exec pins (if needed)
	CreateExecPins();

	// Input - Source type Pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, UObject::StaticClass(), FClassDynamicCastHelper::ClassToCastName);

	// Output - Data Pin
	if (TargetType)
	{
		const FString CastResultPinName = UEdGraphSchema_K2::PN_CastedValuePrefix + TargetType->GetDisplayNameText().ToString();
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Class, *TargetType, *CastResultPinName);
	}

	// Output - Success
	CreateSuccessPin();

	UK2Node::AllocateDefaultPins();
}

FLinearColor UK2Node_ClassDynamicCast::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ClassPinTypeColor;
}

FText UK2Node_ClassDynamicCast::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("NodeTitle", "{0} Class"), Super::GetNodeTitle(TitleType)), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_ClassDynamicCast::GetTooltipText() const
{
	if (TargetType && TargetType->IsChildOf(UInterface::StaticClass()))
	{
		return FText::Format(LOCTEXT("CastToInterfaceTooltip", "Tries to access class as an interface '{0}' it may implement."), FText::FromString(TargetType->GetName()));
	}
	UBlueprint* CastToBP = UBlueprint::GetBlueprintFromClass(TargetType);
	if (CastToBP)
	{
		return FText::Format(LOCTEXT("CastToBPTooltip", "Tries to access class as a blueprint class '{0}' it may inherit from.\n\nNOTE: This will cause the blueprint to always be loaded, which can be expensive."), FText::FromString(CastToBP->GetName()));
	}
	
	const FString ClassName = TargetType ? TargetType->GetName() : TEXT("");
	return FText::Format(LOCTEXT("CastToNativeTooltip", "Tries to access class '{0}' as one it may inherit from."), FText::FromString(ClassName));
}

UEdGraphPin* UK2Node_ClassDynamicCast::GetCastSourcePin() const
{
	UEdGraphPin* Pin = FindPinChecked(FClassDynamicCastHelper::ClassToCastName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_ClassDynamicCast::GetBoolSuccessPin() const
{
	UEdGraphPin* Pin = FindPin(FClassDynamicCastHelper::CastSuccessPinName);
	check((Pin == nullptr) || (Pin->Direction == EGPD_Output));
	return Pin;
}

FNodeHandlingFunctor* UK2Node_ClassDynamicCast::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_DynamicCast(CompilerContext, KCST_MetaCast);
}

#undef LOCTEXT_NAMESPACE
