// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_CommutativeAssociativeBinaryOperator.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Script.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CommutativeAssociativeBinaryOperator)

#define LOCTEXT_NAMESPACE "CommutativeAssociativeBinaryOperatorNode"

UK2Node_CommutativeAssociativeBinaryOperator::UK2Node_CommutativeAssociativeBinaryOperator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NumAdditionalInputs = 0;
}

UEdGraphPin* UK2Node_CommutativeAssociativeBinaryOperator::FindOutPin() const
{
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		if(EEdGraphPinDirection::EGPD_Output == Pins[PinIdx]->Direction)
		{
			return Pins[PinIdx];
		}
	}
	return NULL;
}

UEdGraphPin* UK2Node_CommutativeAssociativeBinaryOperator::FindSelfPin() const
{
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		if(Pins[PinIdx]->PinName == UEdGraphSchema_K2::PN_Self)
		{
			return Pins[PinIdx];
		}
	}
	return NULL;
}

bool UK2Node_CommutativeAssociativeBinaryOperator::CanAddPin() const
{
	return (NumAdditionalInputs + BinaryOperatorInputsNum) < GetMaxInputPinsNum();
}

bool UK2Node_CommutativeAssociativeBinaryOperator::CanRemovePin(const UEdGraphPin* Pin) const
{
	return (
		Pin &&
		Pin->ParentPin == nullptr &&
		NumAdditionalInputs &&
		(INDEX_NONE != Pins.IndexOfByKey(Pin)) &&
		(EEdGraphPinDirection::EGPD_Input == Pin->Direction)
	);
}

UEdGraphPin* UK2Node_CommutativeAssociativeBinaryOperator::GetInputPin(int32 InputPinIndex)
{
	const UEdGraphPin* OutPin = FindOutPin();
	const UEdGraphPin* SelfPin = FindSelfPin();

	int32 CurrentInputIndex = 0;
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		UEdGraphPin* CurrentPin = Pins[PinIdx];
		if((CurrentPin != OutPin) && (CurrentPin != SelfPin))
		{
			if(CurrentInputIndex == InputPinIndex)
			{
				return CurrentPin;
			}
			CurrentInputIndex++;
		}
	}
	return NULL;
}

FEdGraphPinType UK2Node_CommutativeAssociativeBinaryOperator::GetType() const
{
	for (int32 PinIt = 0; PinIt < Pins.Num(); PinIt++)
	{
		if (Pins[PinIt] != FindSelfPin())
		{
			return Pins[PinIt]->PinType;
		}
	}
	return FEdGraphPinType();
}

void UK2Node_CommutativeAssociativeBinaryOperator::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (const UFunction* Function = GetTargetFunction())
	{
		if (Function->HasAnyFunctionFlags(FUNC_BlueprintPure))
		{
			UEdGraphPin* SelfPin = FindSelfPin();
			const FEdGraphPinType InputType = GetType();
			int32 OutPinCount = 0;
			bool bConsistentPinType = true;

			for (UEdGraphPin* Pin : Pins)
			{
				if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
				{
					++OutPinCount;
				}
				else if (Pin != SelfPin)
				{
					if (Pin->PinType != InputType)
					{
						bConsistentPinType = false;
					}
				}
			}

			if (OutPinCount != 1)
			{
				MessageLog.Error(*LOCTEXT("InvalidOutPin", "@@ - Commutative associative binary operators must have a single output pin.").ToString(), this);
			}
			if (!bConsistentPinType)
			{
				MessageLog.Error(*LOCTEXT("InconsistentType", "@@ - Commutative associative binary operator inputs must all be of the same type.").ToString(), this);
			}
		}
		else
		{
			MessageLog.Error(*LOCTEXT("MustBePure", "@@ - Commutative associative binary operators must be pure functions.").ToString(), this);
		}
	}
}

void UK2Node_CommutativeAssociativeBinaryOperator::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	for (int32 i = 0; i < NumAdditionalInputs; ++i)
	{
		AddInputPinInner(i);
	}
}

void UK2Node_CommutativeAssociativeBinaryOperator::AddInputPinInner(int32 AdditionalPinIndex)
{
	const FEdGraphPinType InputType = GetType();
	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.ContainerType = InputType.ContainerType;
	PinParams.bIsReference = InputType.bIsReference;
	PinParams.ValueTerminalType = InputType.PinValueType;
	CreatePin(EGPD_Input, 
		InputType.PinCategory, 
		InputType.PinSubCategory, 
		InputType.PinSubCategoryObject.Get(), 
		GetNameForAdditionalPin(AdditionalPinIndex + BinaryOperatorInputsNum),
		PinParams
	);
}

void UK2Node_CommutativeAssociativeBinaryOperator::AddInputPin()
{
	if(CanAddPin())
	{
		FScopedTransaction Transaction( LOCTEXT("AddPinTx", "AddPin") );
		Modify();

		AddInputPinInner(NumAdditionalInputs);
		++NumAdditionalInputs;
	
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UK2Node_CommutativeAssociativeBinaryOperator::RemoveInputPin(UEdGraphPin* Pin)
{
	if(CanRemovePin(Pin))
	{
		FScopedTransaction Transaction( LOCTEXT("RemovePinTx", "RemovePin") );
		Modify();

		if (RemovePin(Pin))
		{
			--NumAdditionalInputs;

			int32 NameIndex = 0;
			const UEdGraphPin* OutPin = FindOutPin();
			const UEdGraphPin* SelfPin = FindSelfPin();
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* LocalPin = Pins[PinIndex];
				if(LocalPin && (LocalPin != OutPin) && (LocalPin != SelfPin))
				{
					const FName PinName = GetNameForAdditionalPin(NameIndex);
					if (PinName != LocalPin->PinName)
					{
						LocalPin->Modify();
						LocalPin->PinName = PinName;
					}
					NameIndex++;
				}
			}
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}
	}
}

void UK2Node_CommutativeAssociativeBinaryOperator::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		static FName CommutativeAssociativeBinaryOperatorNodeName = FName("CommutativeAssociativeBinaryOperatorNode");
		FText CommutativeAssociativeBinaryOperatorStr = LOCTEXT("CommutativeAssociativeBinaryOperatorNode", "Operator Node");
		if (Context->Pin != NULL)
		{
			if(CanRemovePin(Context->Pin))
			{
				FToolMenuSection& Section = Menu->AddSection(CommutativeAssociativeBinaryOperatorNodeName, CommutativeAssociativeBinaryOperatorStr);
				Section.AddMenuEntry(
					"RemovePin",
					LOCTEXT("RemovePin", "Remove pin"),
					LOCTEXT("RemovePinTooltip", "Remove this input pin"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(const_cast<UK2Node_CommutativeAssociativeBinaryOperator*>(this), &UK2Node_CommutativeAssociativeBinaryOperator::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
					)
				);
			}
		}
		else if(CanAddPin())
		{
			FToolMenuSection& Section = Menu->AddSection(CommutativeAssociativeBinaryOperatorNodeName, CommutativeAssociativeBinaryOperatorStr);
			Section.AddMenuEntry(
				"AddPin",
				LOCTEXT("AddPin", "Add pin"),
				LOCTEXT("AddPinTooltip", "Add another input pin"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_CommutativeAssociativeBinaryOperator*>(this), &UK2Node_CommutativeAssociativeBinaryOperator::AddInputPin)
				)
			);
		}
	}
}

void UK2Node_CommutativeAssociativeBinaryOperator::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (NumAdditionalInputs > 0)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

		UEdGraphPin* LastOutPin = NULL;
		const UFunction* const Function = GetTargetFunction();

		const UEdGraphPin* SrcOutPin = FindOutPin();
		const UEdGraphPin* SrcSelfPin = FindSelfPin();
		UEdGraphPin* SrcFirstInput = GetInputPin(0);
		check(SrcFirstInput);

		for(int32 PinIndex = 0; PinIndex < Pins.Num(); PinIndex++)
		{
			UEdGraphPin* CurrentPin = Pins[PinIndex];
			if( (CurrentPin == SrcFirstInput) || (CurrentPin == SrcOutPin) || (SrcSelfPin == CurrentPin) )
			{
				continue;
			}

			UK2Node_CommutativeAssociativeBinaryOperator* NewOperator = SourceGraph->CreateIntermediateNode<UK2Node_CommutativeAssociativeBinaryOperator>();
			NewOperator->SetFromFunction(Function);
			NewOperator->AllocateDefaultPins();
			CompilerContext.MessageLog.NotifyIntermediateObjectCreation(NewOperator, this);

			UEdGraphPin* NewOperatorInputA = NewOperator->GetInputPin(0);
			check(NewOperatorInputA);
			if(LastOutPin)
			{
				Schema->TryCreateConnection(LastOutPin, NewOperatorInputA);
			}
			else
			{
				// handle first created node (SrcFirstInput is skipped, and has no own node).
				CompilerContext.MovePinLinksToIntermediate(*SrcFirstInput, *NewOperatorInputA);
			}

			UEdGraphPin* NewOperatorInputB = NewOperator->GetInputPin(1);
			check(NewOperatorInputB);
			CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *NewOperatorInputB);

			LastOutPin = NewOperator->FindOutPin();
		}

		check(LastOutPin);

		UEdGraphPin* TrueOutPin = FindOutPin();
		check(TrueOutPin);
		CompilerContext.MovePinLinksToIntermediate(*TrueOutPin, *LastOutPin);

		BreakAllNodeLinks();
	}
}

#undef LOCTEXT_NAMESPACE
