// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_ComponentBoundEvent.h"

#include "Containers/Array.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/ComponentDelegateBinding.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ComponentBoundEvent)

#define LOCTEXT_NAMESPACE "K2Node"

static TAutoConsoleVariable<bool> CVarBPEnableDeprecatedWarningForComponentDelegateNodes(
	TEXT("bp.EnableDeprecatedWarningForComponentDelegateNodes"),
	true,
	TEXT("Show Deprecated warning for component delegate event nodes"),
	ECVF_Cheat);

// @TODO_BH: Remove the CVar for validity checking when we can get all the errors sorted out
namespace PinValidityCheck
{
	/**
	* CVar controls pin validity warning which will throw when a macro graph is silently failing
	* @see UE-100024
	*/
	static bool bDisplayMissingBoundComponentWarning = true;
	static FAutoConsoleVariableRef CVarDisplayMissingBoundComponentWarning(
		TEXT("bp.PinValidityCheck.bDisplayMissingBoundComponentWarning"), bDisplayMissingBoundComponentWarning,
		TEXT("CVar controls pin validity warning which will throw when a bound event has no matching component"),
		ECVF_Default);
}

UK2Node_ComponentBoundEvent::UK2Node_ComponentBoundEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UK2Node_ComponentBoundEvent::Modify(bool bAlwaysMarkDirty)
{
	CachedNodeTitle.MarkDirty();

	return Super::Modify(bAlwaysMarkDirty);
}

bool UK2Node_ComponentBoundEvent::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bDisallowPaste = !Super::CanPasteHere(TargetGraph);
	if (!bDisallowPaste)
	{
		if (const UK2Node_Event* PreExistingNode = FKismetEditorUtilities::FindBoundEventForComponent(FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph), DelegatePropertyName, ComponentPropertyName))
		{
			//UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because it is flagged as an internal event.", *GetFName().ToString());
			bDisallowPaste = true;
		}
	}
	return !bDisallowPaste;

}
FText UK2Node_ComponentBoundEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DelegatePropertyName"), GetTargetDelegateDisplayName());
		Args.Add(TEXT("ComponentPropertyName"), FText::FromName(ComponentPropertyName));

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("ComponentBoundEvent_Title", "{DelegatePropertyName} ({ComponentPropertyName})"), Args), this);
	}
	return CachedNodeTitle;
}

void UK2Node_ComponentBoundEvent::InitializeComponentBoundEventParams(FObjectProperty const* InComponentProperty, const FMulticastDelegateProperty* InDelegateProperty)
{
	if (InComponentProperty && InDelegateProperty)
	{
		ComponentPropertyName = InComponentProperty->GetFName();
		DelegatePropertyName = InDelegateProperty->GetFName();
		DelegateOwnerClass = CastChecked<UClass>(InDelegateProperty->GetOwner<UObject>())->GetAuthoritativeClass();

		EventReference.SetFromField<UFunction>(InDelegateProperty->SignatureFunction, /*bIsConsideredSelfContext =*/false);

		CustomFunctionName = FName(*FString::Printf(TEXT("BndEvt__%s_%s_%s_%s"), *GetBlueprint()->GetName(), *InComponentProperty->GetName(), *GetName(), *EventReference.GetMemberName().ToString()));
		bOverrideFunction = false;
		bInternalEvent = true;
		CachedNodeTitle.MarkDirty();
	}
}

UClass* UK2Node_ComponentBoundEvent::GetDynamicBindingClass() const
{
	return UComponentDelegateBinding::StaticClass();
}

void UK2Node_ComponentBoundEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UComponentDelegateBinding* ComponentBindingObject = CastChecked<UComponentDelegateBinding>(BindingObject);

	FBlueprintComponentDelegateBinding Binding;
	Binding.ComponentPropertyName = ComponentPropertyName;
	Binding.DelegatePropertyName = DelegatePropertyName;
	Binding.FunctionNameToBind = CustomFunctionName;

	CachedNodeTitle.MarkDirty();
	ComponentBindingObject->ComponentDelegateBindings.Add(Binding);
}

void UK2Node_ComponentBoundEvent::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{	
	if (InVariableClass && InVariableClass->IsChildOf(InBlueprint->GeneratedClass))
	{
		// This could be the case if the component that this was originally bound to was removed, and a new one was 
		// added in it's place. @see UE-88511
		if (InNewVarName == ComponentPropertyName)
		{
			FCompilerResultsLog LogResults;
			FMessageLog MessageLog("BlueprintLog");
			LogResults.Error(*LOCTEXT("ComponentBoundEvent_Rename_Error", "There can only be one event node bound to this component! Delete @@ or the other bound event").ToString(), this);

			MessageLog.NewPage(LOCTEXT("ComponentBoundEvent_Rename_Error_Label", "Rename Component Error"));
			MessageLog.AddMessages(LogResults.Messages);
			MessageLog.Notify(LOCTEXT("OnConvertEventToFunctionErrorMsg", "Renaming a component"));
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(this);
		}
		else if (InOldVarName == ComponentPropertyName)
		{
			Modify();
			ComponentPropertyName = InNewVarName;
		}
	}	
}

void UK2Node_ComponentBoundEvent::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	if (PinValidityCheck::bDisplayMissingBoundComponentWarning && !IsDelegateValid())
	{
		MessageLog.Warning(*LOCTEXT("ComponentBoundEvent_Error", "@@ does not have a valid matching component!").ToString(), this);
	}
	Super::ValidateNodeDuringCompilation(MessageLog);
}

bool UK2Node_ComponentBoundEvent::IsDelegateValid() const
{
	const UBlueprint* const BP = GetBlueprint();
	// Validate that the property has not been renamed or deleted via the SCS tree
	return BP && FindFProperty<FObjectProperty>(BP->GeneratedClass, ComponentPropertyName)
		// Validate that the actual declaration for this event has not been deleted 
		// either from a native base class or a BP multicast delegate. The Delegate could have been 
		// renamed/redirected, so also check for a remapped field if we need to
		&& (GetTargetDelegateProperty() || FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName));
}

bool UK2Node_ComponentBoundEvent::HasDeprecatedReference() const
{
	if (CVarBPEnableDeprecatedWarningForComponentDelegateNodes.GetValueOnAnyThread())
	{
		if (const FMulticastDelegateProperty* DelegateProperty = GetTargetDelegateProperty())
		{
			return DelegateProperty->HasAnyPropertyFlags(EPropertyFlags::CPF_Deprecated);	
		}
	}
	return false;
}

FEdGraphNodeDeprecationResponse UK2Node_ComponentBoundEvent::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		const UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
		if (ensureMsgf(Function != nullptr, TEXT("This node should not be able to report having a deprecated reference if the event override cannot be resolved.")))
		{
			Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;
			const FText DetailedMessage = FText::FromString(Function->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage));
			Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(GetTargetDelegateDisplayName(), DetailedMessage);
		}
	}

	return Response;
}

bool UK2Node_ComponentBoundEvent::IsUsedByAuthorityOnlyDelegate() const
{
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();
	return (TargetDelegateProp && TargetDelegateProp->HasAnyPropertyFlags(CPF_BlueprintAuthorityOnly));
}

FMulticastDelegateProperty* UK2Node_ComponentBoundEvent::GetTargetDelegateProperty() const
{
	return FindFProperty<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
}

FText UK2Node_ComponentBoundEvent::GetTargetDelegateDisplayName() const
{
	FMulticastDelegateProperty* Prop = GetTargetDelegateProperty();
	return Prop ? Prop->GetDisplayNameText() : FText::FromName(DelegatePropertyName);
}

FText UK2Node_ComponentBoundEvent::GetTooltipText() const
{
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();
	if (TargetDelegateProp)
	{
		return TargetDelegateProp->GetToolTipText();
	}
	else
	{
		return FText::FromName(DelegatePropertyName);
	}
}

FString UK2Node_ComponentBoundEvent::GetDocumentationLink() const
{
	if (DelegateOwnerClass)
	{
		return FString::Printf(TEXT("Shared/GraphNodes/Blueprint/%s%s"), DelegateOwnerClass->GetPrefixCPP(), *EventReference.GetMemberName().ToString());
	}

	return FString();
}

FString UK2Node_ComponentBoundEvent::GetDocumentationExcerptName() const
{
	return DelegatePropertyName.ToString();
}

void UK2Node_ComponentBoundEvent::ReconstructNode()
{
	// We need to fixup our event reference as it may have changed or been redirected
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();

	// If we couldn't find the target delegate, then try to find it in the property remap table
	if (!TargetDelegateProp)
	{
		FMulticastDelegateProperty* NewProperty = FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
		if (NewProperty)
		{
			// Found a remapped property, update the node
			TargetDelegateProp = NewProperty;
			DelegatePropertyName = NewProperty->GetFName();
		}
	}

	if (TargetDelegateProp && TargetDelegateProp->SignatureFunction)
	{
		EventReference.SetFromField<UFunction>(TargetDelegateProp->SignatureFunction, false);
	}

	CachedNodeTitle.MarkDirty();

	Super::ReconstructNode();
}

void UK2Node_ComponentBoundEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Fix up legacy nodes that may not yet have a delegate pin
	if (Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE)
		{
			DelegateOwnerClass = EventSignatureClass_DEPRECATED;
		}

		// Recover from the period where DelegateOwnerClass was transient
		if (!DelegateOwnerClass && HasValidBlueprint())
		{
			// Search for a component property on the owning class, this should work in most cases
			UBlueprint* ParentBlueprint = GetBlueprint();
			UClass* ParentClass = ParentBlueprint ? *ParentBlueprint->GeneratedClass : NULL;
			if (!ParentClass && ParentBlueprint)
			{
				// Try the skeleton class
				ParentClass = ParentBlueprint->SkeletonGeneratedClass;
			}

			FObjectProperty* ComponentProperty = ParentClass ? CastField<FObjectProperty>(ParentClass->FindPropertyByName(ComponentPropertyName)) : NULL;

			if (ParentClass && ComponentProperty)
			{
				UE_LOGF(LogBlueprint, Warning, "Repaired invalid component bound event in node %ls.", *GetPathName());
				DelegateOwnerClass = ComponentProperty->PropertyClass;
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
