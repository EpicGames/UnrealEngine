// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Composite.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Event.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Composite)

class UObject;

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_Composite::UK2Node_Composite(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanHaveInputs = true;
	bCanHaveOutputs = true;
	bIsEditable = true;
}

void UK2Node_Composite::AllocateDefaultPins()
{
	UK2Node::AllocateDefaultPins();
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if (OutputSourceNode)
	{
		for (TArray<UEdGraphPin*>::TIterator PinIt(OutputSourceNode->Pins); PinIt; ++PinIt)
		{
			UEdGraphPin* PortPin = *PinIt;
			if (PortPin->Direction == EGPD_Input)
			{
				UEdGraphPin* NewPin = CreatePin(UEdGraphPin::GetComplementaryDirection(PortPin->Direction), PortPin->PinType, PortPin->PinName);
				Schema->SetPinAutogeneratedDefaultValue(NewPin, PortPin->GetDefaultAsString());
			}
		}
	}

	if (InputSinkNode)
	{
		for (TArray<UEdGraphPin*>::TIterator PinIt(InputSinkNode->Pins); PinIt; ++PinIt)
		{
			UEdGraphPin* PortPin = *PinIt;
			if (PortPin->Direction == EGPD_Output)
			{
				UEdGraphPin* NewPin = CreatePin(UEdGraphPin::GetComplementaryDirection(PortPin->Direction), PortPin->PinType, PortPin->PinName);
				Schema->SetPinAutogeneratedDefaultValue(NewPin, PortPin->GetDefaultAsString());
			}
		}
	}

	CacheWildcardPins();
}

void UK2Node_Composite::DestroyNode()
{
	// Remove the associated graph if it's exclusively owned by this node
	UEdGraph* GraphToRemove = BoundGraph;
	BoundGraph = NULL;
	Super::DestroyNode();
	
	if (GraphToRemove)
	{
		FBlueprintEditorUtils::RemoveGraph(GetBlueprint(), GraphToRemove, EGraphRemoveFlags::Recompile);
	}
}

void UK2Node_Composite::PostPasteNode()
{
	Super::PostPasteNode();

	if (BoundGraph != nullptr)
	{
		UEdGraph* ParentGraph = CastChecked<UEdGraph>(GetOuter());
		ensure(BoundGraph != ParentGraph);

		const UEdGraphSchema* ParentSchema = ParentGraph->GetSchema();
		check(ParentSchema);

		UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraphChecked(BoundGraph);

		// Update the InputSinkNode / OutputSourceNode pointers to point to the new graph
		TSet<UEdGraphNode*> BoundaryNodes;
		for (int32 NodeIndex = 0; NodeIndex < BoundGraph->Nodes.Num(); ++NodeIndex)
		{
			UEdGraphNode* Node = BoundGraph->Nodes[NodeIndex];
			
			// Remove this node if it should not exist more then one in blueprint
			bool bRemoveNode = false;
			if (UK2Node_Event* Event = Cast<UK2Node_Event>(Node))
			{
				if (FBlueprintEditorUtils::FindOverrideForFunction(BP, Event->EventReference.GetMemberParentClass(Event->GetBlueprintClassFromNode()), Event->EventReference.GetMemberName()))
				{
					bRemoveNode = true;
				}
			}

			// Intentional that we check for exact class here!
			bool bIsTunnelNode = false;
			if (Node->GetClass() == UK2Node_Tunnel::StaticClass())
			{
				bIsTunnelNode = true;
			}

			// Also remove any nodes from the subgraph that are otherwise not schema-compatible
			if (bRemoveNode || (!bIsTunnelNode && !ParentSchema->CanEncapuslateNode(*Node)))
			{
				FBlueprintEditorUtils::RemoveNode(BP, Node, true);
				NodeIndex--;
				continue;
			}
			
			BoundaryNodes.Add(Node);

			if (bIsTunnelNode)
			{
				// Exactly a tunnel node, should be the entrance or exit node
				UK2Node_Tunnel* Tunnel = CastChecked<UK2Node_Tunnel>(Node);

				if (Tunnel->bCanHaveInputs && !Tunnel->bCanHaveOutputs)
				{
					OutputSourceNode = Tunnel;
					Tunnel->InputSinkNode = this;
				}
				else if (Tunnel->bCanHaveOutputs && !Tunnel->bCanHaveInputs)
				{
					InputSinkNode = Tunnel;
					Tunnel->OutputSourceNode = this;
				}
				else
				{
					ensureMsgf(false, TEXT("Unexpected tunnel node '%s' in cloned graph '%s' (both I/O or neither)"), *Tunnel->GetName(), *GetName());
				}
			}
		}

		RenameBoundGraphCloseToName(BoundGraph->GetName());
		ensure(BoundGraph->SubGraphs.Find(ParentGraph) == INDEX_NONE);

		//Nested composites will already be in the SubGraph array
		if(ParentGraph->SubGraphs.Find(BoundGraph) == INDEX_NONE)
		{
			// Set the subgraph's schema class to match the parent (as we do when placing a new node). This is needed in
			// order to ensure that the new subgraph will compile. We've already verified that the new subgraph is schema-
			// compatible at this point (see above).
			BoundGraph->Schema = ParentGraph->Schema;

			ParentGraph->SubGraphs.Add(BoundGraph);
		}

		FEdGraphUtilities::PostProcessPastedNodes(BoundaryNodes);
	}
}

void UK2Node_Composite::PostEditUndo()
{
	Super::PostEditUndo();
	FixupInputAndOutputSink();
}

void UK2Node_Composite::FixupInputAndOutputSink()
{
	if (BoundGraph)
	{
		// Update the InputSinkNode / OutputSourceNode pointers to point to the new graph
		for (UEdGraphNode* Node : BoundGraph->Nodes)
		{
			if (Node->GetClass() == UK2Node_Tunnel::StaticClass())
			{
				// Exactly a tunnel node, should be the entrance or exit node
				UK2Node_Tunnel* Tunnel = CastChecked<UK2Node_Tunnel>(Node);

				if (Tunnel->bCanHaveInputs && !Tunnel->bCanHaveOutputs)
				{
					OutputSourceNode = Tunnel;
					Tunnel->InputSinkNode = this;
				}
				else if (Tunnel->bCanHaveOutputs && !Tunnel->bCanHaveInputs)
				{
					InputSinkNode = Tunnel;
					Tunnel->OutputSourceNode = this;
				}
				else
				{
					ensureMsgf(false, TEXT("Unexpected tunnel node '%s' in cloned graph '%s' (both I/O or neither)"), *Tunnel->GetName(), *GetName());
				}
			}
		}
	}
}

FText UK2Node_Composite::GetTooltipText() const
{
	if (InputSinkNode != NULL)
	{
		if (!InputSinkNode->MetaData.ToolTip.IsEmpty())
		{
			return InputSinkNode->MetaData.ToolTip;
		}
	}

	return LOCTEXT("CollapsedCompositeNode", "Collapsed composite node");
}

FLinearColor UK2Node_Composite::GetNodeTitleColor() const
{
	if (InputSinkNode != NULL)
	{
		return InputSinkNode->MetaData.InstanceTitleColor.ToFColor(false);
	}

	return FLinearColor::White;
}

FText UK2Node_Composite::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (BoundGraph == nullptr)
	{
		return LOCTEXT("InvalidGraph", "Invalid Graph");
	}
	else if (TitleType != ENodeTitleType::FullTitle)
	{
		return FText::FromString(BoundGraph->GetName());
	}
	else if (CachedNodeTitle.IsOutOfDate(this)) // TitleType == ENodeTitleType::FullTitle
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("BoundGraphName"), (BoundGraph) ? FText::FromString(BoundGraph->GetName()) : LOCTEXT("InvalidGraph", "Invalid Graph"));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("Collapsed_Name", "{BoundGraphName}\nCollapsed Graph"), Args), this);
	}
	return CachedNodeTitle;
}

bool UK2Node_Composite::CanUserDeleteNode() const
{
	return true;
}

void UK2Node_Composite::RenameBoundGraphCloseToName(const FString& Name)
{
		//FEdGraphUtilities::RenameGraphCloseToName(BoundGraph, OldGraph->GetName(), 2);
	UEdGraph* ParentGraph = CastChecked<UEdGraph>(GetOuter());

	//Give the graph a unique name
	bool bFoundName = false;
	for (int32 NameIndex = 2; !bFoundName; ++NameIndex)
	{
		const FString NewName = FString::Printf(TEXT("%s_%d"), *Name, NameIndex);
		bool bGraphNameAvailable = false;

		bGraphNameAvailable = IsCompositeNameAvailable(NewName);

		//make sure the name is not used in the scope of BoundGraph or ParentGraph 
		if (bGraphNameAvailable && BoundGraph->Rename(*NewName, ParentGraph, REN_Test) && BoundGraph->Rename(*NewName, BoundGraph->GetOuter(), REN_Test))
		{
			//Name is available
			UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForGraphChecked(BoundGraph);
			BoundGraph->Rename(*NewName, BoundGraph->GetOuter(), (BP->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
			bFoundName = true;
		}
	}
}

bool UK2Node_Composite::IsCompositeNameAvailable( const FString& NewName )
{
	UEdGraph* ParentGraph = CastChecked<UEdGraph>(GetOuter());

	//check to see if the parent graph already has a sub graph by this name
	for (auto It = ParentGraph->SubGraphs.CreateIterator();It;++It)
	{
		UEdGraph* Graph = *It;
		if (Graph->GetName() == NewName)
		{
			return false;
		}
	}	

	if (UK2Node_Composite* Composite = Cast<UK2Node_Composite>(ParentGraph->GetOuter()))
	{
		return Composite->IsCompositeNameAvailable(NewName);
	}
	return true;
}

UObject* UK2Node_Composite::GetJumpTargetForDoubleClick() const
{
	// Dive into the collapsed node
	return BoundGraph;
}

void UK2Node_Composite::PostPlacedNewNode()
{
	// Create a new graph
	BoundGraph = FBlueprintEditorUtils::CreateNewGraph(this, NAME_None, UEdGraph::StaticClass(), GetGraph()->Schema);
	check(BoundGraph);

	// Create the entry/exit nodes inside the new graph
	{
		FGraphNodeCreator<UK2Node_Tunnel> EntryNodeCreator(*BoundGraph);
		UK2Node_Tunnel* EntryNode = EntryNodeCreator.CreateNode();
		EntryNode->bCanHaveOutputs = true;
		EntryNode->bCanHaveInputs = false;
		EntryNode->OutputSourceNode = this;
		EntryNodeCreator.Finalize();

		InputSinkNode = EntryNode;
	}
	{
		FGraphNodeCreator<UK2Node_Tunnel> ExitNodeCreator(*BoundGraph);
		UK2Node_Tunnel* ExitNode = ExitNodeCreator.CreateNode();
		ExitNode->bCanHaveOutputs = false;
		ExitNode->bCanHaveInputs = true;
		ExitNode->InputSinkNode = this;
		ExitNodeCreator.Finalize();

		OutputSourceNode = ExitNode;
	}

	// Add the new graph as a child of our parent graph
	GetGraph()->SubGraphs.Add(BoundGraph);
}

UK2Node_Tunnel* UK2Node_Composite::GetEntryNode() const
{
	check(InputSinkNode);
	return InputSinkNode;
}

UK2Node_Tunnel* UK2Node_Composite::GetExitNode() const
{
	check(OutputSourceNode);
	return OutputSourceNode;
}

void UK2Node_Composite::OnRenameNode(const FString& NewName)
{
	FBlueprintEditorUtils::RenameGraph(BoundGraph, NewName);
}

TSharedPtr<class INameValidatorInterface> UK2Node_Composite::MakeNameValidator() const
{
	return MakeShareable(new FKismetNameValidator(GetBlueprint(), BoundGraph ? BoundGraph->GetFName() : NAME_None));
}

bool UK2Node_Composite::CanCreateUnderSpecifiedSchema(const UEdGraphSchema* DesiredSchema) const
{
	if (Super::CanCreateUnderSpecifiedSchema(DesiredSchema))
	{
		// Check for derived schemas that don't support a collapsed subgraph.
		const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(DesiredSchema);
		return K2Schema->DoesSupportCollapsedNodes();
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
