// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_ConstructObjectFromClass.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ConstructObjectFromClass)

struct FK2Node_ConstructObjectFromClassHelper
{
	static FName GetWorldContextPinName()
	{
		return TEXT("WorldContextObject");
	}

	static FName GetClassPinName()
	{
		return TEXT("Class");
	}

	static FName GetOuterPinFriendlyName()
	{
		return TEXT("Outer");
	}

	static FName GetOuterPinName()
	{
		return UEdGraphSchema_K2::PN_Self;
	}
};

#define LOCTEXT_NAMESPACE "K2Node_ConstructObjectFromClass"

UK2Node_ConstructObjectFromClass::UK2Node_ConstructObjectFromClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Attempts to spawn a new object");
}

UClass* UK2Node_ConstructObjectFromClass::GetClassPinBaseClass() const
{
	return UObject::StaticClass();
}

bool UK2Node_ConstructObjectFromClass::UseWorldContext() const
{
	return FBlueprintEditorUtils::ShouldShowWorldContextPin(GetBlueprint());
}

void UK2Node_ConstructObjectFromClass::AllocateDefaultPins()
{
	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// If required add the world context pin
	if (UseWorldContext())
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), FK2Node_ConstructObjectFromClassHelper::GetWorldContextPinName());
	}

	// Add blueprint pin
	UEdGraphPin* ClassPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, GetClassPinBaseClass(), FK2Node_ConstructObjectFromClassHelper::GetClassPinName());
	
	// Result pin
	UEdGraphPin* ResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, GetClassPinBaseClass(), UEdGraphSchema_K2::PN_ReturnValue);
	
	if (UseOuter())
	{
		UEdGraphPin* OuterPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), UEdGraphSchema_K2::PN_Self);
		OuterPin->PinFriendlyName = FText::FromName(FK2Node_ConstructObjectFromClassHelper::GetOuterPinFriendlyName());
	}

	Super::AllocateDefaultPins();
}

UEdGraphPin* UK2Node_ConstructObjectFromClass::GetOuterPin() const
{
	UEdGraphPin* Pin = FindPin(FK2Node_ConstructObjectFromClassHelper::GetOuterPinName());
	ensure(nullptr == Pin || Pin->Direction == EGPD_Input);
	return Pin;
}

void UK2Node_ConstructObjectFromClass::SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const
{
	MutatablePin.PinToolTip = UEdGraphSchema_K2::TypeToText(MutatablePin.PinType).ToString();

	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	if (K2Schema != nullptr)
	{
		MutatablePin.PinToolTip += TEXT(" ");
		MutatablePin.PinToolTip += K2Schema->GetPinDisplayName(&MutatablePin).ToString();
	}

	MutatablePin.PinToolTip += FString(TEXT("\n")) + PinDescription.ToString();
}

void UK2Node_ConstructObjectFromClass::CreatePinsForClass(UClass* InClass, TArray<UEdGraphPin*>* OutClassPins)
{
	check(InClass);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const UObject* const ClassDefaultObject = InClass->GetDefaultObject(false);

	for (TFieldIterator<FProperty> PropertyIt(InClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		UClass* PropertyClass = CastChecked<UClass>(Property->GetOwner<UObject>());
		const bool bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bIsExposedToSpawn = UEdGraphSchema_K2::IsPropertyExposedOnSpawn(Property);
		const bool bIsSettableExternally = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);

		if(	bIsExposedToSpawn &&
			!Property->HasAnyPropertyFlags(CPF_Parm) && 
			bIsSettableExternally &&
			Property->HasAllPropertyFlags(CPF_BlueprintVisible) &&
			!bIsDelegate &&
			(nullptr == FindPin(Property->GetFName()) ) &&
			FBlueprintEditorUtils::PropertyStillExists(Property))
		{
			if (UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Property->GetFName()))
			{
				K2Schema->ConvertPropertyToPinType(Property, /*out*/ Pin->PinType);
				if (OutClassPins)
				{
					OutClassPins->Add(Pin);
				}

				if (Property->HasAllPropertyFlags(CPF_AdvancedDisplay))
				{
					Pin->bAdvancedView = true;

					if (AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
					{
						AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
					}
				}

				if (UEdGraphSchema_K2::IsPropertyDeprecated(Property))
				{
					Pin->bDeprecated = true;
					Pin->DeprecationMessage = Property->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
				}

				if (ClassDefaultObject && K2Schema->PinDefaultValueIsEditable(*Pin))
				{
					FString DefaultValueAsString;
					const bool bDefaultValueSet = FBlueprintEditorUtils::PropertyValueToString(Property, reinterpret_cast<const uint8*>(ClassDefaultObject), DefaultValueAsString, this);
					check(bDefaultValueSet);
					K2Schema->SetPinAutogeneratedDefaultValue(Pin, DefaultValueAsString);
				}

				// Copy tooltip from the property.
				K2Schema->ConstructBasicPinTooltip(*Pin, Property->GetToolTipText(), Pin->PinToolTip);
			}
		}
	}

	// Change class of output pin
	UEdGraphPin* ResultPin = GetResultPin();
	ResultPin->PinType.PinSubCategoryObject = InClass->GetAuthoritativeClass();
}

UClass* UK2Node_ConstructObjectFromClass::GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch /*=NULL*/) const
{
	UClass* UseSpawnClass = nullptr;
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* ClassPin = GetClassPin(PinsToSearch);
	if (ClassPin && ClassPin->DefaultObject && ClassPin->LinkedTo.Num() == 0)
	{
		UseSpawnClass = CastChecked<UClass>(ClassPin->DefaultObject);
	}
	else if (ClassPin && ClassPin->LinkedTo.Num())
	{
		UEdGraphPin* ClassSource = ClassPin->LinkedTo[0];
		UseSpawnClass = ClassSource ? Cast<UClass>(ClassSource->PinType.PinSubCategoryObject.Get()) : nullptr;
	}

	return UseSpawnClass;
}

void UK2Node_ConstructObjectFromClass::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	AllocateDefaultPins();
	if (UClass* UseSpawnClass = GetClassToSpawn(&OldPins))
	{
		CreatePinsForClass(UseSpawnClass);
	}
	RestoreSplitPins(OldPins);
}

UK2Node::ERedirectType UK2Node_ConstructObjectFromClass::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// the name of the outer pin was changed and its friendly name was updated to match
	// the legacy naming. Use this to identify the change
	if (NewPin->PinName == FK2Node_ConstructObjectFromClassHelper::GetOuterPinName() &&
		OldPin->PinName == FK2Node_ConstructObjectFromClassHelper::GetOuterPinFriendlyName())
	{
		return ERedirectType_Name;
	}
	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

void UK2Node_ConstructObjectFromClass::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	if (UClass* UseSpawnClass = GetClassToSpawn())
	{
		CreatePinsForClass(UseSpawnClass);
	}
}

FString UK2Node_ConstructObjectFromClass::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData = Super::GetPinMetaData(InPinName, InKey);

	if (MetaData.IsEmpty())
	{
		if (InPinName == FK2Node_ConstructObjectFromClassHelper::GetClassPinName() && InKey == FBlueprintMetadata::MD_AllowAbstractClasses)
		{
			MetaData = TEXT("false");
		}
		else if (InPinName == FK2Node_ConstructObjectFromClassHelper::GetWorldContextPinName() && InKey == FBlueprintMetadata::MD_HidePinAssetPicker)
		{
			MetaData = TEXT("true");
		}
	}

	return MetaData;
}

void UK2Node_ConstructObjectFromClass::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);
	OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, CachedNodeTitle.GetCachedText()));
}

bool UK2Node_ConstructObjectFromClass::IsSpawnVarPin(UEdGraphPin* Pin) const
{
	if (Pin->ParentPin)
	{
		return IsSpawnVarPin(Pin->ParentPin);
	}

	return(	Pin->PinName != UEdGraphSchema_K2::PN_Execute &&
			Pin->PinName != UEdGraphSchema_K2::PN_Then &&
			Pin->PinName != UEdGraphSchema_K2::PN_ReturnValue &&
			Pin->PinName != FK2Node_ConstructObjectFromClassHelper::GetClassPinName() &&
			Pin->PinName != FK2Node_ConstructObjectFromClassHelper::GetWorldContextPinName() &&
			Pin->PinName != FK2Node_ConstructObjectFromClassHelper::GetOuterPinName());
}

void UK2Node_ConstructObjectFromClass::OnClassPinChanged()
{
	// Remove all pins related to archetype variables
	TArray<UEdGraphPin*> OldPins = Pins;
	TArray<UEdGraphPin*> OldClassPins;

	for (UEdGraphPin* OldPin : OldPins)
	{
		if (IsSpawnVarPin(OldPin))
		{
			Pins.Remove(OldPin);
			OldClassPins.Add(OldPin);
		}
	}

	CachedNodeTitle.MarkDirty();

	TArray<UEdGraphPin*> NewClassPins;
	if (UClass* UseSpawnClass = GetClassToSpawn())
	{
		CreatePinsForClass(UseSpawnClass, &NewClassPins);
	}

	RestoreSplitPins(OldPins);

	UEdGraphPin* ResultPin = GetResultPin();
	// Cache all the pin connections to the ResultPin, we will attempt to recreate them
	TArray<UEdGraphPin*> ResultPinConnectionList = ResultPin->LinkedTo;
	// Because the archetype has changed, we break the output link as the output pin type will change
	ResultPin->BreakAllPinLinks(true);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Recreate any pin links to the Result pin that are still valid
	for (UEdGraphPin* Connections : ResultPinConnectionList)
	{
		K2Schema->TryCreateConnection(ResultPin, Connections);
	}

	// Rewire the old pins to the new pins so connections are maintained if possible
	RewireOldPinsToNewPins(OldClassPins, Pins, nullptr);

	// Refresh the UI for the graph so the pin changes show up
	GetGraph()->NotifyGraphChanged();

	// Mark dirty
	FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
}

void UK2Node_ConstructObjectFromClass::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);

	if (Pin && (Pin->PinName == FK2Node_ConstructObjectFromClassHelper::GetClassPinName()))
	{
		OnClassPinChanged();
	}
}

void UK2Node_ConstructObjectFromClass::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	if (UEdGraphPin* ClassPin = GetClassPin())
	{
		SetPinToolTip(*ClassPin, LOCTEXT("ClassPinDescription", "The object class you want to construct"));
	}
	if (UEdGraphPin* ResultPin = GetResultPin())
	{
		SetPinToolTip(*ResultPin, LOCTEXT("ResultPinDescription", "The constructed object"));
	}
	if (UEdGraphPin* OuterPin = (UseOuter() ? GetOuterPin() : nullptr))
	{
		SetPinToolTip(*OuterPin, LOCTEXT("OuterPinDescription", "Owner of the constructed object"));
	}

	return Super::GetPinHoverText(Pin, HoverTextOut);
}

void UK2Node_ConstructObjectFromClass::PinDefaultValueChanged(UEdGraphPin* ChangedPin) 
{
	if (ChangedPin && (ChangedPin->PinName == FK2Node_ConstructObjectFromClassHelper::GetClassPinName()))
	{
		OnClassPinChanged();
	}
}

FText UK2Node_ConstructObjectFromClass::GetTooltipText() const
{
	return NodeTooltip;
}

UEdGraphPin* UK2Node_ConstructObjectFromClass::GetClassPin(const TArray<UEdGraphPin*>* InPinsToSearch /*= NULL*/) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* TestPin : *PinsToSearch)
	{
		if (TestPin && TestPin->PinName == FK2Node_ConstructObjectFromClassHelper::GetClassPinName())
		{
			Pin = TestPin;
			break;
		}
	}
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_ConstructObjectFromClass::GetWorldContextPin() const
{
	UEdGraphPin* Pin = FindPin(FK2Node_ConstructObjectFromClassHelper::GetWorldContextPinName());
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_ConstructObjectFromClass::GetResultPin() const
{
	UEdGraphPin* Pin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

FText UK2Node_ConstructObjectFromClass::GetBaseNodeTitle() const
{
	return NSLOCTEXT("K2Node", "ConstructObject_BaseTitle", "Construct Object from Class");
}

FText UK2Node_ConstructObjectFromClass::GetDefaultNodeTitle() const
{
	return NSLOCTEXT("K2Node", "ConstructObject_Title_NONE", "Construct NONE");
}

FText UK2Node_ConstructObjectFromClass::GetNodeTitleFormat() const
{
	return NSLOCTEXT("K2Node", "Construct", "Construct {ClassName}");
}

FText UK2Node_ConstructObjectFromClass::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		return GetBaseNodeTitle();
	}
	else if (UClass* ClassToSpawn = GetClassToSpawn())
	{
		if (CachedNodeTitle.IsOutOfDate(this))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ClassName"), ClassToSpawn->GetDisplayNameText());
			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(GetNodeTitleFormat(), Args), this);
		}
		return CachedNodeTitle;
	}
	return GetDefaultNodeTitle();
}

bool UK2Node_ConstructObjectFromClass::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const 
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return Super::IsCompatibleWithGraph(TargetGraph) && (!Blueprint || FBlueprintEditorUtils::FindUserConstructionScript(Blueprint) != TargetGraph);
}

void UK2Node_ConstructObjectFromClass::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	UClass* ClassToSpawn = GetClassToSpawn();
	const FString ClassToSpawnStr = ClassToSpawn ? ClassToSpawn->GetName() : TEXT( "InvalidClass" );
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "ConstructObjectFromClass" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "ObjectClass" ), ClassToSpawnStr ));
}

void UK2Node_ConstructObjectFromClass::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_ConstructObjectFromClass::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Gameplay);
}

bool UK2Node_ConstructObjectFromClass::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UClass* SourceClass = GetClassToSpawn();
	const UBlueprint* SourceBlueprint = GetBlueprint();
	const bool bResult = (SourceClass && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint));
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}
	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

FText UK2Node_ConstructObjectFromClass::GetKeywords() const
{
	return LOCTEXT("ConstructObjectKeywords", "Create New");
}

#undef LOCTEXT_NAMESPACE
