// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ConvertAsset.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintFieldNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ClassDynamicCast.h"
#include "K2Node_DynamicCast.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ConvertAsset)

class FString;

#define LOCTEXT_NAMESPACE "K2Node_ConvertAsset"

namespace UK2Node_ConvertAssetImpl
{
	static const FName InputPinName("Input");
	static const FName OutputPinName("Output");
}

UEdGraphPin* UK2Node_ConvertAsset::GetInputPin() const
{
	return FindPin(UK2Node_ConvertAssetImpl::InputPinName);
}

UEdGraphPin* UK2Node_ConvertAsset::GetOutputPin() const
{
	return FindPin(UK2Node_ConvertAssetImpl::OutputPinName);
}

UClass* UK2Node_ConvertAsset::GetTargetClass() const
{
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	bool bIsConnected = InputPin && InputPin->LinkedTo.Num() && InputPin->LinkedTo[0];
	UEdGraphPin* SourcePin = bIsConnected ? InputPin->LinkedTo[0] : nullptr;
	return SourcePin ? FBlueprintEditorUtils::GetTypeForPin(*SourcePin) : nullptr;
}

bool UK2Node_ConvertAsset::IsClassType() const
{
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	bool bIsConnected = InputPin && InputPin->LinkedTo.Num() && InputPin->LinkedTo[0];
	UEdGraphPin* SourcePin = bIsConnected ? InputPin->LinkedTo[0] : nullptr;
	return SourcePin ? (SourcePin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass || SourcePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class) : false;
}

bool UK2Node_ConvertAsset::IsConvertToSoft() const
{
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	bool bIsConnected = InputPin && InputPin->LinkedTo.Num() && InputPin->LinkedTo[0];
	UEdGraphPin* SourcePin = bIsConnected ? InputPin->LinkedTo[0] : nullptr;
	return SourcePin ? (SourcePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class || SourcePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) : false;
}

bool UK2Node_ConvertAsset::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	if (InputPin && OtherPin && (InputPin == MyPin) && (MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard))
	{
		if ((OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_SoftObject) &&
			(OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_SoftClass) &&
			(OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Object) &&
			(OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Class))
		{
			return true;
		}
	}
	return false;
}

void UK2Node_ConvertAsset::RefreshPinTypes()
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	UEdGraphPin* OutputPin = FindPin(UK2Node_ConvertAssetImpl::OutputPinName);
	ensure(InputPin && OutputPin);
	if (InputPin && OutputPin)
	{
		const bool bIsConnected = InputPin->LinkedTo.Num() > 0;
		UClass* TargetType = bIsConnected ? GetTargetClass() : nullptr;
		const bool bIsClassType = bIsConnected ? IsClassType() : false;
		const bool bIsConvertToSoft = bIsConnected ? IsConvertToSoft() : false;

		FName InputCategory = UEdGraphSchema_K2::PC_Wildcard;
		FName OutputCategory = UEdGraphSchema_K2::PC_Wildcard;
		if (bIsConnected)
		{
			if (bIsConvertToSoft)
			{
				InputCategory = (bIsClassType ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object);
				OutputCategory = (bIsClassType ? UEdGraphSchema_K2::PC_SoftClass : UEdGraphSchema_K2::PC_SoftObject);
			}
			else
			{
				InputCategory = (bIsClassType ? UEdGraphSchema_K2::PC_SoftClass : UEdGraphSchema_K2::PC_SoftObject);
				OutputCategory = (bIsClassType ? UEdGraphSchema_K2::PC_Class : UEdGraphSchema_K2::PC_Object);
			}
		}
			
		InputPin->PinType = FEdGraphPinType(InputCategory, NAME_None, TargetType, EPinContainerType::None, false, FEdGraphTerminalType() );
		OutputPin->PinType = FEdGraphPinType(OutputCategory, NAME_None, TargetType, EPinContainerType::None, false, FEdGraphTerminalType() );

		PinTypeChanged(InputPin);
		PinTypeChanged(OutputPin);

		if (OutputPin->LinkedTo.Num())
		{
			TArray<UEdGraphPin*> PinsToUnlink = OutputPin->LinkedTo;

			UClass const* CallingContext = nullptr;
			if (UBlueprint const* Blueprint = GetBlueprint())
			{
				CallingContext = Blueprint->GeneratedClass;
				if (CallingContext == NULL)
				{
					CallingContext = Blueprint->ParentClass;
				}
			}

			for (UEdGraphPin* TargetPin : PinsToUnlink)
			{
				if (TargetPin && !K2Schema->ArePinsCompatible(OutputPin, TargetPin, CallingContext))
				{
					OutputPin->BreakLinkTo(TargetPin);
				}
			}
		}
	}
}

void UK2Node_ConvertAsset::PostReconstructNode()
{
	RefreshPinTypes();

	Super::PostReconstructNode();
}

void UK2Node_ConvertAsset::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);
	if (Pin && (UK2Node_ConvertAssetImpl::InputPinName == Pin->PinName))
	{
		RefreshPinTypes();

		GetGraph()->NotifyGraphChanged();
	}
}

void UK2Node_ConvertAsset::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, UK2Node_ConvertAssetImpl::InputPinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UK2Node_ConvertAssetImpl::OutputPinName);
}

UK2Node::ERedirectType UK2Node_ConvertAsset::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// Names changed, only thing that matters is the direction
	if (NewPin->Direction == OldPin->Direction)
	{
		return UK2Node::ERedirectType_Name;
	}
	return UK2Node::ERedirectType_None;
}

void UK2Node_ConvertAsset::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void OverrideUI(FBlueprintActionContext const& Context, IBlueprintNodeBinder::FBindingSet const& /*Bindings*/, FBlueprintActionUiSpec* UiSpecOut)
		{
			bool bShouldPromote = false;
			bool bIsMakeSoft = false;
			for (UEdGraphPin* Pin : Context.Pins)
			{
				// Auto promote for soft refs
				if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
				{
					if (Pin->Direction == EGPD_Output)
					{
						// The wildcards don't get set when connecting right side, so don't promote
						bShouldPromote = true;
					}
				}

				// Change title for hard refs
				if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class || Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
				{
					if (Pin->Direction == EGPD_Output)
					{
						bIsMakeSoft = true;
					}
				}
			}

			if (bShouldPromote)
			{
				UiSpecOut->Category = NSLOCTEXT("BlueprintFunctionNodeSpawner", "EmptyFunctionCategory", "|");
			}
			if (bIsMakeSoft)
			{
				UiSpecOut->MenuName = LOCTEXT("MakeSoftTitle", "Make Soft Reference");
			}
		}
	};

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);
		NodeSpawner->DynamicUiSignatureGetter = UBlueprintFieldNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(GetMenuActions_Utils::OverrideUI);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_ConvertAsset::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	
	UClass* TargetType = GetTargetClass();
	if (TargetType && Schema && (2 == Pins.Num()))
	{
		const bool bIsClassType = IsClassType();
		UClass *TargetClass = GetTargetClass();
		bool bIsErrorFree = true;

		if (IsConvertToSoft())
		{
			//Create Convert Function
			UK2Node_CallFunction* ConvertToObjectFunc = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			FName ConvertFunctionName = bIsClassType
				? GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_ClassToSoftClassReference)
				: GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_ObjectToSoftObjectReference);

			ConvertToObjectFunc->FunctionReference.SetExternalMember(ConvertFunctionName, UKismetSystemLibrary::StaticClass());
			ConvertToObjectFunc->AllocateDefaultPins();

			//Connect input to convert
			UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
			const FName ConvertInputName = bIsClassType ? FName(TEXT("Class")) : FName(TEXT("Object"));
			UEdGraphPin* ConvertInput = ConvertToObjectFunc->FindPin(ConvertInputName);
			bIsErrorFree = InputPin && ConvertInput && CompilerContext.MovePinLinksToIntermediate(*InputPin, *ConvertInput).CanSafeConnect();

			if (UEdGraphPin* ConvertOutput = ConvertToObjectFunc->GetReturnValuePin())
			{
				// Force the convert output pin to the right type. This is only safe because all asset ptrs are type-compatible, the cast is done at resolution time
				ConvertOutput->PinType.PinSubCategoryObject = TargetClass;

				UEdGraphPin* OutputPin = FindPin(UK2Node_ConvertAssetImpl::OutputPinName);
				bIsErrorFree &= OutputPin && CompilerContext.MovePinLinksToIntermediate(*OutputPin, *ConvertOutput).CanSafeConnect();
			}
			else
			{
				bIsErrorFree = false;
			}
		}
		else
		{
			//Create Convert Function
			UK2Node_CallFunction* ConvertToObjectFunc = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			FName ConvertFunctionName = bIsClassType
				? GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_SoftClassReferenceToClass)
				: GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, Conv_SoftObjectReferenceToObject);

			ConvertToObjectFunc->FunctionReference.SetExternalMember(ConvertFunctionName, UKismetSystemLibrary::StaticClass());
			ConvertToObjectFunc->AllocateDefaultPins();

			//Connect input to convert
			UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
			const FName ConvertInputName = bIsClassType ? FName(TEXT("SoftClass")) : FName(TEXT("SoftObject"));
			UEdGraphPin* ConvertInput = ConvertToObjectFunc->FindPin(ConvertInputName);
			bIsErrorFree = InputPin && ConvertInput && CompilerContext.MovePinLinksToIntermediate(*InputPin, *ConvertInput).CanSafeConnect();

			UEdGraphPin* ConvertOutput = ConvertToObjectFunc->GetReturnValuePin();
			UEdGraphPin* InnerOutput = nullptr;
			if (UObject::StaticClass() != TargetType)
			{
				//Create Cast Node
				UK2Node_DynamicCast* CastNode = bIsClassType
					? CompilerContext.SpawnIntermediateNode<UK2Node_ClassDynamicCast>(this, SourceGraph)
					: CompilerContext.SpawnIntermediateNode<UK2Node_DynamicCast>(this, SourceGraph);
				CastNode->SetPurity(true);
				CastNode->TargetType = TargetType;
				CastNode->AllocateDefaultPins();

				// Connect Object/Class to Cast
				UEdGraphPin* CastInput = CastNode->GetCastSourcePin();
				bIsErrorFree &= ConvertOutput && CastInput && Schema->TryCreateConnection(ConvertOutput, CastInput);

				// Connect output to cast
				InnerOutput = CastNode->GetCastResultPin();
			}
			else
			{
				InnerOutput = ConvertOutput;
			}

			UEdGraphPin* OutputPin = FindPin(UK2Node_ConvertAssetImpl::OutputPinName);
			bIsErrorFree &= OutputPin && InnerOutput && CompilerContext.MovePinLinksToIntermediate(*OutputPin, *InnerOutput).CanSafeConnect();
		}

		if (!bIsErrorFree)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("InternalConnectionError", "K2Node_ConvertAsset: Internal connection error. @@").ToString(), this);
		}

		BreakAllNodeLinks();
	}
}

FText UK2Node_ConvertAsset::GetCompactNodeTitle() const
{
	return FText::FromString(TEXT("\x2022"));
}

FText UK2Node_ConvertAsset::GetMenuCategory() const
{
	return LOCTEXT("UK2Node_LoadAssetGetMenuCategory", "Utilities");
}

FText UK2Node_ConvertAsset::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const bool bIsConvertToSoft = IsConvertToSoft();

	if (bIsConvertToSoft)
	{
		return LOCTEXT("MakeSoftTitle", "Make Soft Reference");
	}

	return LOCTEXT("UK2Node_ConvertAssetGetNodeTitle", "Resolve Soft Reference");
}

FText UK2Node_ConvertAsset::GetKeywords() const
{
	// Return old name here
	return LOCTEXT("UK2Node_ConvertAssetGetKeywords", "Resolve Asset ID Convert Soft");
}

FText UK2Node_ConvertAsset::GetTooltipText() const
{
	UEdGraphPin* InputPin = FindPin(UK2Node_ConvertAssetImpl::InputPinName);
	const bool bIsConvertToSoft = IsConvertToSoft();
	if (bIsConvertToSoft)
	{
		return LOCTEXT("MakeSoftTooltip", "Takes a hard Class or Object reference and makes a Soft Reference.");
	}
	else if (InputPin && InputPin->LinkedTo.Num() > 0)
	{
		return LOCTEXT("UK2Node_ConvertAssetGetTooltipText", "Resolves a Soft Reference and gets the Class or Object it is pointing to. If the object isn't already loaded in memory this will return none.");
	}

	return LOCTEXT("UnknownTypeTooltip", "Resolves or makes a Soft Reference, connect a soft or hard reference to the input pin.");
}

#undef LOCTEXT_NAMESPACE
