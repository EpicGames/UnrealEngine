// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Copy.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Copy)

class FKismetCompilerContext;
struct FBPTerminal;

#define LOCTEXT_NAMESPACE "K2Node_Copy"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_Select

class FKCHandler_Copy : public FNodeHandlingFunctor
{
protected:
	TMap<UEdGraphNode*, FBPTerminal*> BoolTermMap;

public:
	FKCHandler_Copy(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		FNodeHandlingFunctor::RegisterNets(Context, Node);

		// Create the net for the return value manually as it's a special case Output Direction pin
		UK2Node_Copy* CopyNode = Cast<UK2Node_Copy>(Node);
		UEdGraphPin* CopyResultPin = CopyNode->GetCopyResultPin();

		FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(CopyResultPin, Context.NetNameMap->MakeValidName(CopyResultPin));
		Context.NetMap.Add(CopyResultPin, Term);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		// Cast the node and get all the input pins
		UK2Node_Copy* CopyNode = Cast<UK2Node_Copy>(Node);

		// Get the kismet term for the copy result
		UEdGraphPin* CopyResultPin = CopyNode->GetCopyResultPin();
		FBPTerminal** CopyResultTerm = Context.NetMap.Find(CopyResultPin);

		// Get the kismet term from the option pin
		UEdGraphPin* InputReferencePin = FEdGraphUtilities::GetNetFromPin(Node->Pins[0]);
		FBPTerminal** InputReferenceTerm = Context.NetMap.Find(InputReferencePin);

		// Copy the reference value into the output pin so the value will be returned as a copy.
		FKismetCompilerUtilities::CreateObjectAssignmentStatement(Context, Node, *InputReferenceTerm, *CopyResultTerm);
	}
};

UK2Node_Copy::UK2Node_Copy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_Copy::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, UEdGraphSchema_K2::PN_Item);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UEdGraphSchema_K2::PN_ReturnValue);

	Super::AllocateDefaultPins();
}

FText UK2Node_Copy::GetTooltipText() const
{
	return LOCTEXT("CopyNodeTooltip", "Outputs a copy of the value passed into it.");
}

FText UK2Node_Copy::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Copy", "Copy");
}

void UK2Node_Copy::PostReconstructNode()
{
	if (GetInputReferencePin()->LinkedTo.Num() > 0)
	{
		PropagatePinType(GetInputReferencePin()->LinkedTo[0]->PinType);
	}
	else if (GetCopyResultPin()->LinkedTo.Num() > 0)
	{
		PropagatePinType(GetCopyResultPin()->LinkedTo[0]->PinType);
	}
}

/** Determine if any pins are connected, if so make all the other pins the same type, if not, make sure pins are switched back to wildcards */
void UK2Node_Copy::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	if (Pin->ParentPin == nullptr)
	{
		UEdGraphPin* InputPin = GetInputReferencePin();
		UEdGraphPin* ResultPin = GetCopyResultPin();

		// If a pin connection is made, then we need to propagate the change to the other pins and validate all connections, otherwise reset them to Wildcard pins
		if (Pin->LinkedTo.Num() > 0)
		{
			PropagatePinType(Pin->LinkedTo[0]->PinType);
		}
		else if (InputPin->LinkedTo.Num() == 0 && ResultPin->LinkedTo.Num() == 0)
		{
			InputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			InputPin->PinType.PinSubCategory = TEXT("");
			InputPin->PinType.PinSubCategoryObject = NULL;

			ResultPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			ResultPin->PinType.PinSubCategory = TEXT("");
			ResultPin->PinType.PinSubCategoryObject = NULL;

			InputPin->BreakAllPinLinks();
			ResultPin->BreakAllPinLinks();
		}
	}
}

UEdGraphPin* UK2Node_Copy::GetInputReferencePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Item);
	check(Pin);
	return Pin;
}

UEdGraphPin* UK2Node_Copy::GetCopyResultPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin);
	return Pin;
}

void UK2Node_Copy::PinTypeChanged(UEdGraphPin* Pin)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	{
		// Set the return value
		UEdGraphPin* ReturnPin = GetCopyResultPin();
		if (ReturnPin->PinType != Pin->PinType)
		{
			// Recombine the sub pins back into the ReturnPin
			if (ReturnPin->SubPins.Num() > 0)
			{
				Schema->RecombinePin(ReturnPin->SubPins[0]);
			}
			ReturnPin->PinType = Pin->PinType;
			Schema->SetPinAutogeneratedDefaultValueBasedOnType(ReturnPin);
		}
	}

	// Let the graph know to refresh
	GetGraph()->NotifyGraphChanged();

	UBlueprint* Blueprint = GetBlueprint();
	if(!Blueprint->bBeingCompiled)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		Blueprint->BroadcastChanged();
	}
}

bool UK2Node_Copy::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (OtherPin)
	{
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			OutReason = LOCTEXT("ExecConnectionDisallowed", "Cannot connect with Exec pin.").ToString();
			return true;
		}
		else if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object || OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class
			|| OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject || OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass
			|| OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			OutReason = FText::Format(LOCTEXT("ObjectConnectionDisallowed", "Cannot connect with {0} pin."), FText::FromName(OtherPin->PinType.PinCategory)).ToString();
			return true;
		}
		else if (OtherPin->PinType.IsContainer())
		{
			OutReason = LOCTEXT("ArrayConnectionDisallowed", "Cannot connect with container pin.").ToString();
			return true;
		}
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

FNodeHandlingFunctor* UK2Node_Copy::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return static_cast<FNodeHandlingFunctor*>(new FKCHandler_Copy(CompilerContext));
}

void UK2Node_Copy::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_Copy::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Utilities);
}

void UK2Node_Copy::PropagatePinType(FEdGraphPinType& InType)
{
	UClass const* CallingContext = NULL;
	if (UBlueprint const* Blueprint = GetBlueprint())
	{
		CallingContext = Blueprint->GeneratedClass;
		if (CallingContext == NULL)
		{
			CallingContext = Blueprint->ParentClass;
		}
	}

	UEdGraphPin* InputPin = GetInputReferencePin();
	UEdGraphPin* ResultPin = GetCopyResultPin();

	InputPin->PinType = InType;
	ResultPin->PinType = InType;

	InputPin->PinType.ContainerType = EPinContainerType::None;
	InputPin->PinType.bIsReference = false;

	ResultPin->PinType.ContainerType = EPinContainerType::None;
	ResultPin->PinType.bIsReference = false;

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Verify that all previous connections to this pin are still valid with the new type
	for (TArray<UEdGraphPin*>::TIterator ConnectionIt(InputPin->LinkedTo); ConnectionIt; ++ConnectionIt)
	{
		UEdGraphPin* ConnectedPin = *ConnectionIt;
		if (!Schema->ArePinsCompatible(InputPin, ConnectedPin, CallingContext))
		{
			InputPin->BreakLinkTo(ConnectedPin);
		}
	}

	// Verify that all previous connections to this pin are still valid with the new type
	for (TArray<UEdGraphPin*>::TIterator ConnectionIt(ResultPin->LinkedTo); ConnectionIt; ++ConnectionIt)
	{
		UEdGraphPin* ConnectedPin = *ConnectionIt;
		if (!Schema->ArePinsCompatible(ResultPin, ConnectedPin, CallingContext))
		{
			ResultPin->BreakLinkTo(ConnectedPin);
		}
	}
}


#undef LOCTEXT_NAMESPACE
