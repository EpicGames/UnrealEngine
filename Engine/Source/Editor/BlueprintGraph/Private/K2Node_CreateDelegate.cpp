// Copyright Epic Games, Inc. All Rights Reserved.
#include "K2Node_CreateDelegate.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "DelegateNodeHandlers.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_Event.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CreateDelegate)

class FKismetCompilerContext;

namespace UE::BlueprintGraph::Private
{
	static const FName DelegateOutputName = TEXT("OutputDelegate");
	static const FName InputObjectName = TEXT("InputObject");
}

UK2Node_CreateDelegate::UK2Node_CreateDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_CreateDelegate::AllocateDefaultPins()
{
	if (UEdGraphPin* ObjPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), UEdGraphSchema_K2::PN_Self))
	{
		ObjPin->PinFriendlyName = NSLOCTEXT("K2Node", "CreateDelegate_ObjectInputName", "Object");
	}

	if(UEdGraphPin* DelegatePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Delegate, UE::BlueprintGraph::Private::DelegateOutputName))
	{
		DelegatePin->PinFriendlyName = NSLOCTEXT("K2Node", "CreateDelegate_DelegateOutName", "Event");
	}

	Super::AllocateDefaultPins();
}

UK2Node::ERedirectType UK2Node_CreateDelegate::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// Handles remap of InputObject to Self, from 4.10 time frame
	if (OldPin->PinName == UE::BlueprintGraph::Private::InputObjectName && NewPin->PinName == UEdGraphSchema_K2::PN_Self)
	{
		return ERedirectType_Name;
	}

	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

bool UK2Node_CreateDelegate::IsValid(FString* OutMsg, bool bDontUseSkeletalClassForSelf) const
{
	FName FunctionName = GetFunctionName();
	if (FunctionName == NAME_None)
	{
		if (OutMsg)
		{
			*OutMsg = NSLOCTEXT("K2Node", "No_function_name", "No function/event specified.").ToString();
		}
		return false;
	}

	const UEdGraphPin* DelegatePin = GetDelegateOutPin();
	if (!DelegatePin)
	{
		if (OutMsg)
		{
			*OutMsg = NSLOCTEXT("K2Node", "No_delegate_out_pin", "Malformed node - there's no delegate output pin.").ToString();
		}
		return false;
	}

	const UFunction* Signature = GetDelegateSignature();
	if (!Signature)
	{
		if (OutMsg)
		{
			*OutMsg = NSLOCTEXT("K2Node", "Signature_not_found", "Unable to determine expected signature - is the delegate pin connected?").ToString();
		}
		return false;
	}

	for (int PinIter = 1; PinIter < DelegatePin->LinkedTo.Num(); PinIter++)
	{
		UEdGraphPin* OtherPin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(DelegatePin->LinkedTo[PinIter]);
		
		// A null linked pin can occur if we run across an unconnected knot.
		// In this case, it's expected that it wouldn't have a valid function signature to check.
		if (OtherPin)
		{
			const UFunction* OtherSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(OtherPin->PinType.PinSubCategoryMemberReference);
			if (!OtherSignature || !Signature->IsSignatureCompatibleWith(OtherSignature))
			{
				if (OutMsg)
				{
					if (const UK2Node_BaseMCDelegate* DelegateNode = OtherPin ? Cast<const UK2Node_BaseMCDelegate>(OtherPin->GetOwningNode()) : nullptr)
					{
						const FString DelegateName = DelegateNode->GetPropertyName().ToString();

						*OutMsg = FText::Format(NSLOCTEXT("K2Node", "Bad_delegate_connection_named_fmt", "A connected delegate ({0}) has an incompatible signature - has that delegate changed?"),
							FText::FromString(DelegateName)).ToString();
					}
					else
					{
						*OutMsg = NSLOCTEXT("K2Node", "Bad_delegate_connection", "A connected delegate's signature is incompatible - has that delegate changed?").ToString();
					}
				}
				return false;
			}
		}
	}

	UClass* ScopeClass = GetScopeClass(bDontUseSkeletalClassForSelf);
	if (!ScopeClass)
	{
		if (OutMsg)
		{
			FString SelfPinName;
			if (UEdGraphPin* SelfPin = GetObjectInPin())
			{
				SelfPinName = SelfPin->PinFriendlyName.IsEmpty() ? SelfPin->PinFriendlyName.ToString() : SelfPin->PinName.ToString();
			}
			else
			{
				SelfPinName = UEdGraphSchema_K2::PN_Self.ToString();
			}

			*OutMsg = FText::Format(NSLOCTEXT("K2Node", "Class_not_found_fmt", "Unable to determine context for the selected function/event: '{0}' - make sure the target '{1}' pin is properly set up."),
				FText::FromString(FunctionName.ToString()), FText::FromString(SelfPinName)).ToString();
		}
		return false;
	}

	FMemberReference MemberReference;
	MemberReference.SetDirect(SelectedFunctionName, SelectedFunctionGuid, ScopeClass, false);
	const UFunction* FoundFunction = MemberReference.ResolveMember<UFunction>();
	if (!FoundFunction)
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(NSLOCTEXT("K2Node", "Function_not_found_fmt", "Unable to find the selected function/event: '{0}' - has it been deleted?"),
				FText::FromString(FunctionName.ToString())).ToString();

		}
		return false;
	}
	if (!Signature->IsSignatureCompatibleWith(FoundFunction))
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(NSLOCTEXT("K2Node", "Function_not_compatible_fmt", "The function/event '{0}' does not match the necessary signature - has the delegate or function/event changed?"),
				FText::FromString(FunctionName.ToString())).ToString();
		}
		return false;
	}
	if (!UEdGraphSchema_K2::FunctionCanBeUsedInDelegate(FoundFunction))
	{
		if (OutMsg)
		{
			*OutMsg = NSLOCTEXT("K2Node", "Function_cannot_be_used_in_delegate", "The selected function/event is not bindable - is the function/event deprecated, pure or latent?").ToString();
		}
		return false;
	}

	if (!FoundFunction->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
	{
		for (int PinIter = 0; PinIter < DelegatePin->LinkedTo.Num(); PinIter++)
		{
			const UEdGraphPin* OtherPin = DelegatePin->LinkedTo[PinIter];
			const UK2Node_BaseMCDelegate* Node = OtherPin ? Cast<const UK2Node_BaseMCDelegate>(OtherPin->GetOwningNode()) : NULL;
			if (Node && Node->IsAuthorityOnly())
			{
				if (OutMsg)
				{
					*OutMsg = FText::Format(NSLOCTEXT("K2Node", "WrongDelegateAuthorityOnlyFmt", "The selected function/event ('{0}') is not compatible with this delegate (the delegate is server-only) - try marking the function/event AuthorityOnly."),
						FText::FromString(FunctionName.ToString())).ToString();
				}
				return false;
			}
		}
	}

	return true;
}

void UK2Node_CreateDelegate::ValidationAfterFunctionsAreCreated(class FCompilerResultsLog& MessageLog, bool bFullCompile) const
{
	FString Msg;
	if(!IsValid(&Msg, bFullCompile))
	{
 		MessageLog.Error(*FString::Printf( TEXT("@@ %s %s"), *NSLOCTEXT("K2Node", "WrongDelegate", "Signature Error:").ToString(), *Msg), this);
	}
}

void UK2Node_CreateDelegate::HandleAnyChangeWithoutNotifying()
{
	const auto Blueprint = HasValidBlueprint() ? GetBlueprint() : nullptr;
	const auto SelfScopeClass = Blueprint ? Blueprint->SkeletonGeneratedClass : nullptr;
	const auto ParentClass = GetScopeClass();
	const bool bIsSelfScope = SelfScopeClass && ParentClass && ((SelfScopeClass->IsChildOf(ParentClass)) || (SelfScopeClass->ClassGeneratedBy == ParentClass->ClassGeneratedBy));

	FMemberReference FunctionReference;
	FunctionReference.SetDirect(SelectedFunctionName, SelectedFunctionGuid, GetScopeClass(), bIsSelfScope);

	if (FunctionReference.ResolveMember<UFunction>(SelfScopeClass))
	{
		SelectedFunctionName = FunctionReference.GetMemberName();
		SelectedFunctionGuid = FunctionReference.GetMemberGuid();

		if (!SelectedFunctionGuid.IsValid())
		{
			UBlueprint::GetGuidFromClassByFieldName<UFunction>(ParentClass, SelectedFunctionName, SelectedFunctionGuid);
		}
	}

	if(!IsValid())
	{
		// do not clear the name, so we can keep it around as a hint/guide for 
		// users (so they can better determine what went wrong)
		if (const UEdGraphPin* DelegatePin = GetDelegateOutPin())
		{
			if (DelegatePin->LinkedTo.Num() == 0)
			{
				// ok to clear if they've disconnected the delegate pin
				SelectedFunctionName = NAME_None;
			}
		}
		SelectedFunctionGuid.Invalidate();
	}
}

void UK2Node_CreateDelegate::HandleAnyChange(UEdGraph*& OutGraph, UBlueprint*& OutBlueprint)
{
	const FName OldSelectedFunctionName = GetFunctionName();
	HandleAnyChangeWithoutNotifying();
	if (OldSelectedFunctionName != GetFunctionName())
	{
		OutGraph = GetGraph();
		OutBlueprint = GetBlueprint();
	}
	else
	{
		OutGraph = nullptr;
		OutBlueprint = nullptr;
	}
}

void UK2Node_CreateDelegate::HandleAnyChange(bool bForceModify)
{
	const FName OldSelectedFunctionName = GetFunctionName();
	HandleAnyChangeWithoutNotifying();
	if (bForceModify || (OldSelectedFunctionName != GetFunctionName()))
	{
		if(UEdGraph* Graph = GetGraph())
		{
			Graph->NotifyGraphChanged();
		}

		UBlueprint* Blueprint = GetBlueprint();
		if(Blueprint && !Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			Blueprint->BroadcastChanged();
		}
	}
	else if (GetFunctionName() == NAME_None)
	{
		if(UEdGraph* Graph = GetGraph())
		{
			Graph->NotifyGraphChanged();
		}
	}
}

void UK2Node_CreateDelegate::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::PinConnectionListChanged(Pin);
	UBlueprint* Blueprint = GetBlueprint();
	if(Blueprint && !Blueprint->bBeingCompiled)
	{
		HandleAnyChange();
	}
	else
	{
		HandleAnyChangeWithoutNotifying();
	}
}

void UK2Node_CreateDelegate::PinTypeChanged(UEdGraphPin* Pin)
{
	Super::PinTypeChanged(Pin);

	HandleAnyChangeWithoutNotifying();
}

void UK2Node_CreateDelegate::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();
	UBlueprint* Blueprint = GetBlueprint();
	if(Blueprint && !Blueprint->bBeingCompiled)
	{
		HandleAnyChange();
	}
	else
	{
		HandleAnyChangeWithoutNotifying();
	}
}

void UK2Node_CreateDelegate::PostReconstructNode()
{
	Super::PostReconstructNode();
	HandleAnyChange();
}

UFunction* UK2Node_CreateDelegate::GetDelegateSignature() const
{
	UEdGraphPin* Pin = GetDelegateOutPin();
	check(Pin);
	UFunction* Result = nullptr;

	if (Pin->LinkedTo.Num())
	{
		if (UEdGraphPin* ResultPin = Pin->LinkedTo[0])
		{
			// The knot nodes may not necessarily have the correct info for PinSubCategoryMemberReference.
			// The safer approach is to traverse the knots to find the endpoint node, and then use its 
			// PinSubCategoryMemberReference during resolution of the member reference.

			UEdGraphPin* LinkedPin = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(ResultPin);

			if (LinkedPin && (UEdGraphSchema_K2::PC_Delegate == LinkedPin->PinType.PinCategory))
			{
				Result = FMemberReference::ResolveSimpleMemberReference<UFunction>(LinkedPin->PinType.PinSubCategoryMemberReference);
			}
		}
	}

	return Result;
}

UClass* UK2Node_CreateDelegate::GetScopeClass(bool bDontUseSkeletalClassForSelf/* = false*/) const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Self);
	if (Pin == nullptr)
	{
		// The BlueprintNodeTemplateCache creates nodes but doesn't call allocate default pins.
		// SMyBlueprint::OnDeleteGraph calls this function on every UK2Node_CreateDelegate. Each of 
		// these systems is violating some first principles, so I've settled on this null check.
		return nullptr;
	}
	check(Pin->LinkedTo.Num() <= 1);
	bool bUseSelf = false;
	if(Pin->LinkedTo.Num() == 0)
	{
		bUseSelf = true;
	}
	else
	{
		if(UEdGraphPin* ResultPin = Pin->LinkedTo[0])
		{
			ensure(UEdGraphSchema_K2::PC_Object == ResultPin->PinType.PinCategory);
			if (UEdGraphSchema_K2::PN_Self == ResultPin->PinType.PinSubCategory)
			{
				bUseSelf = true;
			}
			if(UClass* TrueScopeClass = Cast<UClass>(ResultPin->PinType.PinSubCategoryObject.Get()))
			{
				if(UBlueprint* ScopeClassBlueprint = Cast<UBlueprint>(TrueScopeClass->ClassGeneratedBy))
				{
					if(ScopeClassBlueprint->SkeletonGeneratedClass)
					{
						return ScopeClassBlueprint->SkeletonGeneratedClass;
					}
				}
				return TrueScopeClass;
			}
		}
	}

	if (bUseSelf && HasValidBlueprint())
	{
		if (UBlueprint* ScopeClassBlueprint = GetBlueprint())
		{
			return bDontUseSkeletalClassForSelf ? ScopeClassBlueprint->GeneratedClass : ScopeClassBlueprint->SkeletonGeneratedClass;
		}
	}

	return nullptr;
}

FName UK2Node_CreateDelegate::GetFunctionName() const
{
	return SelectedFunctionName;
}

UEdGraphPin* UK2Node_CreateDelegate::GetDelegateOutPin() const
{
	return FindPin(UE::BlueprintGraph::Private::DelegateOutputName);
}

UEdGraphPin* UK2Node_CreateDelegate::GetObjectInPin() const
{
	return FindPin(UEdGraphSchema_K2::PN_Self);
}

FText UK2Node_CreateDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "CreateDelegate", "Create Event");
}

UObject* UK2Node_CreateDelegate::GetJumpTargetForDoubleClick() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UClass* ScopeClass = GetScopeClass();

	if (UBlueprint* ScopeClassBlueprint = (ScopeClass != nullptr) ? Cast<UBlueprint>(ScopeClass->ClassGeneratedBy) : nullptr)
	{
		if (UEdGraph* FoundGraph = FindObject<UEdGraph>(ScopeClassBlueprint, *GetFunctionName().ToString()))
		{
			if (!FBlueprintEditorUtils::IsGraphIntermediate(FoundGraph))
			{
				return FoundGraph;
			}
		}
		for (auto UbergraphIt = ScopeClassBlueprint->UbergraphPages.CreateIterator(); UbergraphIt; ++UbergraphIt)
		{
			UEdGraph* Graph = (*UbergraphIt);
			if (!FBlueprintEditorUtils::IsGraphIntermediate(Graph))
			{
				TArray<UK2Node_Event*> EventNodes;
				Graph->GetNodesOfClass(EventNodes);
				for (UK2Node_Event* EventNode : EventNodes)
				{
					if (GetFunctionName() == EventNode->GetFunctionName())
					{
						return EventNode;
					}
				}
			}
		}
	}

	return GetDelegateSignature();
}

void UK2Node_CreateDelegate::FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);

	UK2Node_CreateDelegate* OtherCreateDelegate = Cast<UK2Node_CreateDelegate>(OtherNode);
	if (OtherCreateDelegate && OtherCreateDelegate->SelectedFunctionName != SelectedFunctionName)
	{		
		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::NODE_PROPERTY;
		Diff.Node1 = this;
		Diff.Node2 = OtherNode;
		Diff.DisplayString = FText::Format(
			NSLOCTEXT("K2Node", "DIF_SignatureFunction", "Signature function changed from {0} to {1}"), 
			FText::FromName(SelectedFunctionName), 
			FText::FromName(OtherCreateDelegate->SelectedFunctionName));
		Diff.Category = EDiffType::MODIFICATION;

		Results.Add(Diff);
	}
}

void UK2Node_CreateDelegate::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	const FName FunctionName = GetFunctionName();
	if (!FunctionName.IsNone())
	{
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromName(FunctionName)));

		if (const UClass* ScopeClass = GetScopeClass(/*bDontUseSkeletalClassForSelf=*/true))
		{
			const FString FuncOriginClassName = ScopeClass->GetPathName();
			OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_FuncOriginClass, FText::FromString(FuncOriginClassName)));
		}
	}
}

FNodeHandlingFunctor* UK2Node_CreateDelegate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_CreateDelegate(CompilerContext);
}

void UK2Node_CreateDelegate::SetFunction(FName Name)
{
	SelectedFunctionName = Name;
	SelectedFunctionGuid.Invalidate();
}

FText UK2Node_CreateDelegate::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Delegates);
}

void UK2Node_CreateDelegate::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* NodeClass = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(NodeClass))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
		check(NodeSpawner);
		ActionRegistrar.AddBlueprintAction(NodeClass, NodeSpawner);
	}
}
