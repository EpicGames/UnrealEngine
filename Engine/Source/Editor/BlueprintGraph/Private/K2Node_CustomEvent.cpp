// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_CustomEvent.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintEventNodeSpawner.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "FindInBlueprintManager.h"
#include "FindInBlueprints.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_BaseMCDelegate.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_CustomEvent)

struct FLinearColor;

#define LOCTEXT_NAMESPACE "K2Node_CustomEvent"

/**
 * Attempts to find a CustomEvent node associated with the specified function.
 * 
 * @param  CustomEventFunc	The function you want to find an associated node for.
 * @return A pointer to the found node (NULL if a corresponding node wasn't found)
 */
static const UK2Node_CustomEvent* FindCustomEventNodeFromFunction(UFunction* CustomEventFunc)
{
	const UK2Node_CustomEvent* FoundEventNode = nullptr;
	if (CustomEventFunc != nullptr)
	{
		const UObject* const FuncOwner = CustomEventFunc->GetOuter();
		check(FuncOwner != nullptr);

		// if the found function is a NOT a native function (it's user generated)
		if (FuncOwner->IsA(UBlueprintGeneratedClass::StaticClass()))
		{
			const UBlueprintGeneratedClass* FuncClass = Cast<UBlueprintGeneratedClass>(CustomEventFunc->GetOuter());
			check(FuncClass != nullptr);
			const UBlueprint* FuncBlueprint = Cast<UBlueprint>(FuncClass->ClassGeneratedBy);
			check(FuncBlueprint != nullptr);

			TArray<UK2Node_CustomEvent*> BpCustomEvents;
			FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_CustomEvent>(FuncBlueprint, BpCustomEvents);

			// look to see if the function that this is overriding is a custom-event
			for (const UK2Node_CustomEvent* const UserEvent : BpCustomEvents)
			{
				check(UserEvent);
				if (UserEvent->CustomFunctionName == CustomEventFunc->GetFName())
				{
					FoundEventNode = UserEvent;
					break;
				}
			}
		}
	}

	return FoundEventNode;
}

/**
 * Custom handler for validating CustomEvent renames
 */
class FCustomEventNameValidator : public FKismetNameValidator
{
public:
	FCustomEventNameValidator(UK2Node_CustomEvent const* CustomEventIn)
		: FKismetNameValidator(CustomEventIn->GetBlueprint(), CustomEventIn->CustomFunctionName)
		, CustomEvent(CustomEventIn)
	{
		check(CustomEvent != nullptr);
	}

	// Begin INameValidatorInterface
	virtual EValidatorResult IsValid(FString const& Name, bool bOriginal = false) override
	{
		UBlueprint* Blueprint = CustomEvent->GetBlueprint();
		check(Blueprint != nullptr);

		EValidatorResult NameValidity = FKismetNameValidator::IsValid(Name, bOriginal);
		if ((NameValidity == EValidatorResult::Ok) || (NameValidity == EValidatorResult::ExistingName))
		{
			UFunction* ParentFunction = FindUField<UFunction>(Blueprint->ParentClass, *Name);
			// if this custom-event is overriding a function belonging to the blueprint's parent
			if (ParentFunction != nullptr)
			{
				UK2Node_CustomEvent const* OverriddenEvent = FindCustomEventNodeFromFunction(ParentFunction);
				// if the function that we're overriding isn't another custom event,
				// then we can't name it this (only allow custom-event to override other custom-events)
				if (OverriddenEvent == nullptr)
				{
					NameValidity = EValidatorResult::AlreadyInUse;
				}		
			}
		}
		else if (NameValidity == EValidatorResult::AlreadyInUse)
		{
			auto Predicate_EventGraphs = [Name](const TObjectPtr<UEdGraph>& InEventGraph) -> bool
			{
				return InEventGraph && InEventGraph->HasAnyFlags(RF_Transient) && InEventGraph->GetName() == Name;
			};

			// Allow a transient event subgraph (compiler artifact) that matches the existing name
			// to pass if there are no other event nodes that would use this name at compile time.
			// This type of collision is a false positive that won't result in a conflict, because
			// the custom event node won't enter the Blueprint's namespace until the next compile,
			// and the compiler will regenerate the transient event subgraphs array on a full pass.
			if (Blueprint->EventGraphs.FindByPredicate(Predicate_EventGraphs))
			{
				TArray<UK2Node_Event*> AllEventNodes;
				FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(Blueprint, AllEventNodes);
				UK2Node_Event** MatchingNodePtr = AllEventNodes.FindByPredicate([Name](UK2Node_Event* InEventNode)
				{
					if (InEventNode->bOverrideFunction)
					{
						return InEventNode->EventReference.GetMemberName().ToString() == Name;
					}
					else if (InEventNode->CustomFunctionName != NAME_None)
					{
						return InEventNode->CustomFunctionName.ToString() == Name;
					}
					
					return false;
				});

				if (!MatchingNodePtr || *MatchingNodePtr == CustomEvent)
				{
					NameValidity = EValidatorResult::Ok;
				}
			}
		}
		return NameValidity;
	}
	// End INameValidatorInterface

private:
	UK2Node_CustomEvent const* CustomEvent;
};

UK2Node_CustomEvent::UK2Node_CustomEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bOverrideFunction = false;
	bIsEditable = true;
	bCanRenameNode = true;
	bIsDeprecated = false;
	bCallInEditor = false;

	FunctionFlags = (FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public);
}

void UK2Node_CustomEvent::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		CachedNodeTitle.MarkDirty();

		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AccessSpecifiersForCustomEvents)
		{
			FunctionFlags |= (FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_Public);
		}
	}
}

FText UK2Node_CustomEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType != ENodeTitleType::FullTitle)
	{
		return FText::FromName(CustomFunctionName);
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FText RPCString = UK2Node_Event::GetLocalizedNetString(FunctionFlags, false);
		
		FFormatNamedArguments Args;
		Args.Add(TEXT("FunctionName"), FText::FromName(CustomFunctionName));
		Args.Add(TEXT("RPCString"), RPCString);

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "CustomEvent_Name", "{FunctionName}{RPCString}\nCustom Event"), Args), this);
	}

	return CachedNodeTitle;
}

bool UK2Node_CustomEvent::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	if (!IsEditable())
	{
		return false;
	}

	// Make sure that if this is an exec node we are allowed one.
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if(InDesiredDirection == EGPD_Input)
	{
		OutErrorMessage = NSLOCTEXT("K2Node", "AddInputPinError", "Cannot add input pins to custom event node!");
		return false;
	}
	else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Exec && !CanModifyExecutionWires())
	{
		OutErrorMessage = LOCTEXT("MultipleExecPinError", "Cannot support more exec pins!");
		return false;
	}
	else
	{
		TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>> TypeTree;
		Schema->GetVariableTypeTree(TypeTree, ETypeTreeFilter::RootTypesOnly);

		bool bIsValid = false;
		for (TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>& TypeInfo : TypeTree)
		{
			FEdGraphPinType CurrentType = TypeInfo->GetPinType(false);
			// only concerned with the list of categories
			if (CurrentType.PinCategory == InPinType.PinCategory)
			{
				bIsValid = true;
				break;
			}
		}

		if (!bIsValid)
		{
			OutErrorMessage = LOCTEXT("AddInputPinError", "Cannot add pins of this type to custom event node!");
			return false;
		}
	}

	return true;
}

UEdGraphPin* UK2Node_CustomEvent::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	UEdGraphPin* NewPin = CreatePin(EGPD_Output, NewPinInfo->PinType, NewPinInfo->PinName);
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	K2Schema->SetPinAutogeneratedDefaultValue(NewPin, NewPinInfo->PinDefaultValue);
	return NewPin;
}

bool UK2Node_CustomEvent::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(PinInfo, NewDefaultValue))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);

		return true;
	}
	return false;
}

void UK2Node_CustomEvent::RenameCustomEventCloseToName(int32 StartIndex)
{
	bool bFoundName = false;
	const FString& BaseName = CustomFunctionName.ToString();

	for (int32 NameIndex = StartIndex; !bFoundName; ++NameIndex)
	{
		const FString NewName = FString::Printf(TEXT("%s_%d"), *BaseName, NameIndex);
		if (Rename(*NewName, GetOuter(), REN_Test))
		{
			UBlueprint* Blueprint = GetBlueprint();
			CustomFunctionName = FName(NewName.GetCharArray().GetData());
			Rename(*NewName, GetOuter(), (Blueprint->bIsRegeneratingOnLoad ? REN_AllowPackageLinkerMismatch : 0) | REN_DontCreateRedirectors);
			bFoundName = true;
		}
	}
}

void UK2Node_CustomEvent::OnRenameNode(const FString& NewName)
{
	CustomFunctionName = *NewName;
	CachedNodeTitle.MarkDirty();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

TSharedPtr<class INameValidatorInterface> UK2Node_CustomEvent::MakeNameValidator() const
{
	return MakeShareable(new FCustomEventNameValidator(this));
}

bool UK2Node_CustomEvent::IsOverride() const
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint != NULL);

	UFunction* ParentFunction = FindUField<UFunction>(Blueprint->ParentClass, CustomFunctionName);
	UK2Node_CustomEvent const* OverriddenEvent = FindCustomEventNodeFromFunction(ParentFunction);

	return (OverriddenEvent != NULL);
}

uint32 UK2Node_CustomEvent::GetNetFlags() const
{
	uint32 NetFlags = (FunctionFlags & FUNC_NetFuncFlags);
	if (IsOverride())
	{
		UBlueprint* Blueprint = GetBlueprint();
		check(Blueprint != NULL);

		UFunction* ParentFunction = FindUField<UFunction>(Blueprint->ParentClass, CustomFunctionName);
		check(ParentFunction != NULL);

		// inherited net flags take precedence 
		NetFlags = (ParentFunction->FunctionFlags & FUNC_NetFuncFlags);
	}

	// Sanitize NetFlags, only allow replication flags that can be supported by the online system
	// This mirrors logic in ProcessFunctionSpecifiers in HeaderParser.cpp. Basically if we want to 
	// replicate a function we need to know whether we're replicating on the client or the server.
	if (!(NetFlags & FUNC_Net))
	{
		NetFlags = 0;
	}

	return NetFlags;
}

void UK2Node_CustomEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint != NULL);

	UFunction* ParentFunction = FindUField<UFunction>(Blueprint->ParentClass, CustomFunctionName);
	// if this custom-event is overriding a function belonging to the blueprint's parent
	if (ParentFunction != NULL)
	{
		UObject const* const FuncOwner = ParentFunction->GetOuter();
		check(FuncOwner != NULL);

		// if this custom-event is attempting to override a native function, we can't allow that
		if (!FuncOwner->IsA(UBlueprintGeneratedClass::StaticClass()))
		{
			MessageLog.Error(*FText::Format(LOCTEXT("NativeFunctionConflictFmt", "@@ name conflicts with a native '{0}' function"), FText::FromString(FuncOwner->GetName())).ToString(), this);
		}
		else 
		{
			UK2Node_CustomEvent const* OverriddenEvent = FindCustomEventNodeFromFunction(ParentFunction);
			// if the function that this is attempting to override is NOT another 
			// custom-event, then we want to error (a custom-event shouldn't override something different)
			if (OverriddenEvent == NULL)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("NonCustomEventOverride", "@@ name conflicts with a '{0}' function"), FText::FromString(FuncOwner->GetName())).ToString(), this);
			}
			// else, we assume the user was attempting to override the parent's custom-event
			// the signatures could still be off, but FKismetCompilerContext::PrecompileFunction() should catch that
		}		
	}
}

void UK2Node_CustomEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintEventNodeSpawner::Create(GetClass(), FName());
		check(NodeSpawner != nullptr);

		auto SetupCustomEventNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
		{
			UK2Node_CustomEvent* EventNode = CastChecked<UK2Node_CustomEvent>(NewNode);
			UBlueprint* Blueprint = EventNode->GetBlueprint();

			// in GetNodeTitle(), we use an empty CustomFunctionName to identify a menu entry
			if (!bIsTemplateNode)
			{
				EventNode->CustomFunctionName = FBlueprintEditorUtils::FindUniqueCustomEventName(Blueprint);
			}
			EventNode->bIsEditable = true;
		};

		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(SetupCustomEventNodeLambda);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_CustomEvent::FixupPinStringDataReferences(FArchive* SavingArchive)
{
	Super::FixupPinStringDataReferences(SavingArchive);
	if (SavingArchive)
	{ 
		UpdateUserDefinedPinDefaultValues();
	}
}

void UK2Node_CustomEvent::ReconstructNode()
{
	CachedNodeTitle.MarkDirty();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const UEdGraphPin* DelegateOutPin = FindPin(DelegateOutputName);
	const UEdGraphPin* LinkedPin = ( DelegateOutPin && DelegateOutPin->LinkedTo.Num() && DelegateOutPin->LinkedTo[0] ) ? FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(DelegateOutPin->LinkedTo[0]) : nullptr;

	const UFunction* DelegateSignature = nullptr;

	if ( LinkedPin )
	{
		if ( const UK2Node_BaseMCDelegate* OtherNode = Cast<const UK2Node_BaseMCDelegate>(LinkedPin->GetOwningNode()) )
		{
			DelegateSignature = OtherNode->GetDelegateSignature();
		}
		else if ( LinkedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate )
		{
			DelegateSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(LinkedPin->PinType.PinSubCategoryMemberReference);
		}
	}
	
	const bool bUseDelegateSignature = (nullptr == FindEventSignatureFunction()) && DelegateSignature;

	if (bUseDelegateSignature)
	{
		SetDelegateSignature(DelegateSignature);
	}

	Super::ReconstructNode();
}

void UK2Node_CustomEvent::SetDelegateSignature(const UFunction* DelegateSignature)
{
	if (DelegateSignature == nullptr)
	{
		return;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	
	TArray < TSharedPtr<FUserPinInfo> > OldPins = UserDefinedPins;
	UserDefinedPins.Empty();
	for (TFieldIterator<FProperty> PropIt(DelegateSignature); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		const FProperty* Param = *PropIt;
		if (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm))
		{
			FEdGraphPinType PinType;
			K2Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);

			FName NewPinName = Param->GetFName();
			int32 Index = 1;
			while ((DelegateOutputName == NewPinName) || (UEdGraphSchema_K2::PN_Then == NewPinName))
			{
				++Index;
				NewPinName = *FString::Printf(TEXT("%s%d"), *NewPinName.ToString(), Index);
			}
			TSharedPtr<FUserPinInfo> NewPinInfo = MakeShareable(new FUserPinInfo());
			NewPinInfo->PinName = NewPinName;
			NewPinInfo->PinType = PinType;
			NewPinInfo->DesiredPinDirection = EGPD_Output;
			int32 NewIndex = UserDefinedPins.Num();

			// Copy over old default value if type matches
			if (OldPins.IsValidIndex(NewIndex) && OldPins[NewIndex].IsValid())
			{
				TSharedPtr<FUserPinInfo> OldPinInfo = OldPins[NewIndex];
				if (NewPinInfo->PinName == OldPinInfo->PinName && NewPinInfo->PinType == OldPinInfo->PinType && NewPinInfo->DesiredPinDirection == OldPinInfo->DesiredPinDirection)
				{
					NewPinInfo->PinDefaultValue = OldPinInfo->PinDefaultValue;
				}
			}

			UserDefinedPins.Add(NewPinInfo);
		}
	}
}

UK2Node_CustomEvent* UK2Node_CustomEvent::CreateFromFunction(FVector2D GraphPosition, UEdGraph* ParentGraph, const FString& Name, const UFunction* Function, bool bSelectNewNode/* = true*/)
{
	UK2Node_CustomEvent* CustomEventNode = NULL;
	if(ParentGraph && Function)
	{
		CustomEventNode = NewObject<UK2Node_CustomEvent>(ParentGraph);
		CustomEventNode->CustomFunctionName = FName(*Name);
		CustomEventNode->SetFlags(RF_Transactional);
		ParentGraph->Modify();
		ParentGraph->AddNode(CustomEventNode, true, bSelectNewNode);
		CustomEventNode->CreateNewGuid();
		CustomEventNode->PostPlacedNewNode();
		CustomEventNode->AllocateDefaultPins();

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;
			if (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				FEdGraphPinType PinType;
				K2Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);
				CustomEventNode->CreateUserDefinedPin(Param->GetFName(), PinType, EGPD_Output);
			}
		}

		CustomEventNode->NodePosX = static_cast<int32>(GraphPosition.X);
		CustomEventNode->NodePosY = static_cast<int32>(GraphPosition.Y);
		CustomEventNode->SnapToGrid(GetDefault<UEditorStyleSettings>()->GridSnapSize);
	}

	return CustomEventNode;
}

bool UK2Node_CustomEvent::IsEditable() const
{
	const UEdGraphPin* DelegateOutPin = FindPin(DelegateOutputName);
	if(DelegateOutPin && DelegateOutPin->LinkedTo.Num())
	{
		return false;
	}
	return Super::IsEditable();
}

bool UK2Node_CustomEvent::IsUsedByAuthorityOnlyDelegate() const
{
	if(const UEdGraphPin* DelegateOutPin = FindPin(DelegateOutputName))
	{
		for(auto PinIter = DelegateOutPin->LinkedTo.CreateConstIterator(); PinIter; ++PinIter)
		{
			const UEdGraphPin* LinkedPin = *PinIter;
			const UK2Node_BaseMCDelegate* Node = LinkedPin ? Cast<const UK2Node_BaseMCDelegate>(LinkedPin->GetOwningNode()) : NULL;
			if(Node && Node->IsAuthorityOnly())
			{
				return true;
			}
		}
	}

	return false;
}

FText UK2Node_CustomEvent::GetTooltipText() const
{
	return LOCTEXT("AddCustomEvent_Tooltip", "An event with customizable name and parameters.");
}

FString UK2Node_CustomEvent::GetDocumentationLink() const
{
	// Use the main k2 node doc
	return UK2Node::GetDocumentationLink();
}

FString UK2Node_CustomEvent::GetDocumentationExcerptName() const
{
	return TEXT("UK2Node_CustomEvent");
}

FSlateIcon UK2Node_CustomEvent::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), bCallInEditor ? "GraphEditor.CallInEditorEvent_16x" : "GraphEditor.CustomEvent_16x");
}

void UK2Node_CustomEvent::AutowireNewNode(UEdGraphPin* FromPin)
{
	Super::AutowireNewNode(FromPin);

	if (auto DelegateOutPin = FindPin(DelegateOutputName))
	{
		if (DelegateOutPin->LinkedTo.Num())
		{
			ReconstructNode();
		}
	}
}

void UK2Node_CustomEvent::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	bool bNeedsNameUpdate = true;
	bool bNeedsNativeNameUpdate = true;
	for (FSearchTagDataPair& SearchData : OutTaggedMetaData)
	{
		// Should always be the first item, but there is no guarantee
		if (bNeedsNameUpdate && SearchData.Key.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
		{
			SearchData.Value = FText::FromString(FName::NameToDisplayString(CustomFunctionName.ToString(), false));
			bNeedsNameUpdate = false;
		}
		else if (bNeedsNativeNameUpdate && SearchData.Key.CompareTo(FFindInBlueprintSearchTags::FiB_NativeName) == 0)
		{
			SearchData.Value = FText::FromName(CustomFunctionName);
			bNeedsNativeNameUpdate = false;
		}

		// If no more keys need updating, break
		if (!bNeedsNameUpdate && !bNeedsNativeNameUpdate)
		{
			break;
		}
	}
}

FText UK2Node_CustomEvent::GetKeywords() const
{
	FText ParentKeywords = Super::GetKeywords();

	FFormatNamedArguments Args;
	Args.Add(TEXT("ParentKeywords"), ParentKeywords);
	return FText::Format(LOCTEXT("CustomEventKeywords", "{ParentKeywords} Custom"), Args);
}

FEdGraphNodeDeprecationResponse UK2Node_CustomEvent::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		// Only warn on override usage.
		if (IsOverride())
		{
			FText EventName = FText::FromName(GetFunctionName());
			FText DetailedMessage = FText::FromString(DeprecationMessage);
			Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(EventName, DetailedMessage);
		}
		else
		{
			// Allow the source event to be marked as deprecated in the class that defines it without warning, but use a note to visually indicate that the definition itself has been deprecated.
			Response.MessageType = EEdGraphNodeDeprecationMessageType::Note;
			Response.MessageText = LOCTEXT("DeprecatedCustomEventMessage", "@@: This custom event has been marked as deprecated. It can be safely deleted if all references have been replaced or removed.");
		}
	}

	return Response;
}

bool UK2Node_CustomEvent::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	bool bResult = false;

	// We use the dependencies of the linked node instead of the resulting function signature because a globally defined 
	// delegate won't match a dependency check (it has no owner). 
	const UEdGraphPin* DelegateOutPin = FindPin(DelegateOutputName);
	const UEdGraphPin* LinkedPin = (DelegateOutPin && DelegateOutPin->LinkedTo.Num() && DelegateOutPin->LinkedTo[0]) ? FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(DelegateOutPin->LinkedTo[0]) : nullptr;
	if (LinkedPin)
	{
		if (UK2Node* OtherNode = Cast<UK2Node>(LinkedPin->GetOwningNode()))
		{
			bResult = OtherNode->HasExternalDependencies(OptionalOutput);
		}
	}

	bResult |= Super::HasExternalDependencies(OptionalOutput);
	return bResult;
}

#undef LOCTEXT_NAMESPACE
