// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DeadClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_DeadClass)

UK2Node_DeadClass::UK2Node_DeadClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	return;
}
