// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DelegateSet.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Event.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_DelegateSet)

#define LOCTEXT_NAMESPACE "K2Node_DelegateSet"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_BindToMulticastDelegate

class FKCHandler_BindToMulticastDelegate : public FNodeHandlingFunctor
{
protected:
	TMap<UEdGraphNode*, FBPTerminal*> LocalDelegateMap;

public:
	FKCHandler_BindToMulticastDelegate(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_DelegateSet* DelegateNode = Cast<UK2Node_DelegateSet>(Node);
		if( DelegateNode )
		{
			CompilerContext.MessageLog.Warning(*FString(*LOCTEXT("DeprecatedDelegateSet_Warning", "DelegateSet node @@ is Deprecated. It should be replaced by an EventCaller Bind node").ToString()), DelegateNode);

			// Create a term to store the locally created delegate that we'll use to add to the MC delegate
			FBPTerminal* DelegateTerm = Context.CreateLocalTerminal();
			DelegateTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Delegate;
			FMemberReference::FillSimpleMemberReference<UFunction>(DelegateNode->GetDelegateSignature(), DelegateTerm->Type.PinSubCategoryMemberReference);
			DelegateTerm->Source = Node;
			DelegateTerm->Name = Context.NetNameMap->MakeValidName(Node, TEXT("TempBindingDelegate"));
			LocalDelegateMap.Add(Node, DelegateTerm);

			// The only net we need to register for this node is the delegate's target (self) pin, since the others are expanded to their own event node
			RegisterDelegateNet(Context, DelegateNode);
		}
	}

	void RegisterDelegateNet(FKismetFunctionContext& Context, UK2Node_DelegateSet* DelegateNode)
	{
		check(DelegateNode);

		UEdGraphPin* DelegatePin = DelegateNode->GetDelegateOwner();
		check(DelegatePin);

		// Find the property on the specified scope
		FProperty* BoundProperty = NULL;
		for (TFieldIterator<FProperty> It(DelegateNode->DelegatePropertyClass, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			FProperty* Prop = *It;
			if( Prop->GetFName() == DelegateNode->DelegatePropertyName )
			{
				check(Prop->HasAllPropertyFlags(CPF_BlueprintAssignable));
				BoundProperty = Prop;
				break;
			}
		}

		// Create a term for this property
		if( BoundProperty != NULL )
		{
			FBPTerminal* Term = new FBPTerminal();
			Context.VariableReferences.Add(Term);
			Term->CopyFromPin(DelegatePin, DelegatePin->PinName);
			Term->AssociatedVarProperty = BoundProperty;

			Context.NetMap.Add(DelegatePin, Term);

			// Find the context for this term (the object owning the delegate property)
			FBPTerminal** pContextTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(DelegatePin));
			if( pContextTerm )
			{
				Term->Context = *pContextTerm;
			}
			else
			{
				CompilerContext.MessageLog.Error(*FString(*LOCTEXT("FindDynamicallyBoundDelegate_Error", "Couldn't find target for dynamically bound delegate node @@").ToString()), DelegateNode);
			}
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		const UK2Node_DelegateSet* DelegateNode = Cast<UK2Node_DelegateSet>(Node);
		check(DelegateNode);

		// Verify that the event has a darget to be bound to
		UEdGraphPin* DelegateOwnerPin = DelegateNode->GetDelegateOwner();

		if (DelegateOwnerPin == nullptr || DelegateOwnerPin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Error(*FString(*LOCTEXT("FindDynamicallyBoundDelegate_Error", "Couldn't find target for dynamically bound delegate node @@").ToString()), DelegateNode);
			return;
		}

		FBPTerminal** pDelegateOwnerTerm = Context.NetMap.Find(DelegateOwnerPin);

		// Create a delegate name term
		FBPTerminal* DelegateNameTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		DelegateNameTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Name;
		DelegateNameTerm->Name = DelegateNode->GetDelegateTargetEntryPointName().ToString();
		DelegateNameTerm->bIsLiteral = true;

		// Create a local delegate, which we can then add to the multicast delegate
		FBPTerminal* LocalDelegate = *LocalDelegateMap.Find(Node);
		FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
		Statement.Type = KCST_Assignment;
		Statement.LHS = LocalDelegate;
		Statement.RHS.Add(DelegateNameTerm);

		// Add the local delegate to the MC delegate
		FBlueprintCompiledStatement& AddStatement = Context.AppendStatementForNode(Node);
		AddStatement.Type = KCST_AddMulticastDelegate;
		AddStatement.LHS = *pDelegateOwnerTerm;
		AddStatement.RHS.Add(LocalDelegate);

		GenerateSimpleThenGoto(Context, *Node, DelegateNode->FindPin(UEdGraphSchema_K2::PN_Then));

		FNodeHandlingFunctor::Compile(Context, Node);
	}
};

UK2Node_DelegateSet::UK2Node_DelegateSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_DelegateSet::AllocateDefaultPins()
{
	// Cache off the delegate signature, which will update the DelegatePropertyName as well, if it's been redirected
	UFunction* DelegateSignature = GetDelegateSignature();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, DelegatePropertyClass, DelegatePropertyName);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_DelegateEntry);
	
	CreatePinsForFunctionEntryExit(DelegateSignature, true);

	Super::AllocateDefaultPins();
}

FText UK2Node_DelegateSet::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "CreateEventForDelegate", "Create an event tied to the delegate {0}"), FText::FromName(DelegatePropertyName)), this);
		if (UFunction* Function = GetDelegateSignature())
		{
			const FText SignatureTooltip = Function->GetToolTipText();

			if (!SignatureTooltip.IsEmpty())
			{
				CachedTooltip.SetCachedText(FText::Format(LOCTEXT("DelegateSet_SubtitledTooltip", "{0}\n{1}"), (FText&)CachedTooltip, SignatureTooltip), this);
			}
		}
	}
	return CachedTooltip;
}

FText UK2Node_DelegateSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("DelegatePropertyName"), FText::FromName(DelegatePropertyName));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "Assign_Name", "Assign {DelegatePropertyName}"), Args), this);
	}
	return CachedNodeTitle;
}

UEdGraphPin* UK2Node_DelegateSet::GetDelegateOwner() const
{
	UEdGraphPin* Pin = FindPin(DelegatePropertyName);
	check(Pin);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UFunction* UK2Node_DelegateSet::GetDelegateSignature()
{
	FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(DelegatePropertyClass, DelegatePropertyName);

	if( !DelegateProperty )
	{
		// Attempt to find a remapped delegate property
		FMulticastDelegateProperty* NewProperty = FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegatePropertyClass, DelegatePropertyName);
		if( NewProperty )
		{
			// Found a remapped property, update the node
			DelegateProperty = NewProperty;
			DelegatePropertyName = NewProperty->GetFName();
			DelegatePropertyClass = Cast<UClass>(NewProperty->GetOwner<UObject>());
		}
	}

	return (DelegateProperty != NULL) ? DelegateProperty->SignatureFunction : NULL;
}

UFunction* UK2Node_DelegateSet::GetDelegateSignature() const
{
	FMulticastDelegateProperty* DelegateProperty = FindFProperty<FMulticastDelegateProperty>(DelegatePropertyClass, DelegatePropertyName);

	if( !DelegateProperty )
	{
		// Attempt to find a remapped delegate property
		DelegateProperty = FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegatePropertyClass, DelegatePropertyName);
	}

	return (DelegateProperty != NULL) ? DelegateProperty->SignatureFunction : NULL;
}

void UK2Node_DelegateSet::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	// If we are overriding a function, but we can;t find the function we are overriding, that is a compile error
	if(GetDelegateSignature() == NULL)
	{
		MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingDelegateSig_ErrorFmt", "Unable to find delegate '{0}' for @@"), FText::FromString(DelegatePropertyName.ToString())).ToString(), this);
	}
}

FNodeHandlingFunctor* UK2Node_DelegateSet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_BindToMulticastDelegate(CompilerContext);
}


UK2Node::ERedirectType UK2Node_DelegateSet::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType OrginalResult = Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if ((ERedirectType::ERedirectType_None == OrginalResult) && NewPin && OldPin)
	{
		bool const bOldPinIsObj = (OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface);
		bool const bNewPinIsObj = (OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object) || (OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface);

		if ((NewPin->Direction == EGPD_Input && OldPin->Direction == EGPD_Input) &&
			bOldPinIsObj && bNewPinIsObj)
		{
			return ERedirectType_Name;
		}
	}
	return OrginalResult;
}

void UK2Node_DelegateSet::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (SourceGraph != CompilerContext.ConsolidatedEventGraph)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "InvalidNodeOutsideUbergraph_Error", "Unexpected node @@ found outside ubergraph.").ToString(), this);
	}
	else
	{
		if (UFunction* TargetFunction = GetDelegateSignature())
		{
			// First, create an event node matching the delegate signature
			UK2Node_Event* DelegateEvent = CompilerContext.SpawnIntermediateNode<UK2Node_Event>(this, SourceGraph);
			DelegateEvent->EventReference.SetFromField<UFunction>(TargetFunction, false);
			DelegateEvent->CustomFunctionName = GetDelegateTargetEntryPointName();
			DelegateEvent->bInternalEvent = true;
			DelegateEvent->AllocateDefaultPins();

			// Move the pins over to the newly created event node
			for( TArray<UEdGraphPin*>::TIterator PinIt(DelegateEvent->Pins); PinIt; ++PinIt )
			{
				UEdGraphPin* CurrentPin = *PinIt;
				check(CurrentPin);

				if( CurrentPin->Direction == EGPD_Output )
				{
					if( CurrentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec )
					{
						// Hook up the exec pin specially, since it has a different name on the dynamic delegate node
						UEdGraphPin* OldExecPin = FindPin(UEdGraphSchema_K2::PN_DelegateEntry);
						check(OldExecPin);
						CompilerContext.MovePinLinksToIntermediate(*OldExecPin, *CurrentPin);
					}
					else if( CurrentPin->PinName != UK2Node_Event::DelegateOutputName )
					{
						// Hook up all other pins, EXCEPT the delegate output pin, which isn't needed in this case
						UEdGraphPin* OldPin = FindPin(CurrentPin->PinName);
						if( !OldPin )
						{
							// If we couldn't find the old pin, the function signature is out of date.  Tell them to reconstruct
							CompilerContext.MessageLog.Error(*NSLOCTEXT("KismetCompiler", "EventNodeOutOfDate_Error", "Event node @@ is out-of-date.  Please refresh it.").ToString(), this);
							return;
						}

						CompilerContext.MovePinLinksToIntermediate(*OldPin, *CurrentPin);
					}
				}
			}
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("DelegateSigNotFound_Error", "Set Delegate node @@ unable to find function.").ToString(), this);
		}
	}
}

#undef LOCTEXT_NAMESPACE
