// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_DoOnceMultiInput.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_DoOnceMultiInput)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node::ERedirectType UK2Node_DoOnceMultiInput::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// Temp work around: remove whitespaces from pin names before doing string comparison.

	FString NewName = NewPin->PinName.ToString();
	FString OldName = OldPin->PinName.ToString();

	NewName.ReplaceInline(TEXT(" "), TEXT(""));
	OldName.ReplaceInline(TEXT(" "), TEXT(""));
	
	if (NewName == OldName)
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if (OuterGraph && OuterGraph->Schema)
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if (!K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType))
			{
				return ERedirectType_Name;
			}
		}
	}

	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

FText UK2Node_DoOnceMultiInput::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("DoOnceMultiInput", "DoOnce MultiInput");
}

FText UK2Node_DoOnceMultiInput::GetNameForPin(int32 PinIndex, bool In)
{
	check(PinIndex < GetMaxInputPinsNum());
	FString Name;
	Name.AppendChar(TCHAR('A') + static_cast<TCHAR>(PinIndex));
	
	FFormatNamedArguments Args;
	Args.Add(TEXT("Identifier"), FText::FromString(Name));
	Args.Add(TEXT("Direction"), In ? LOCTEXT("DoOnceMultiIn", "In") : LOCTEXT("DoOnceMultiOut", "Out"));

	return FText::Format(LOCTEXT("DoOnceMultiInputPinName", "{Identifier} {Direction}"), Args);
}

UK2Node_DoOnceMultiInput::UK2Node_DoOnceMultiInput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NumAdditionalInputs = 0;
}

UEdGraphPin* UK2Node_DoOnceMultiInput::FindOutPin() const
{
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		if(EEdGraphPinDirection::EGPD_Output == Pins[PinIdx]->Direction)
		{
			return Pins[PinIdx];
		}
	}
	return NULL;
}

UEdGraphPin* UK2Node_DoOnceMultiInput::FindSelfPin() const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinName == UEdGraphSchema_K2::PN_Self)
		{
			return Pin;
		}
	}
	return nullptr;
}

bool UK2Node_DoOnceMultiInput::CanAddPin() const
{
	return (NumAdditionalInputs < GetMaxInputPinsNum());
}

bool UK2Node_DoOnceMultiInput::CanRemovePin(const UEdGraphPin* Pin) const
{
	return (
		Pin &&
		NumAdditionalInputs &&
		(INDEX_NONE != Pins.IndexOfByKey(Pin)) &&
		(EEdGraphPinDirection::EGPD_Input == Pin->Direction)
	);
}

UEdGraphPin* UK2Node_DoOnceMultiInput::GetInputPin(int32 InputPinIndex)
{
	const UEdGraphPin* SelfPin = FindSelfPin();

	int32 CurrentInputIndex = 0;
	for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
	{
		UEdGraphPin* CurrentPin = Pins[PinIdx];
		if ((CurrentPin != SelfPin) && CurrentPin->Direction == EGPD_Input)
		{
			if(CurrentInputIndex == InputPinIndex)
			{
				return CurrentPin;
			}
			CurrentInputIndex++;
		}
	}
	return NULL;
}

UEdGraphPin* UK2Node_DoOnceMultiInput::GetOutputPin(int32 InputPinIndex)
{
	const UEdGraphPin* SelfPin = FindSelfPin();

	int32 CurrentInputIndex = 0;
	for (int32 PinIdx = 0; PinIdx<Pins.Num(); PinIdx++)
	{
		UEdGraphPin* CurrentPin = Pins[PinIdx];
		if ((CurrentPin != SelfPin) && CurrentPin->Direction == EGPD_Output)
		{
			if (CurrentInputIndex == InputPinIndex)
			{
				return CurrentPin;
			}
			CurrentInputIndex++;
		}
	}
	return NULL;
}

FEdGraphPinType UK2Node_DoOnceMultiInput::GetInType() const
{
	for (int32 PinIt = 0; PinIt < Pins.Num(); PinIt++)
	{
		if (Pins[PinIt] != FindSelfPin())
		{
			return Pins[PinIt]->PinType;
		}
	}
	return FEdGraphPinType();
}

FEdGraphPinType UK2Node_DoOnceMultiInput::GetOutType() const
{
	for (int32 PinIt = 0; PinIt < Pins.Num(); PinIt++)
	{
		if (Pins[PinIt] != FindOutPin())
		{
			return Pins[PinIt]->PinType;
		}
	}
	return FEdGraphPinType();
}

void UK2Node_DoOnceMultiInput::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	FText InputPinAName = GetNameForPin(0, true);
	UEdGraphPin* InputPinA = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, NAME_None, nullptr, *InputPinAName.BuildSourceString());
	InputPinA->PinFriendlyName = InputPinAName;

	FText OutputPinAName = GetNameForPin(0, false);
	UEdGraphPin* OutputPinA = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, NAME_None, nullptr, *OutputPinAName.BuildSourceString());
	OutputPinA->PinFriendlyName = OutputPinAName;

	UEdGraphPin* DoOnceResetIn = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, NAME_None, nullptr, TEXT("Reset In"));
	DoOnceResetIn->PinFriendlyName = LOCTEXT("DoOnceResetIn", "Reset In");

	UEdGraphPin* DoOnceResetOut = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, NAME_None, nullptr, TEXT("Reset Out"));
	DoOnceResetOut->PinFriendlyName = LOCTEXT("DoOnceResetOut", "Reset Out");

	for (int32 i = 0; i < NumAdditionalInputs; ++i)
	{
		AddPinsInner(i+1);
	}
}

void UK2Node_DoOnceMultiInput::AddPinsInner(int32 AdditionalPinIndex)
{
	{
		const FEdGraphPinType InputType = GetInType();
		FText InputPinName = GetNameForPin(AdditionalPinIndex, true);
		UEdGraphPin* InputPin = CreatePin(EGPD_Input, InputType, *InputPinName.BuildSourceString());
		InputPin->PinFriendlyName = InputPinName;
	}

	{
		const FEdGraphPinType OutputType = GetOutType();
		FText OutputPinName = GetNameForPin(AdditionalPinIndex, false);
		UEdGraphPin* OutputPin = CreatePin(EGPD_Output, OutputType, *OutputPinName.BuildSourceString());
		OutputPin->PinFriendlyName = OutputPinName;
	}
}

void UK2Node_DoOnceMultiInput::AddInputPin()
{
	if(CanAddPin())
	{
		FScopedTransaction Transaction( LOCTEXT("AddPinTx", "AddPin") );
		Modify();

		AddPinsInner(NumAdditionalInputs + NumBaseInputs);
		++NumAdditionalInputs;
	
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

void UK2Node_DoOnceMultiInput::RemoveInputPin(UEdGraphPin* Pin)
{
	if(CanRemovePin(Pin))
	{
		FScopedTransaction Transaction( LOCTEXT("RemovePinTx", "RemovePin") );
		Modify();

		int32 PinRemovalIndex = INDEX_NONE;
		if (Pins.Find(Pin, /*out*/ PinRemovalIndex))
		{
			Pins.RemoveAt(PinRemovalIndex);
			Pin->MarkAsGarbage();
			--NumAdditionalInputs;

			int32 NameIndex = 0;
			const UEdGraphPin* OutPin = FindOutPin();
			const UEdGraphPin* SelfPin = FindSelfPin();
			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* LocalPin = Pins[PinIndex];
				if(LocalPin && (LocalPin != OutPin) && (LocalPin != SelfPin))
				{
					const FName PinName = *GetNameForPin(NameIndex + NumBaseInputs, true).BuildSourceString();  // FIXME
					if(PinName != LocalPin->PinName)
					{
						LocalPin->Modify();
						LocalPin->PinName = PinName;
					}
					NameIndex++;
				}
			}
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}
	}
}

void UK2Node_DoOnceMultiInput::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		static FName CommutativeAssociativeBinaryOperatorNodeName = FName("CommutativeAssociativeBinaryOperatorNode");
		FText CommutativeAssociativeBinaryOperatorStr = LOCTEXT("CommutativeAssociativeBinaryOperatorNode", "Operator Node");
		if (Context->Pin != NULL)
		{
			if(CanRemovePin(Context->Pin))
			{
				FToolMenuSection& Section = Menu->AddSection(CommutativeAssociativeBinaryOperatorNodeName, CommutativeAssociativeBinaryOperatorStr);
				Section.AddMenuEntry(
					"RemovePin",
					LOCTEXT("RemovePin", "Remove pin"),
					LOCTEXT("RemovePinTooltip", "Remove this input pin"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(const_cast<UK2Node_DoOnceMultiInput*>(this), &UK2Node_DoOnceMultiInput::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
					)
				);
			}
		}
		else if(CanAddPin())
		{
			FToolMenuSection& Section = Menu->AddSection(CommutativeAssociativeBinaryOperatorNodeName, CommutativeAssociativeBinaryOperatorStr);
			Section.AddMenuEntry(
				"AddPin",
				LOCTEXT("AddPin", "Add pin"),
				LOCTEXT("AddPinTooltip", "Add another input pin"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_DoOnceMultiInput*>(this), &UK2Node_DoOnceMultiInput::AddInputPin)
				)
			);
		}
	}
}

void UK2Node_DoOnceMultiInput::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	/////////////////////////////
	// Temporary Variable node
	/////////////////////////////

	// Create the node
	UK2Node_TemporaryVariable* TempVarNode = SourceGraph->CreateIntermediateNode<UK2Node_TemporaryVariable>();
	TempVarNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
	TempVarNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(TempVarNode, this);
	// Give a reference of the variable node to the multi gate node
	DataNode = TempVarNode;

	// Create the conditional node we're replacing the enum node for

	for (int32 idx = 0; idx < NumBaseInputs + NumAdditionalInputs + 1 /*ResetPin*/; ++idx)
	{
		UEdGraphPin* ExecPin = GetInputPin(idx);
		UEdGraphPin* ThenPin = GetOutputPin(idx);

		check(ExecPin);
		check(ThenPin);

		// AssignmentNode
		UK2Node_AssignmentStatement* AssignmentNode = SourceGraph->CreateIntermediateNode<UK2Node_AssignmentStatement>();
		AssignmentNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AssignmentNode, this);
		AssignmentNode->GetVariablePin()->PinType = TempVarNode->GetVariablePin()->PinType;
		AssignmentNode->GetVariablePin()->MakeLinkTo(TempVarNode->GetVariablePin());
		AssignmentNode->GetValuePin()->PinType = TempVarNode->GetVariablePin()->PinType;
			
		if (!ExecPin->PinName.ToString().Contains(TEXT("Reset"))) // Fixme this wont work for localization
		{
			// BranchNode
			UK2Node_IfThenElse* BranchNode = SourceGraph->CreateIntermediateNode<UK2Node_IfThenElse>();
			BranchNode->AllocateDefaultPins();
			CompilerContext.MessageLog.NotifyIntermediateObjectCreation(BranchNode, this);

			// -------------------------------------------------

			// Coerce the wildcards pin types (set the default of the value to 0)
			AssignmentNode->GetValuePin()->DefaultValue = TEXT("1");

			// -------------------------------------------------

			// Link Tempvariable with the branch condtional
			Schema->TryCreateConnection(TempVarNode->GetVariablePin(), BranchNode->GetConditionPin());
			
			// Link our input exec pin into the branch node
			CompilerContext.MovePinLinksToIntermediate(*ExecPin, *BranchNode->GetExecPin());

			// link branch else (false) to assigment node (set temp variable to true)
			Schema->TryCreateConnection(BranchNode->GetElsePin(), AssignmentNode->GetExecPin());

			// link set temp variable node to our ouput then pin
			CompilerContext.MovePinLinksToIntermediate(*ThenPin, *AssignmentNode->GetThenPin());

		}
		else
		{
			// Coerce the wildcards pin types (set the default of the value to 1)				
			AssignmentNode->GetValuePin()->DefaultValue = TEXT("0");

			// -------------------------------------------------

			// Link our input exec pin into the branch node
			CompilerContext.MovePinLinksToIntermediate(*ExecPin, *AssignmentNode->GetExecPin());

			// link set temp variable node to our ouput then pin
			CompilerContext.MovePinLinksToIntermediate(*ThenPin, *AssignmentNode->GetThenPin());
		}
	}

	// Break all links to the Select node so it goes away for at scheduling time
	BreakAllNodeLinks();
}

void UK2Node_DoOnceMultiInput::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_DoOnceMultiInput::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::FlowControl);
}

#undef LOCTEXT_NAMESPACE
