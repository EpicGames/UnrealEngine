// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_DynamicCast.h"

#include "BlueprintActionFilter.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintEditorModule.h"
#include "BlueprintEditorSettings.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "DynamicCastHandler.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_DynamicCast)

class FKismetCompilerContext;

#define LOCTEXT_NAMESPACE "K2Node_DynamicCast"

namespace UK2Node_DynamicCastImpl
{
	static const FName CastSuccessPinName("bSuccess");
}

UK2Node_DynamicCast::UK2Node_DynamicCast(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PureState(EPureState::UseDefault)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// @todo_deprecated - Remove these later.
	bIsPureCast = false;
	bIsPureCast_DEPRECATED = bIsPureCast;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UK2Node_DynamicCast::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::DynamicCastNodesUsePureStateEnum)
	{
		if (bIsPureCast_DEPRECATED)
		{
			PureState = EPureState::Pure;
		}
		else
		{
			PureState = EPureState::Impure;
		}
	}
	else
	{
		Ar << PureState;
	}
}

void UK2Node_DynamicCast::CreateExecPins()
{
	InitPureState();

	const bool bIsNodePure = IsNodePure();
	if (!bIsNodePure)
	{
		// Input - Execution Pin
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

		// Output - Execution Pins
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_CastSucceeded);
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_CastFailed);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// @todo_deprecated - Remove this later.
	bIsPureCast = bIsNodePure;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UK2Node_DynamicCast::CreateSuccessPin()
{
	UEdGraphPin* BoolSuccessPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Boolean, UK2Node_DynamicCastImpl::CastSuccessPinName);
	BoolSuccessPin->bHidden = !IsNodePure();
}

void UK2Node_DynamicCast::AllocateDefaultPins()
{
	const bool bReferenceObsoleteClass = TargetType && TargetType->HasAnyClassFlags(CLASS_NewerVersionExists);
	if (bReferenceObsoleteClass)
	{
		Message_Error(FString::Printf(TEXT("Node '%s' references obsolete class '%s'"), *GetPathName(), *TargetType->GetPathName()));
	}
	ensure(!bReferenceObsoleteClass);

	// Exec pins (if needed)
	CreateExecPins();

	// Input - Source type Pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, UObject::StaticClass(), UEdGraphSchema_K2::PN_ObjectToCast);

	// Output - Data Pin
	if (TargetType)
	{
		const FString CastResultPinName = UEdGraphSchema_K2::PN_CastedValuePrefix + TargetType->GetDisplayNameText().ToString();
		if (TargetType->IsChildOf(UInterface::StaticClass()))
		{
			CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Interface, *TargetType, *CastResultPinName);
		}
		else 
		{
			CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, *TargetType, *CastResultPinName);
		}
	}

	// Output - Success
	CreateSuccessPin();

	Super::AllocateDefaultPins();
}

FLinearColor UK2Node_DynamicCast::GetNodeTitleColor() const
{
	return FLinearColor(0.0f, 0.55f, 0.62f);
}

FSlateIcon UK2Node_DynamicCast::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Cast_16x");
	return Icon;
}

FText UK2Node_DynamicCast::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TargetType == nullptr)
	{
		return NSLOCTEXT("K2Node_DynamicCast", "BadCastNode", "Bad cast node");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		// If casting to BP class, use BP name not class name (ie. remove the _C)
		FString TargetName;
		UBlueprint* CastToBP = UBlueprint::GetBlueprintFromClass(TargetType);
		if (CastToBP != NULL)
		{
			TargetName = CastToBP->GetName();
		}
		else
		{
			TargetName = TargetType->GetName();
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("TargetName"), FText::FromString(TargetName));

		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node_DynamicCast", "CastTo", "Cast To {TargetName}"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_DynamicCast::GetTooltipText() const
{
	if (TargetType && TargetType->IsChildOf(UInterface::StaticClass()))
	{
		return FText::Format(LOCTEXT("CastToInterfaceTooltip", "Tries to access object as an interface '{0}' it may implement."), FText::FromString(TargetType->GetName()));
	}
	
	UBlueprint* CastToBP = UBlueprint::GetBlueprintFromClass(TargetType);
	if (CastToBP)
	{
		return FText::Format(LOCTEXT("CastToBPTooltip", "Tries to access object as a blueprint class '{0}' it may be an instance of.\n\nNOTE: This will cause the blueprint to always be loaded, which can be expensive."), FText::FromString(CastToBP->GetName()));
	}

	const FString ClassName = TargetType ? TargetType->GetName() : TEXT("");

	return FText::Format(LOCTEXT("CastToNativeTooltip", "Tries to access object as a class '{0}' it may be an instance of."), FText::FromString(ClassName));
}

void UK2Node_DynamicCast::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		{
			FText MenuEntryTitle = LOCTEXT("MakePureTitle", "Convert to pure cast");
			FText MenuEntryTooltip = LOCTEXT("MakePureTooltip", "Removes the execution pins to make the node more versatile (NOTE: the cast could still fail, resulting in an invalid output).");

			bool bCanTogglePurity = true;
			auto CanExecutePurityToggle = [](bool const bInCanTogglePurity)->bool
			{
				return bInCanTogglePurity;
			};

			if (IsNodePure())
			{
				MenuEntryTitle = LOCTEXT("MakeImpureTitle", "Convert to impure cast");
				MenuEntryTooltip = LOCTEXT("MakeImpureTooltip", "Adds in branching execution pins so that you can separatly handle when the cast fails/succeeds.");

				const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(GetSchema());
				check(K2Schema != nullptr);

				bCanTogglePurity = K2Schema->DoesGraphSupportImpureFunctions(GetGraph());
				if (!bCanTogglePurity)
				{
					MenuEntryTooltip = LOCTEXT("CannotMakeImpureTooltip", "This graph does not support impure calls (and you should therefore test the cast's result for validity).");
				}
			}

			FToolMenuSection& Section = Menu->AddSection("K2NodeDynamicCast", LOCTEXT("DynamicCastHeader", "Cast"));
			Section.AddMenuEntry(
				"TogglePurity",
				MenuEntryTitle,
				MenuEntryTooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_DynamicCast*>(this), &UK2Node_DynamicCast::TogglePurity),
					FCanExecuteAction::CreateStatic(CanExecutePurityToggle, bCanTogglePurity),
					FIsActionChecked()
				)
			);
		}
	}
}

void UK2Node_DynamicCast::PostReconstructNode()
{
	Super::PostReconstructNode();
	// update the pin name (to "Interface" if an interface is connected)
	NotifyPinConnectionListChanged(GetCastSourcePin());
}

UEdGraphPin* UK2Node_DynamicCast::GetValidCastPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_CastSucceeded);
	check((Pin != nullptr) || IsNodePure());
	check((Pin == nullptr) || (Pin->Direction == EGPD_Output));
	return Pin;
}

UEdGraphPin* UK2Node_DynamicCast::GetInvalidCastPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_CastFailed);
	check((Pin != nullptr) || IsNodePure());
	check((Pin == nullptr) || (Pin->Direction == EGPD_Output));
	return Pin;
}

UEdGraphPin* UK2Node_DynamicCast::GetCastResultPin() const
{
	if(TargetType != nullptr)
	{
		for (int32 PinIdx = 0; PinIdx < Pins.Num(); PinIdx++)
		{
			if (Pins[PinIdx]->PinType.PinSubCategoryObject == *TargetType
				&& Pins[PinIdx]->Direction == EGPD_Output
				&& Pins[PinIdx]->PinName.ToString().StartsWith(UEdGraphSchema_K2::PN_CastedValuePrefix))
			{
				return Pins[PinIdx];
			}
		}
	}
		
	return nullptr;
}

UEdGraphPin* UK2Node_DynamicCast::GetCastSourcePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_ObjectToCast);
	check(Pin);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_DynamicCast::GetBoolSuccessPin() const
{
	UEdGraphPin* Pin = FindPin(UK2Node_DynamicCastImpl::CastSuccessPinName);
	check((Pin == nullptr) || (Pin->Direction == EGPD_Output));
	return Pin;
}

void UK2Node_DynamicCast::SetPurity(bool bNewPurity)
{
	InitPureState();

	if (bNewPurity != IsNodePure())
	{
		if (bNewPurity)
		{
			PureState = EPureState::Pure;
		}
		else
		{
			PureState = EPureState::Impure;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// @todo_deprecated - Remove this later.
		bIsPureCast = bNewPurity;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		const bool bHasBeenConstructed = (Pins.Num() > 0);
		if (bHasBeenConstructed)
		{
			ReconstructNode();
		}
	}
}

void UK2Node_DynamicCast::TogglePurity()
{
	const bool bIsNodePure = IsNodePure();
	const FText TransactionTitle = bIsNodePure ? LOCTEXT("TogglePurityToImpure", "Convert to Impure Cast") : LOCTEXT("TogglePurityToPure", "Convert to Pure Cast");
	const FScopedTransaction Transaction( TransactionTitle );
	Modify();

	SetPurity(!bIsNodePure);
}

UK2Node::ERedirectType UK2Node_DynamicCast::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if((ERedirectType_None == RedirectType) && (NULL != NewPin) && (NULL != OldPin))
	{
		const bool bProperPrefix = 
			NewPin->PinName.ToString().StartsWith(UEdGraphSchema_K2::PN_CastedValuePrefix, ESearchCase::CaseSensitive) && 
			OldPin->PinName.ToString().StartsWith(UEdGraphSchema_K2::PN_CastedValuePrefix, ESearchCase::CaseSensitive);

		const bool bClassMatch = NewPin->PinType.PinSubCategoryObject.IsValid() &&
			(NewPin->PinType.PinSubCategoryObject == OldPin->PinType.PinSubCategoryObject);

		if(bProperPrefix && bClassMatch)
		{
			RedirectType = ERedirectType_Name;
		}
	}
	return RedirectType;
}

FNodeHandlingFunctor* UK2Node_DynamicCast::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_DynamicCast(CompilerContext, KCST_DynamicCast);
}

bool UK2Node_DynamicCast::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();
	UClass* SourceClass = *TargetType;
	const bool bResult = (SourceClass != NULL) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}
	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

FText UK2Node_DynamicCast::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("ActionMenuCategory", "Casting")), this);
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_DynamicCast::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddSubObject(TargetType.Get());

	return NodeSignature;
}

bool UK2Node_DynamicCast::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	bool bIsDisallowed = Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);

	if (MyPin == GetCastSourcePin())
	{
		const FEdGraphPinType& OtherPinType = OtherPin->PinType;
		const FText OtherPinName = OtherPin->PinFriendlyName.IsEmpty() ? FText::FromName(OtherPin->PinName) : OtherPin->PinFriendlyName;

		if (OtherPinType.IsContainer())
		{
			bIsDisallowed = true;
			OutReason = LOCTEXT("CannotContainerCast", "You cannot cast containers of objects.").ToString();
		}
		else if (TargetType == nullptr)
		{
			bIsDisallowed = true;
			OutReason = LOCTEXT("InvalidTargetType", "This cast has an invalid target type (was the class deleted without a redirect?).").ToString();
		}
		else if (OtherPinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
		{
			// allow all interface casts, but don't allow casting non-objects to interfaces
		}
		else if (OtherPinType.PinCategory == UEdGraphSchema_K2::PC_Object)
		{
			// let's handle wasted cast inputs with warnings in ValidateNodeDuringCompilation() instead
		}
		else
		{
			bIsDisallowed = true;
			OutReason = LOCTEXT("NonObjectCast", "You can only cast objects/interfaces.").ToString();
		}
	}
	return bIsDisallowed;
}

void UK2Node_DynamicCast::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	if (Pin == GetCastSourcePin())
	{
		Pin->PinFriendlyName = FText::GetEmpty();

		FEdGraphPinType& InputPinType = Pin->PinType;
		if (Pin->LinkedTo.Num() == 0)
		{
			InputPinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			InputPinType.PinSubCategory = NAME_None;
			InputPinType.PinSubCategoryObject = nullptr;
		}
		else
		{
			const FEdGraphPinType& ConnectedPinType = Pin->LinkedTo[0]->PinType;
			if (ConnectedPinType.PinCategory == UEdGraphSchema_K2::PC_Interface)
			{
				Pin->PinFriendlyName = LOCTEXT("InterfaceInputName", "Interface");
				InputPinType.PinCategory = UEdGraphSchema_K2::PC_Interface;
				InputPinType.PinSubCategoryObject = ConnectedPinType.PinSubCategoryObject;
			}
			else if (ConnectedPinType.PinCategory == UEdGraphSchema_K2::PC_Object)
			{
				InputPinType.PinCategory = UEdGraphSchema_K2::PC_Object;
				InputPinType.PinSubCategoryObject = UObject::StaticClass();
			}
		}
	}
}

void UK2Node_DynamicCast::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// Update exec pins if we converted from impure to pure
	ReconnectPureExecPins(OldPins);
}

void UK2Node_DynamicCast::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UEdGraphPin* SourcePin = GetCastSourcePin();
	if (SourcePin->LinkedTo.Num() > 0)
	{
		UClass* SourceType = *TargetType;
		if (SourceType == nullptr)
		{
			return;
		}
		SourceType = SourceType->GetAuthoritativeClass();

		for (UEdGraphPin* CastInput : SourcePin->LinkedTo)
		{
			if (CastInput == nullptr)
			{
				continue;
			}

			const FEdGraphPinType& SourcePinType = CastInput->PinType;
			if (SourcePinType.PinCategory != UEdGraphSchema_K2::PC_Object)
			{
				// all other types should have been rejected by IsConnectionDisallowed()
				continue;
			}

			UClass* SourceClass = Cast<UClass>(SourcePinType.PinSubCategoryObject.Get());
			if ((SourceClass == nullptr) && (SourcePinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self))
			{
				if (UK2Node* K2Node = Cast<UK2Node>(CastInput->GetOwningNode()))
				{
					SourceClass = K2Node->GetBlueprint()->GeneratedClass;
				}
			}

			if (SourceClass == nullptr)
			{
				const FString SourcePinName = CastInput->PinFriendlyName.IsEmpty() ? CastInput->PinName.ToString() : CastInput->PinFriendlyName.ToString();

				FText const ErrorFormat = LOCTEXT("BadCastInputFmt", "'{0}' does not have a clear object type (invalid input into @@).");
				MessageLog.Error( *FText::Format(ErrorFormat, FText::FromString(SourcePinName)).ToString(), this );

				continue;
			}
			SourceClass = SourceClass->GetAuthoritativeClass();

			if (SourceClass == SourceType)
			{
				const FString SourcePinName = CastInput->PinFriendlyName.IsEmpty() ? CastInput->PinName.ToString() : CastInput->PinFriendlyName.ToString();

				FText const WarningFormat = LOCTEXT("EqualObjectCastFmt", "'{0}' is already a '{1}', you don't need @@.");
				MessageLog.Note( *FText::Format(WarningFormat, FText::FromString(SourcePinName), TargetType->GetDisplayNameText()).ToString(), this );
			}
			else if (SourceClass->IsChildOf(SourceType))
			{
				const FString SourcePinName = CastInput->PinFriendlyName.IsEmpty() ? CastInput->PinName.ToString() : CastInput->PinFriendlyName.ToString();

				FText const WarningFormat = LOCTEXT("UnneededObjectCastFmt", "'{0}' is already a '{1}' (which inherits from '{2}'), so you don't need @@.");
				MessageLog.Note( *FText::Format(WarningFormat, FText::FromString(SourcePinName), SourceClass->GetDisplayNameText(), TargetType->GetDisplayNameText()).ToString(), this );
			}
			else if ((!SourceType || !SourceType->IsChildOf(SourceClass)) && !FKismetEditorUtilities::IsClassABlueprintInterface(SourceType))
			{
				FText const WarningFormat = LOCTEXT("DisallowedObjectCast", "'{0}' does not inherit from '{1}' (@@ would always fail).");
				MessageLog.Warning( *FText::Format(WarningFormat, TargetType->GetDisplayNameText(), SourceClass->GetDisplayNameText()).ToString(), this );
			}
		}
	}
}


bool UK2Node_DynamicCast::ReconnectPureExecPins(TArray<UEdGraphPin*>& OldPins)
{
	if (IsNodePure())
	{
		// look for an old exec pin
		UEdGraphPin* PinExec = nullptr;
		for (UEdGraphPin* Pin : OldPins)
		{
			if (Pin->PinName == UEdGraphSchema_K2::PN_Execute)
			{
				PinExec = Pin;
				break;
			}
		}
		if (PinExec)
		{
			// look for old then pin
			UEdGraphPin* PinThen = nullptr;
			for (UEdGraphPin* Pin : OldPins)
			{
				if (Pin->PinName == UEdGraphSchema_K2::PN_Then)
				{
					PinThen = Pin;
					break;
				}
			}
			if (PinThen)
			{
				// reconnect all incoming links to old exec pin to the far end of the old then pin.
				if (PinThen->LinkedTo.Num() > 0)
				{
					UEdGraphPin* PinThenLinked = PinThen->LinkedTo[0];
					while (PinExec->LinkedTo.Num() > 0)
					{
						UEdGraphPin* PinExecLinked = PinExec->LinkedTo[0];
						PinExecLinked->BreakLinkTo(PinExec);
						PinExecLinked->MakeLinkTo(PinThenLinked);
					}
					return true;
				}
			}
		}
	}
	return false;
}

bool UK2Node_DynamicCast::IsActionFilteredOut(const FBlueprintActionFilter& Filter)
{
	bool bIsFilteredOut = false;

	if (Filter.HasAnyFlags(FBlueprintActionFilter::BPFILTER_RejectNonImportedFields))
	{
		TSharedPtr<IBlueprintEditor> BlueprintEditor = Filter.Context.EditorPtr.Pin();
		if (BlueprintEditor.IsValid() && TargetType)
		{
			bIsFilteredOut = BlueprintEditor->IsNonImportedObject(TargetType);
		}
	}

	return bIsFilteredOut;
}

void UK2Node_DynamicCast::InitPureState()
{	
	const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(GetSchema());
	
	// The schema may be null if this node is created during UEdGraphSchema_K2::FindSpecializedConversionNode
	// because the parent graph would not be set, causing GetSchema to always return null. 
	if (K2Schema && !K2Schema->DoesGraphSupportImpureFunctions(GetGraph()))
	{
		PureState = EPureState::Pure;
	}
	else if (PureState == EPureState::UseDefault)
	{
		// Ensure the node is either pure or impure, based on current settings.
		const UBlueprintEditorSettings* BlueprintSettings = GetDefault<UBlueprintEditorSettings>();
		if (BlueprintSettings->bFavorPureCastNodes)
		{
			PureState = EPureState::Pure;
		}
		else
		{
			PureState = EPureState::Impure;
		}
	}
}

#undef LOCTEXT_NAMESPACE
