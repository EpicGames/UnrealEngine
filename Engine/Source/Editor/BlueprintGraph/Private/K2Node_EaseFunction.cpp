// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EaseFunction.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EaseFunction)

#define LOCTEXT_NAMESPACE "K2Node_EaseFunction"

struct FEaseFunctionNodeHelper
{
	static const FName EaseFuncPinName;
	static const FName AlphaPinName;
	static const FName APinFloatName;
	static const FName BPinFloatName;
	static const FName ResultPinName;
	static const FName BlendExpPinName;
	static const FName StepsPinName;
	static const FName ShortestPathPinName;

	static const FName GetEaseFuncPinName()
	{
		return EaseFuncPinName;
	}

	static const FName GetAlphaPinName()
	{
		return AlphaPinName;
	}

	static const FName GetAPinName()
	{
		return APinFloatName;
	}

	static const FName GetBPinName()
	{
		return BPinFloatName;
	}

	static const FName GetResultPinName()
	{
		return ResultPinName;
	}

	static const FName GetBlendExpPinName()
	{
		return BlendExpPinName;
	}

	static const FName GetStepsPinName()
	{
		return StepsPinName;
	}

	static const FName GetShortestPathPinName()
	{
		return ShortestPathPinName;
	}
};

const FName FEaseFunctionNodeHelper::EaseFuncPinName(TEXT("Function"));
const FName FEaseFunctionNodeHelper::AlphaPinName(TEXT("Alpha"));
const FName FEaseFunctionNodeHelper::APinFloatName(TEXT("A"));
const FName FEaseFunctionNodeHelper::BPinFloatName(TEXT("B"));
const FName FEaseFunctionNodeHelper::ResultPinName(TEXT("Result"));
const FName FEaseFunctionNodeHelper::BlendExpPinName(TEXT("BlendExp"));
const FName FEaseFunctionNodeHelper::StepsPinName(TEXT("Steps"));
const FName FEaseFunctionNodeHelper::ShortestPathPinName(TEXT("ShortestPath"));

UK2Node_EaseFunction::UK2Node_EaseFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CachedEaseFuncPin(nullptr)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Interpolates from value A to value B using a user specified easing function");
	EaseFunctionName = NAME_None;
}

void UK2Node_EaseFunction::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add the first pin representing all available easing functions. If EEasingFunc changes its name this will fail a runtime check!
	UEnum* EaseFuncEnum = StaticEnum<EEasingFunc::Type>();
	check(EaseFuncEnum);
	CachedEaseFuncPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, EaseFuncEnum, FEaseFunctionNodeHelper::GetEaseFuncPinName());
	SetPinToolTip(*CachedEaseFuncPin, LOCTEXT("EaseFunsPinDescription", "Specifies the desired ease function to be applied. If connected no customization is possible."));

	// Make sure that the default value is set correctly if none has been set
	K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(CachedEaseFuncPin);

	UEdGraphPin* AlphaPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, FEaseFunctionNodeHelper::GetAlphaPinName());

	SetPinToolTip(*AlphaPin, LOCTEXT("AlphaPinTooltip", "Alpha value used to specify the easing in time."));

	// Add wildcard pins for A, B and the return Pin
	UEdGraphPin* APin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, FEaseFunctionNodeHelper::GetAPinName());
	SetPinToolTip(*APin, LOCTEXT("APinDescription", "Easing start value"));

	UEdGraphPin* BPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, FEaseFunctionNodeHelper::GetBPinName());
	SetPinToolTip(*BPin, LOCTEXT("BPinDescription", "Easing end value"));

	UEdGraphPin* ResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, FEaseFunctionNodeHelper::GetResultPinName());
	SetPinToolTip(*ResultPin, LOCTEXT("ResultPinDescription", "Easing result value"));

	UEdGraphPin* ShortestPathPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, FEaseFunctionNodeHelper::GetShortestPathPinName());
	SetPinToolTip(*ShortestPathPin, LOCTEXT("ShortestPathPinTooltip", "When interpolating the shortest path should be taken."));
	K2Schema->SetPinAutogeneratedDefaultValue(ShortestPathPin, TEXT("true"));

	UEdGraphPin* BlendExpPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, FEaseFunctionNodeHelper::GetBlendExpPinName());

	SetPinToolTip(*BlendExpPin, LOCTEXT("BlendExpPinDescription", "Blend Exponent for basic ease functions"));
	K2Schema->SetPinAutogeneratedDefaultValue(BlendExpPin, TEXT("2.0"));

	UEdGraphPin* StepsPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, FEaseFunctionNodeHelper::GetStepsPinName());
	SetPinToolTip(*StepsPin, LOCTEXT("StepsPinDescription", "Number of steps required to go from A to B"));
	K2Schema->SetPinAutogeneratedDefaultValue(StepsPin, TEXT("2"));

	GenerateExtraPins();

	RefreshPinVisibility();
}

FText UK2Node_EaseFunction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EaseFunction_Title", "Ease");
}

FText UK2Node_EaseFunction::GetTooltipText() const
{
	return NodeTooltip;
}

FSlateIcon UK2Node_EaseFunction::GetIconAndTint(FLinearColor& OutColor) const
{
	// The function icon seams the best choice!
	OutColor = GetNodeTitleColor();
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon;
}

void UK2Node_EaseFunction::SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const
{
	MutatablePin.PinToolTip = UEdGraphSchema_K2::TypeToText(MutatablePin.PinType).ToString();

	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	if (K2Schema != nullptr)
	{
		MutatablePin.PinToolTip += TEXT(" ");
		MutatablePin.PinToolTip += K2Schema->GetPinDisplayName(&MutatablePin).ToString();
	}

	MutatablePin.PinToolTip += FString(TEXT("\n")) + PinDescription.ToString();
}

void UK2Node_EaseFunction::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	const auto EaseFuncPin = GetEaseFuncPin();
	if (Pin == EaseFuncPin)
	{
		RefreshPinVisibility();
		GetGraph()->NotifyGraphChanged();
	}
	else
	{
		PinTypeChanged(Pin);
	}
}


void UK2Node_EaseFunction::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	const auto EaseFuncPin = GetEaseFuncPin();
	if (Pin == EaseFuncPin )
	{
		RefreshPinVisibility();
		GetGraph()->NotifyGraphChanged();
	}
}

void UK2Node_EaseFunction::PostReconstructNode()
{
	Super::PostReconstructNode();

	// Check in which state we are at the moment
	RefreshPinVisibility();

	// Find a pin that has connections to use to jumpstart the wildcard process
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		if (Pins[PinIndex]->PinName == FEaseFunctionNodeHelper::GetAPinName() ||
			Pins[PinIndex]->PinName == FEaseFunctionNodeHelper::GetBPinName() ||
			Pins[PinIndex]->PinName == FEaseFunctionNodeHelper::GetResultPinName())
		{
			// Take default pin values into account in case we can securly convert the string default value into a PinType
			// this is currently not the case but could be considered in the future.
			if (Pins[PinIndex]->LinkedTo.Num() > 0)
			{
				PinTypeChanged(Pins[PinIndex]);
				break;
			}
		}
	}

	GenerateExtraPins();
}

bool UK2Node_EaseFunction::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	// Check the pin name and see if it matches on of our three base pins
	if (MyPin->PinName == FEaseFunctionNodeHelper::GetAPinName() || 
		MyPin->PinName == FEaseFunctionNodeHelper::GetBPinName() || 
		MyPin->PinName == FEaseFunctionNodeHelper::GetResultPinName())
	{
		const bool bConnectionOk = (
			OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real ||
			(
				OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
				OtherPin->PinType.PinSubCategoryObject.IsValid() &&
				(
					OtherPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Vector") ||
					OtherPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Rotator") ||
					OtherPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Transform")
					)
				)
			) && !OtherPin->PinType.IsContainer();

		if (!bConnectionOk)
		{
			OutReason = LOCTEXT("PinConnectionDisallowed", "Pin type is not supported by function.").ToString();
			return true;
		}
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_EaseFunction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EaseFunction::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Math, LOCTEXT("InterpCategory", "Interpolation")), this);
	}
	return CachedCategory;
}

void UK2Node_EaseFunction::ChangePinType(UEdGraphPin* Pin)
{
	PinTypeChanged(Pin);
}

void UK2Node_EaseFunction::PinTypeChanged(UEdGraphPin* Pin)
{
	bool bChanged = false;

	if (Pin->PinName == FEaseFunctionNodeHelper::GetAPinName() ||
		Pin->PinName == FEaseFunctionNodeHelper::GetBPinName() ||
		Pin->PinName == FEaseFunctionNodeHelper::GetResultPinName())
	{
		// Get pin refs
		UEdGraphPin* APin = FindPin(FEaseFunctionNodeHelper::GetAPinName());
		UEdGraphPin* BPin = FindPin(FEaseFunctionNodeHelper::GetBPinName());
		UEdGraphPin* ResultPin = FindPin(FEaseFunctionNodeHelper::GetResultPinName());

		// Propagate the type change or reset to wildcard PinType
		if (Pin->LinkedTo.Num() > 0)
		{
			UEdGraphPin* InstigatorPin = Pin->LinkedTo[0];

			bChanged |= UpdatePin(APin, InstigatorPin);
			bChanged |= UpdatePin(BPin, InstigatorPin);
			bChanged |= UpdatePin(ResultPin, InstigatorPin);

			if (bChanged)
			{
				// Just in case we switch to an invalid function clean it first
				EaseFunctionName = NAME_None;

				// Generate the right function name
				if (InstigatorPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real)
				{
					EaseFunctionName = TEXT("Ease");
				}
				else if (InstigatorPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
				{
					if (InstigatorPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Vector"))
					{
						EaseFunctionName = TEXT("VEase");
					}
					else if (InstigatorPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Rotator"))
					{
						EaseFunctionName = TEXT("REase");
					}
					else if (InstigatorPin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Transform"))
					{
						EaseFunctionName = TEXT("TEase");
					}
				}
			}
		}
		else
		{	
			if (APin->GetDefaultAsString().IsEmpty() && APin->LinkedTo.Num() == 0 &&
				BPin->GetDefaultAsString().IsEmpty() && BPin->LinkedTo.Num() == 0 &&
				ResultPin->LinkedTo.Num() == 0)
			{
				// Restore wild card pin
				APin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				APin->PinType.PinSubCategory = NAME_None;
				APin->PinType.PinSubCategoryObject = nullptr;

				// Propagate change
				UpdatePin(BPin, APin);
				UpdatePin(ResultPin, APin);

				// Make sure the function name is nulled out
				EaseFunctionName = TEXT("");
				bChanged = true;			
			}
		}

		// Pin connections and data changed in some way
		if (bChanged)
		{
			SetPinToolTip(*APin, LOCTEXT("APinDescription", "Easing start value"));
			SetPinToolTip(*BPin, LOCTEXT("BPinDescription", "Easing end value"));
			SetPinToolTip(*ResultPin, LOCTEXT("ResultPinDescription", "Easing result value"));

			// Let our subclasses generate some pins if required, this way we can add any aditional pins required by some types for examples
			GenerateExtraPins();

			// Let the graph know to refresh
			GetGraph()->NotifyGraphChanged();

			UBlueprint* Blueprint = GetBlueprint();
			if (!Blueprint->bBeingCompiled)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
				Blueprint->BroadcastChanged();
			}
		}
	}

	Super::PinTypeChanged(Pin);
}

bool UK2Node_EaseFunction::UpdatePin(UEdGraphPin* MyPin, UEdGraphPin* OtherPin)
{
	// If the data type changed break links first
	if (MyPin->PinType != OtherPin->PinType)
	{
		MyPin->PinType = OtherPin->PinType;
		return true;
	}	
	return false;
}

void UK2Node_EaseFunction::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	/**
		At the end of this, the UK2Node_EaseFunction will not be a part of the Blueprint, it merely handles connecting
		the other nodes into the Blueprint.
	*/

	UFunction* Function = UKismetMathLibrary::StaticClass()->FindFunctionByName(EaseFunctionName);
	if (Function == NULL)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InvalidFunctionName", "BaseAsyncTask: Type not supported or not initialized. @@").ToString(), this);
		return;
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// The call function does all the real work, each child class implementing easing for  a given type provides
	// the name of the desired function
	UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		
	CallFunction->SetFromFunction(Function);
	CallFunction->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, this);

	// Move the ease function and the alpha connections from us to the call function
	CompilerContext.MovePinLinksToIntermediate(*FindPin(FEaseFunctionNodeHelper::GetEaseFuncPinName()), *CallFunction->FindPin(TEXT("EasingFunc")));
	CompilerContext.MovePinLinksToIntermediate(*FindPin(FEaseFunctionNodeHelper::GetAlphaPinName()), *CallFunction->FindPin(TEXT("Alpha")));

	// Move base connections to the call function's connections
	CompilerContext.MovePinLinksToIntermediate(*FindPin(FEaseFunctionNodeHelper::GetAPinName()), *CallFunction->FindPin(TEXT("A")));
	CompilerContext.MovePinLinksToIntermediate(*FindPin(FEaseFunctionNodeHelper::GetBPinName()), *CallFunction->FindPin(TEXT("B")));
	CompilerContext.MovePinLinksToIntermediate(*FindPin(FEaseFunctionNodeHelper::GetResultPinName()), *CallFunction->GetReturnValuePin());

	// Now move the custom pins to their new locations
	UEdGraphPin* ShortestPathPin = FindPinChecked(FEaseFunctionNodeHelper::GetShortestPathPinName());
	if (!ShortestPathPin->bHidden)
	{
		CompilerContext.MovePinLinksToIntermediate(*ShortestPathPin, *CallFunction->FindPinChecked(TEXT("bShortestPath")));
	}

	UEdGraphPin* BlendExpPin = FindPinChecked(FEaseFunctionNodeHelper::GetBlendExpPinName());
	if (!BlendExpPin->bHidden)
	{
		CompilerContext.MovePinLinksToIntermediate(*BlendExpPin, *CallFunction->FindPinChecked(FEaseFunctionNodeHelper::GetBlendExpPinName()));
	}

	UEdGraphPin* StepsPin = FindPinChecked(FEaseFunctionNodeHelper::GetStepsPinName());
	if (!StepsPin->bHidden)
	{
		CompilerContext.MovePinLinksToIntermediate(*StepsPin, *CallFunction->FindPinChecked(FEaseFunctionNodeHelper::GetStepsPinName()));
	}

	// Cleanup links to ourself and we are done!
	BreakAllNodeLinks();
}

UEdGraphPin* UK2Node_EaseFunction::GetEaseFuncPin() const
{
	if (!CachedEaseFuncPin)
	{
		const_cast<UK2Node_EaseFunction*>(this)->CachedEaseFuncPin = FindPinChecked(FEaseFunctionNodeHelper::GetEaseFuncPinName());
	}
	return CachedEaseFuncPin;
}

void UK2Node_EaseFunction::RefreshPinVisibility()
{
	const auto EaseFuncPin = GetEaseFuncPin();
	UEnum * Enum = StaticEnum<EEasingFunc::Type>();
	check(Enum != NULL);
	const int64 NewEasingFunc = CanCustomizeCurve() ? Enum->GetValueByName(*EaseFuncPin->DefaultValue) : -1;
	
	// Early exit in case no changes are required
	const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());

	UEdGraphPin* BlendExpPin = FindPinChecked(FEaseFunctionNodeHelper::GetBlendExpPinName());

	if (NewEasingFunc == -1 ||
		NewEasingFunc == EEasingFunc::EaseIn ||
		NewEasingFunc == EEasingFunc::EaseOut ||
		NewEasingFunc == EEasingFunc::EaseInOut)
	{
		// Show the BlendExpPin
		BlendExpPin->bHidden = false;
	}
	else
	{
		// Hide the BlendExpPin:
		BlendExpPin->BreakAllPinLinks();
		BlendExpPin->bHidden = true;
	}

	UEdGraphPin* StepsPin = FindPinChecked(FEaseFunctionNodeHelper::GetStepsPinName());
	if (NewEasingFunc == -1 || 
		NewEasingFunc == EEasingFunc::Step)
	{
		// Show the Steps pin:
		StepsPin->bHidden = false;
	}
	else
	{
		// Hide the Steps pin:
		StepsPin->BreakAllPinLinks();
		StepsPin->bHidden = true;
	}
}

void UK2Node_EaseFunction::GenerateExtraPins()
{
	// Add pins based on the pin type
	const UEdGraphPin* APin = FindPinChecked(FEaseFunctionNodeHelper::GetAPinName());
	const bool bIsRotator = APin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && APin->PinType.PinSubCategoryObject.Get()->GetName() == TEXT("Rotator");
	UEdGraphPin* ShortestPathPin = FindPinChecked(FEaseFunctionNodeHelper::GetShortestPathPinName());
	if (bIsRotator)
	{
		ShortestPathPin->bHidden = false;
	}
	else
	{
		ShortestPathPin->BreakAllPinLinks();
		ShortestPathPin->bHidden = true;
	}
}

void UK2Node_EaseFunction::ResetToWildcards()
{
	FScopedTransaction Transaction(LOCTEXT("ResetToDefaultsTx", "ResetToDefaults"));
	Modify();

	// Get pin refs
	UEdGraphPin* APin = FindPin(FEaseFunctionNodeHelper::GetAPinName());
	UEdGraphPin* BPin = FindPin(FEaseFunctionNodeHelper::GetBPinName());
	UEdGraphPin* ResultPin = FindPin(FEaseFunctionNodeHelper::GetResultPinName());
	
	// Set all to defaults and break links
	APin->DefaultValue = TEXT("");
	BPin->DefaultValue = TEXT("");
	APin->BreakAllPinLinks();
	BPin->BreakAllPinLinks();
	ResultPin->BreakAllPinLinks();

	// Do the rest of the work, we will not recompile because the wildcard pins will prevent it
	PinTypeChanged(APin);
}

void UK2Node_EaseFunction::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		if (Context->Pin == NULL)
		{
			{
				FToolMenuSection& Section = Menu->AddSection("UK2Node_EaseFunction", LOCTEXT("ContextMenuHeader", "Ease"));
				Section.AddMenuEntry(
					"AddPin",
					LOCTEXT("AddPin", "Reset to Wildcards"),
					LOCTEXT("AddPinTooltip", "Resets A, B and Results pins to its default wildcard state"),
					FSlateIcon(),
					FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_EaseFunction*>(this), &UK2Node_EaseFunction::ResetToWildcards)
					)
				);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
