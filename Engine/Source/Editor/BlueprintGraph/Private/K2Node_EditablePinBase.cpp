// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EditablePinBase.h"
#include "UObject/UnrealType.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Misc/FeedbackContext.h"
#include "EdGraphSchema_K2.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"

// Ensure that the UserDefinedPin's "desired direction" matches the direction of 
// the EdGraphPin that it corresponds to. Somehow it is possible for these to get 
// out of sync, and we're not entirely sure how/why.
//
// @TODO: Determine how these get out of sync and fix that up so we can guard it 
//        with a version check and not have to do this anymore for updated assets

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EditablePinBase)
#define ALWAYS_VALIDATE_DESIRED_PIN_DIRECTION_ON_LOAD 1


//////////////////////////////////////////////////////////////////////////
// FKismetUserDeclaredFunctionMetadata

bool FKismetUserDeclaredFunctionMetadata::HasMetaData(FName Key) const
{
	const FString* ValuePtr = nullptr;
	if (!Key.IsNone())
	{
		ValuePtr = MetaDataMap.Find(Key);
	}
	return ValuePtr != nullptr;
}

const FString& FKismetUserDeclaredFunctionMetadata::GetMetaData(FName Key) const
{
	// if not found, return a static empty string
	static FString EmptyString;

	if (Key.IsNone())
	{
		return EmptyString;
	}
	const FString* ValuePtr = MetaDataMap.Find(Key);
	return ValuePtr ? *ValuePtr : EmptyString;
}

void FKismetUserDeclaredFunctionMetadata::SetMetaData(FName Key, FString&& Value)
{
	if (!Key.IsNone())
	{
		MetaDataMap.Add(Key, MoveTempIfPossible(Value));
	}
}

void FKismetUserDeclaredFunctionMetadata::SetMetaData(FName Key, const FStringView Value)
{
	if (!Key.IsNone())
	{
		MetaDataMap.Add(Key, FString(Value));
	}
}

void FKismetUserDeclaredFunctionMetadata::RemoveMetaData(FName Key)
{
	if (!Key.IsNone())
	{
		MetaDataMap.Remove(Key);
	}
}

const TMap<FName, FString>& FKismetUserDeclaredFunctionMetadata::GetMetaDataMap() const
{
	return MetaDataMap;
}

//////////////////////////////////////////////////////////////////////////
// FUserPinInfo

FArchive& operator<<(FArchive& Ar, FUserPinInfo& Info)
{
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::PinsStoreFName)
	{
		Ar << Info.PinName;
	}
	else
	{
		FString PinNameStr;
		Ar << PinNameStr;
		Info.PinName = *PinNameStr;
	}

	if (Ar.UEVer() >= VER_UE4_SERIALIZE_PINTYPE_CONST)
	{
		Info.PinType.Serialize(Ar);
		Ar << Info.DesiredPinDirection;
	}
	else
	{
		check(Ar.IsLoading());

		bool bIsArray = (Info.PinType.ContainerType == EPinContainerType::Array);
		Ar << bIsArray;

		bool bIsReference = Info.PinType.bIsReference;
		Ar << bIsReference;

		Info.PinType.ContainerType = (bIsArray ? EPinContainerType::Array : EPinContainerType::None);
		Info.PinType.bIsReference = bIsReference;

		FString PinCategoryStr;
		FString PinSubCategoryStr;
		
		Ar << PinCategoryStr;
		Ar << PinSubCategoryStr;

		Info.PinType.PinCategory = *PinCategoryStr;
		Info.PinType.PinSubCategory = *PinSubCategoryStr;

		bool bFixupPinCategories =
			((Info.PinType.PinCategory == TEXT("double")) || (Info.PinType.PinCategory == TEXT("float")));

		if (bFixupPinCategories)
		{
			Info.PinType.PinCategory = TEXT("real");
			Info.PinType.PinSubCategory = TEXT("double");
		}

		Ar << Info.PinType.PinSubCategoryObject;
	}

	Ar << Info.PinDefaultValue;

	return Ar;
}

//////////////////////////////////////////////////////////////////////////
// UK2Node_EditablePinBase

UK2Node_EditablePinBase::UK2Node_EditablePinBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_EditablePinBase::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Add in pins based on the user defined pins in this node
	for(int32 i = 0; i < UserDefinedPins.Num(); i++)
	{
		FText DummyErrorMsg;
		if ((!IsEditable() || CanCreateUserDefinedPin(UserDefinedPins[i]->PinType, UserDefinedPins[i]->DesiredPinDirection, DummyErrorMsg)) && !FindPin(UserDefinedPins[i]->PinName))
		{
			CreatePinFromUserDefinition(UserDefinedPins[i]);
		}
	}
}

UEdGraphPin* UK2Node_EditablePinBase::CreateUserDefinedPin(const FName InPinName, const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, bool bUseUniqueName)
{
	// Sanitize the name, if needed
	const FName NewPinName = bUseUniqueName ? CreateUniquePinName(InPinName) : InPinName;

	// First, add this pin to the user-defined pins
	TSharedPtr<FUserPinInfo> NewPinInfo = MakeShareable( new FUserPinInfo() );
	NewPinInfo->PinName = NewPinName;
	NewPinInfo->PinType = InPinType;
	NewPinInfo->DesiredPinDirection = InDesiredDirection;
	UserDefinedPins.Add(NewPinInfo);

	// Then, add the pin to the actual Pins array
	UEdGraphPin* NewPin = CreatePinFromUserDefinition(NewPinInfo);

	return NewPin;
}

void UK2Node_EditablePinBase::RemoveUserDefinedPin(TSharedPtr<FUserPinInfo> PinToRemove)
{
	RemoveUserDefinedPinByName(PinToRemove->PinName);
}

void UK2Node_EditablePinBase::RemoveUserDefinedPinByName(const FName PinName)
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->PinName == PinName)
		{
			Pin->Modify();

			Pins.Remove(Pin);
			Pin->MarkAsGarbage();

			if (UBlueprint* Blueprint = GetBlueprint())
			{
				FKismetDebugUtilities::RemovePinWatch(Blueprint, Pin);
			}

			break;
		}
	}

	// Remove the description from the user-defined pins array
	UserDefinedPins.RemoveAll([&](const TSharedPtr<FUserPinInfo>& UDPin)
	{
		return UDPin.IsValid() && (UDPin->PinName == PinName);
	});
}

bool UK2Node_EditablePinBase::UserDefinedPinExists(const FName PinName) const
{
	return UserDefinedPins.ContainsByPredicate([&PinName](const TSharedPtr<FUserPinInfo>& UDPin)
	{
		return UDPin.IsValid() && (UDPin->PinName == PinName);
	});
}

void UK2Node_EditablePinBase::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	Super::ExportCustomProperties(Out, Indent);

	const FUserPinInfo DefaultPinInfo;

	for (int32 PinIndex = 0; PinIndex < UserDefinedPins.Num(); ++PinIndex)
	{
		const FUserPinInfo& PinInfo = *UserDefinedPins[PinIndex].Get();
		
		FString PinInfoStr;
		FUserPinInfo::StaticStruct()->ExportText(PinInfoStr, &PinInfo, &DefaultPinInfo, this, 0, nullptr, false);

		Out.Logf( TEXT("%sCustomProperties UserDefinedPin %s\r\n"), FCString::Spc(Indent), *PinInfoStr);
	}
}

void UK2Node_EditablePinBase::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	if (FParse::Command(&SourceText, TEXT("UserDefinedPin")))
	{
		TSharedPtr<FUserPinInfo> SharedPinInfo = MakeShareable(new FUserPinInfo());
		FUserPinInfo* PinInfo = SharedPinInfo.Get();

		FUserPinInfo::StaticStruct()->ImportText(SourceText, PinInfo, this, 0, Warn, TEXT("PinInfo"), false);

		UserDefinedPins.Add(SharedPinInfo);
	}
	else
	{
		Super::ImportCustomProperties(SourceText, Warn);
	}
}

void UK2Node_EditablePinBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	TArray<FUserPinInfo> SerializedItems;
	if (Ar.IsLoading())
	{
		Ar << SerializedItems;

		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		UserDefinedPins.Empty(SerializedItems.Num());
		for (int32 Index = 0; Index < SerializedItems.Num(); ++Index)
		{
			TSharedPtr<FUserPinInfo> PinInfo = MakeShareable(new FUserPinInfo(SerializedItems[Index]));

#if !ALWAYS_VALIDATE_DESIRED_PIN_DIRECTION_ON_LOAD
			// @TODO - Replace with a version check here if/when we are able to avoid this validation step on updated assets.
			const bool bValidateDesiredPinDirection = true;
#endif
			// Ensure that array type inputs and non-array type pass-by-reference inputs are also marked 'const' for both custom
			// event signatures and interface functions that have no return value. Since arrays are implicitly passed by reference,
			// and since events do not have return values/outputs, this equates to marking the parameter as 'const Type&' in native.
			// code. Also note that since UHT already blocks non-const reference types from being compiled into a MC delegate signature,
			// any existing custom event param pins that were implicitly created via "Assign" in the Blueprint editor's context menu
			// will previously have had 'const' set for its pin type.
			//
			// This ensures that (a) we don't emit the "no reference will be returned" note on custom event and implemented interface
			// event nodes with array inputs added by the user via the Details panel, and (b) we don't emit the "no reference will be
			// returned" warning on custom event and implemented interface event nodes with struct/object inputs added by the user in
			// the Details tab that are also explicitly set to pass-by-reference. That message is intended to convey one should not
			// expect the input to also be treated like an output - if the value is modified inside the event, it won't be reflected
			// back out to the caller through the reference. The message should only be seen for implemented event signatures with I/O
			// parameters that are explicitly passed by reference and declared in native C++ code using 'UPARAM(ref)' markup instead
			// of 'const Type&' - i.e. the 'const' form should be used to declare input ref args in native events with no return value.
			//
			// However, on the Blueprint side for new custom event and new interface event signature input arguments, we do not currently
			// expose a 'const' qualifier for pass-by-reference parameters, so we're implicitly adding one here to older input ref pin
			// types to be consistent with the native side. The Blueprint compiler currently ignores 'const' in terms of whether or not
			// the referenced value or object is actually treated as read-only in the event's implementation, but this may change later.
			//
			// Note that Blueprint details customization will set 'const' for all new custom event and implemented interface event node
			// placements with an array type or pass-by-reference input param. See FBlueprintGraphArgumentLayout::PinInfoChanged().
			//
			const bool bValidateConstRefPinTypes = Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::EditableEventsUseConstRefParameters
				&& ShouldUseConstRefParams();

			// Avoid the FindPin() call if we don't need to do it.
#if !ALWAYS_VALIDATE_DESIRED_PIN_DIRECTION_ON_LOAD
			if (bValidateDesiredPinDirection || bValidateConstRefPinTypes)
#endif
			{
				if (UEdGraphPin* NodePin = FindPin(PinInfo->PinName))
				{
#if !ALWAYS_VALIDATE_DESIRED_PIN_DIRECTION_ON_LOAD
					if (bValidateDesiredPinDirection)
#endif
					{
						// NOTE: the second FindPin call here to keep us from altering a pin with the same 
						//       name but different direction (in case there is two)
						if (PinInfo->DesiredPinDirection != NodePin->Direction && FindPin(PinInfo->PinName, PinInfo->DesiredPinDirection) == nullptr)
						{
							PinInfo->DesiredPinDirection = NodePin->Direction;
						}
					}

					if (bValidateConstRefPinTypes)
					{
						// Note that we should only get here if ShouldUseConstRefParams() indicated this node represents an event function with no outputs (above).
						if (!NodePin->PinType.bIsConst
							&& NodePin->Direction == EGPD_Output
							&& !K2Schema->IsExecPin(*NodePin)
							&& !K2Schema->IsDelegateCategory(NodePin->PinType.PinCategory))
						{
							// Add 'const' to either an array pin type (always passed by reference) or a pin type that's explicitly flagged to be passed by reference.
							NodePin->PinType.bIsConst = NodePin->PinType.IsArray() || NodePin->PinType.bIsReference;

							// Also mirror the flag into the UserDefinedPins array.
							PinInfo->PinType.bIsConst = NodePin->PinType.bIsConst;
						}
					}
				}
			}

			// We need to fixup text default values because many of them got saved with the wrong package namespace
			/*if (!PinInfo->PinDefaultValue.IsEmpty() && PinInfo->PinType.PinCategory == UEdGraphSchema_K2::PC_Text)
			{
				FString UseDefaultValue;
				UObject* UseDefaultObject = nullptr;
				FText UseDefaultText;

				K2Schema->GetPinDefaultValuesFromString(PinInfo->PinType, this, PinInfo->PinDefaultValue, UseDefaultValue, UseDefaultObject, UseDefaultText,false);

				if (!UseDefaultText.IsEmpty())
				{
					// This will have the namespace changed if needed
					PinInfo->PinDefaultValue.Reset();
					FTextStringHelper::WriteToBuffer(PinInfo->PinDefaultValue, UseDefaultText);
				}
			}*/

			UserDefinedPins.Add(PinInfo);
		}
	}
	else if(Ar.IsSaving())
	{
		SerializedItems.Empty(UserDefinedPins.Num());

		for (int32 Index = 0; Index < UserDefinedPins.Num(); ++Index)
		{
			SerializedItems.Add(*(UserDefinedPins[Index].Get()));
		}

		Ar << SerializedItems;
	}
	else
	{
		// We want to avoid destroying and recreating FUserPinInfo, because that will invalidate 
		// any WeakPtrs to those entries:
		for(TSharedPtr<FUserPinInfo>& PinInfo : UserDefinedPins )
		{
			Ar << *PinInfo;
		}
	}
}

void UK2Node_EditablePinBase::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UK2Node_EditablePinBase* This = CastChecked<UK2Node_EditablePinBase>(InThis);
	for (int32 Index = 0; Index < This->UserDefinedPins.Num(); ++Index)
	{
		FUserPinInfo PinInfo = *This->UserDefinedPins[Index].Get();
		Collector.AddReferencedObject(PinInfo.PinType.PinSubCategoryObject, This);
	}
	Super::AddReferencedObjects( This, Collector );
}

void UK2Node_EditablePinBase::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	static bool bRecursivelyChangingDefaultValue = false;

	// Only do this if we're editable and not already calling this code
	if (!bIsEditable || bRecursivelyChangingDefaultValue)
	{
		return;
	}

	// See if this is a user defined pin
	for (int32 Index = 0; Index < UserDefinedPins.Num(); ++Index)
	{
		TSharedPtr<FUserPinInfo> PinInfo = UserDefinedPins[Index];
		if (Pin->PinName == PinInfo->PinName && Pin->Direction == PinInfo->DesiredPinDirection)
		{
			FString DefaultsString = Pin->GetDefaultAsString();

			if (DefaultsString != PinInfo->PinDefaultValue)
			{
				// Make sure this doesn't get called recursively
				TGuardValue<bool> CircularGuard(bRecursivelyChangingDefaultValue, true);
				ModifyUserDefinedPinDefaultValue(PinInfo, Pin->GetDefaultAsString());
			}
		}
	}
}

bool UK2Node_EditablePinBase::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& InDefaultValue)
{
	FString NewDefaultValue = InDefaultValue;

	if (!UpdateEdGraphPinDefaultValue(PinInfo, NewDefaultValue))
	{
		return false;
	}

	PinInfo->PinDefaultValue = NewDefaultValue;
	return true;
}

bool UK2Node_EditablePinBase::UpdateUserDefinedPinDefaultValues()
{
	bool bAnyChanged = false;
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	
	// If any of our pins got fixed up, we need to refresh our user pin default values
	for (TSharedPtr<FUserPinInfo> PinInfo : UserDefinedPins)
	{
		if (UEdGraphPin* Pin = FindPin(PinInfo->PinName))
		{
			if (Pin->Direction == PinInfo->DesiredPinDirection)
			{
				if (!K2Schema->DoesDefaultValueMatch(*Pin, PinInfo->PinDefaultValue))
				{
					bAnyChanged |= ModifyUserDefinedPinDefaultValue(PinInfo, Pin->GetDefaultAsString());
				}
			}
		}
	}
	return bAnyChanged;
}

bool UK2Node_EditablePinBase::UpdateEdGraphPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, FString& NewDefaultValue)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	// Find and modify the current pin
	if (UEdGraphPin* OldPin = FindPin(PinInfo->PinName))
	{
		FString SavedDefaultValue = OldPin->DefaultValue;

		K2Schema->SetPinAutogeneratedDefaultValue(OldPin, NewDefaultValue);

		// Validate the new default value
		FString ErrorString = K2Schema->IsCurrentPinDefaultValid(OldPin);

		if (!ErrorString.IsEmpty())
		{
			NewDefaultValue = SavedDefaultValue;
			K2Schema->SetPinAutogeneratedDefaultValue(OldPin, SavedDefaultValue);

			return false;
		}
	}

	return true;
}

bool UK2Node_EditablePinBase::CreateUserDefinedPinsForFunctionEntryExit(const UFunction* Function, bool bForFunctionEntry)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Create the inputs and outputs
	bool bAllPinsGood = true;
	for ( TFieldIterator<FProperty> PropIt(Function); PropIt && ( PropIt->PropertyFlags & CPF_Parm ); ++PropIt )
	{
		FProperty* Param = *PropIt;

		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);

		if ( bIsFunctionInput == bForFunctionEntry )
		{
			const EEdGraphPinDirection Direction = bForFunctionEntry ? EGPD_Output : EGPD_Input;

			FEdGraphPinType PinType;
			K2Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);

			const bool bPinGood = (CreateUserDefinedPin(Param->GetFName(), PinType, Direction) != nullptr);

			bAllPinsGood = bAllPinsGood && bPinGood;
		}
	}

	return bAllPinsGood;
}

FUserPinInfo::FUserPinInfo(const UEdGraphPin& InPin)
	: PinName(InPin.GetFName())
	, PinType(InPin.PinType)
	, DesiredPinDirection(InPin.Direction)
	, PinDefaultValue(InPin.DefaultValue)
{
}
