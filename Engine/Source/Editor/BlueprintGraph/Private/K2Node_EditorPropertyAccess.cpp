// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EditorPropertyAccess.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EditorPropertyAccess)

#define LOCTEXT_NAMESPACE "K2Node_EditorPropertyAccess"

void UK2Node_EditorPropertyAccessBase::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* ResultPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);
	ResultPin->PinFriendlyName = LOCTEXT("ResultPinFriendlyName", "Success?");
}

void UK2Node_EditorPropertyAccessBase::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

bool UK2Node_EditorPropertyAccessBase::CanPasteHere(const UEdGraph* TargetGraph) const
{
	bool bCanPaste = Super::CanPasteHere(TargetGraph);
	if (bCanPaste)
	{
		bCanPaste &= FBlueprintEditorUtils::IsEditorUtilityBlueprint(FBlueprintEditorUtils::FindBlueprintForGraphChecked(TargetGraph));
	}
	return bCanPaste;
}

bool UK2Node_EditorPropertyAccessBase::IsActionFilteredOut(const FBlueprintActionFilter& Filter)
{
	bool bIsFilteredOut = Super::IsActionFilteredOut(Filter);
	if (!bIsFilteredOut)
	{
		for (UEdGraph* TargetGraph : Filter.Context.Graphs)
		{
			bIsFilteredOut |= !CanPasteHere(TargetGraph);
		}
	}
	return bIsFilteredOut;
}

void UK2Node_EditorPropertyAccessBase::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	static const FName PN_Object = "Object";
	static const FName PN_PropertyName = "PropertyName";

	UEdGraphPin* ObjectPin = FindPinChecked(PN_Object, EGPD_Input);
	if (ObjectPin->LinkedTo.Num() == 0 && !ObjectPin->DefaultObject)
	{
		MessageLog.Error(*LOCTEXT("UnsetObject", "No object set on @@").ToString(), this);
	}

	UEdGraphPin* PropertyNamePin = FindPinChecked(PN_PropertyName, EGPD_Input);
	if (PropertyNamePin->LinkedTo.Num() == 0 && FName(*PropertyNamePin->DefaultValue).IsNone())
	{
		MessageLog.Error(*LOCTEXT("UnsetPropertyName", "No property name set on @@").ToString(), this);
	}
}

UK2Node_GetEditorProperty::UK2Node_GetEditorProperty()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, GetEditorProperty), UKismetSystemLibrary::StaticClass());
}

UK2Node_SetEditorProperty::UK2Node_SetEditorProperty()
{
	FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, SetEditorProperty), UKismetSystemLibrary::StaticClass());
}

#undef LOCTEXT_NAMESPACE
