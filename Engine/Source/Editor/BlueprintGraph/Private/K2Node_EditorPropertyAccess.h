// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "BlueprintActionFilter.h"
#include "K2Node_CallFunction.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "K2Node_EditorPropertyAccess.generated.h"

class FBlueprintActionDatabaseRegistrar;
class UEdGraph;
class UObject;

UCLASS(Abstract)
class UK2Node_EditorPropertyAccessBase : public UK2Node_CallFunction
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface.
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface.

	//~ Begin UK2Node Interface
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual bool CanPasteHere(const UEdGraph* TargetGraph) const override;
	virtual bool IsActionFilteredOut(const class FBlueprintActionFilter& Filter) override;
	virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UK2Node Interface

protected:
	//~ Begin UK2Node_CallFunction interface
	virtual bool CanToggleNodePurity() const override { return false; }
	//~ End UK2Node_CallFunction interface
};

UCLASS()
class UK2Node_GetEditorProperty : public UK2Node_EditorPropertyAccessBase
{
	GENERATED_BODY()

public:
	UK2Node_GetEditorProperty();
};

UCLASS()
class UK2Node_SetEditorProperty : public UK2Node_EditorPropertyAccessBase
{
	GENERATED_BODY()

public:
	UK2Node_SetEditorProperty();
};
