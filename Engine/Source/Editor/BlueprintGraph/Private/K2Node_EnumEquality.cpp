// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_EnumEquality.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformMath.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Knot.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EnumEquality)

#define LOCTEXT_NAMESPACE "K2Node_EnumEquality"

UK2Node_EnumEquality::UK2Node_EnumEquality(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_EnumEquality::AllocateDefaultPins()
{
	// Create the return value pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Boolean, UEdGraphSchema_K2::PN_ReturnValue);

	// Create the input pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, TEXT("A"));
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, TEXT("B"));

	Super::AllocateDefaultPins();
}

FText UK2Node_EnumEquality::GetTooltipText() const
{
	return LOCTEXT("EnumEqualityTooltip", "Returns true if A is equal to B (A == B)");
}

FText UK2Node_EnumEquality::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("EqualEnum", "Equal (Enum)");
}

void UK2Node_EnumEquality::PostReconstructNode()
{
	// Do type determination before enum fixup
	PinConnectionListChanged(GetInput1Pin());
	PinConnectionListChanged(GetInput2Pin());

	Super::PostReconstructNode();
}

/** Determine if any pins are connected, if so make all the other pins the same type, if not, make sure pins are switched back to wildcards */
void UK2Node_EnumEquality::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* Input1Pin = GetInput1Pin();
	UEdGraphPin* Input2Pin = GetInput2Pin();

	if (Pin == Input1Pin || Pin == Input2Pin)
	{
		if ((Input1Pin->LinkedTo.Num() == 0) && (Input2Pin->LinkedTo.Num() == 0))
		{
			// Restore the wildcard status
			Input1Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			Input1Pin->PinType.PinSubCategory= NAME_None;
			Input1Pin->PinType.PinSubCategoryObject = nullptr;
			Input2Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			Input2Pin->PinType.PinSubCategory = NAME_None;
			Input2Pin->PinType.PinSubCategoryObject = nullptr;
			Schema->SetPinAutogeneratedDefaultValueBasedOnType(Input1Pin);
			Schema->SetPinAutogeneratedDefaultValueBasedOnType(Input2Pin);
			// We have to refresh the graph to get the enum dropdowns to go away
			GetGraph()->NotifyNodeChanged(this);
		}
		else if (Pin->LinkedTo.Num() > 0)
		{
			UEdGraphPin* LinkedPin = Pin->LinkedTo[0];
			check(LinkedPin);
			
			// If we're linked to a knot node, ensure that its enum data is up-to-date.
			// Otherwise, PinSubCategoryObject might be null if we're reconstructed prior to our knot dependencies.
			UK2Node_Knot* LinkedKnotNode = Cast<UK2Node_Knot>(LinkedPin->GetOwningNode());
			if (LinkedKnotNode)
			{
				LinkedKnotNode->PropagatePinType();
			}

			// Make sure the pin is a valid enum
			if (LinkedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
				LinkedPin->PinType.PinSubCategoryObject.IsValid() &&
				LinkedPin->PinType.PinSubCategoryObject.Get()->IsA(UEnum::StaticClass()))
			{
				Pin->PinType = LinkedPin->PinType;

				UEdGraphPin* OtherPin = (Input1Pin == Pin) ? Input2Pin : Input1Pin;

				// Enforce the type on the other pin
				OtherPin->PinType = Pin->PinType;
				UEdGraphSchema_K2::ValidateExistingConnections(OtherPin);
				// If we copied the pin type to a wildcard we have to refresh the graph to get the enum dropdown
				if (OtherPin->LinkedTo.Num() <= 0 && OtherPin->DefaultValue.IsEmpty())
				{
					Schema->SetPinAutogeneratedDefaultValueBasedOnType(OtherPin);
					GetGraph()->NotifyNodeChanged(this);
				}
			}
			// A valid enum wasn't used to break the links
			else
			{
				Pin->BreakAllPinLinks();
			}
		}
		else if(Pin->DefaultValue.IsEmpty())
		{
			Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
		}
	}
}

bool UK2Node_EnumEquality::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	bool const bOtherPinIsEnum = (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte) &&
		(OtherPin->PinType.PinSubCategoryObject.IsValid()) &&
		(OtherPin->PinType.PinSubCategoryObject.Get()->IsA(UEnum::StaticClass()));


	bool bIsDisallowed = false;
	if (!bOtherPinIsEnum && (MyPin->Direction == EGPD_Input))
	{
		bIsDisallowed = true;
		OutReason = LOCTEXT("InputIsNotEnum", "Cannot use the enum equality operator on anything but enums.").ToString();
	}
	return bIsDisallowed;
}

UEdGraphPin* UK2Node_EnumEquality::GetReturnValuePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin);
	return Pin;
}

UEdGraphPin* UK2Node_EnumEquality::GetInput1Pin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("A"));
	check(Pin);
	return Pin;
}

UEdGraphPin* UK2Node_EnumEquality::GetInput2Pin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("B"));
	check(Pin);
	return Pin;
}

void UK2Node_EnumEquality::GetConditionalFunction(FName& FunctionName, UClass** FunctionClass) const
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, EqualEqual_ByteByte);
	*FunctionClass = UKismetMathLibrary::StaticClass();
}

FNodeHandlingFunctor* UK2Node_EnumEquality::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_EnumEquality::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	// Get the enum equality node and the KismetMathLibrary function info for use when we build those nodes
	FName ConditionalFunctionName = "";
	UClass* ConditionalFunctionClass = NULL;
	GetConditionalFunction(ConditionalFunctionName, &ConditionalFunctionClass);

	// Create the conditional node we're replacing the enum node for
	UK2Node_CallFunction* ConditionalNode = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
	ConditionalNode->FunctionReference.SetExternalMember(ConditionalFunctionName, ConditionalFunctionClass);
	ConditionalNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(ConditionalNode, this);

	// Rewire the enum pins to the new conditional node
	UEdGraphPin* LeftSideConditionalPin = ConditionalNode->FindPinChecked(TEXT("A"));
	UEdGraphPin* RightSideConditionalPin = ConditionalNode->FindPinChecked(TEXT("B"));
	UEdGraphPin* ReturnConditionalPin = ConditionalNode->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* Input1Pin = GetInput1Pin();
	UEdGraphPin* Input2Pin = GetInput2Pin();
	LeftSideConditionalPin->PinType = Input1Pin->PinType;
	RightSideConditionalPin->PinType = Input2Pin->PinType;
	CompilerContext.MovePinLinksToIntermediate(*Input1Pin, *LeftSideConditionalPin);
	CompilerContext.MovePinLinksToIntermediate(*Input2Pin, *RightSideConditionalPin);
	CompilerContext.MovePinLinksToIntermediate(*GetReturnValuePin(), *ReturnConditionalPin);

	// Break all links to the Select node so it goes away for at scheduling time
	BreakAllNodeLinks();
}

void UK2Node_EnumEquality::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_EnumEquality::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Enum);
}

#undef LOCTEXT_NAMESPACE
