// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_EnumInequality.h"

//#include "KismetCompiler.h"
#include "Kismet/KismetMathLibrary.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EnumInequality)

class UClass;
//#include "BlueprintNodeSpawner.h"
//#include "EditorCategoryUtils.h"
//#include "BlueprintActionDatabaseRegistrar.h"

#define LOCTEXT_NAMESPACE "K2Node_EnumInequality"

UK2Node_EnumInequality::UK2Node_EnumInequality(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UK2Node_EnumInequality::GetTooltipText() const
{
	return LOCTEXT("EnumInequalityTooltip", "Returns true if A is NOT equal to B (A != B)");
}

FText UK2Node_EnumInequality::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node_EnumInequality", "NotEqualEnum", "Not Equal (Enum)");
}

void UK2Node_EnumInequality::GetConditionalFunction(FName& FunctionName, UClass** FunctionClass) const
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, NotEqual_ByteByte);
	*FunctionClass = UKismetMathLibrary::StaticClass();
}


#undef LOCTEXT_NAMESPACE
