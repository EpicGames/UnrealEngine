// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_EnumLiteral.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFieldNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CastByteToEnum.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Field.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_EnumLiteral)

class UBlueprintNodeSpawner;
class UEdGraph;
struct FLinearColor;

const FName UK2Node_EnumLiteral::GetEnumInputPinName()
{
	static const FName Name(TEXT("Enum"));
	return Name;
}

UK2Node_EnumLiteral::UK2Node_EnumLiteral(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_EnumLiteral::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
	if (!Enum)
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "EnumLiteral_NullEnumError", "Undefined Enum in @@").ToString(), this);
	}
}

void UK2Node_EnumLiteral::AddSearchMetaDataInfo(TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	const UEdGraphPin* Pin = FindPinChecked(GetEnumInputPinName());
	if (!Pin->DefaultValue.IsEmpty())
	{
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(Pin->DefaultValue)));

		const int32 ValueIndex = Enum ? Enum->GetIndexByName(*Enum->GenerateFullEnumName(*Pin->DefaultValue)) : INDEX_NONE;
		if (ValueIndex != INDEX_NONE)
		{
			FText SearchName = FText::FormatOrdered(NSLOCTEXT("K2Node", "EnumLiteral_SearchName", "{0} - {1}"), GetTooltipText(), Enum->GetDisplayNameTextByIndex(ValueIndex));

			// Find the Name, populated by Super::AddSearchMetaDataInfo
			for (FSearchTagDataPair& SearchData : OutTaggedMetaData)
			{
				// Should always be the first item, but there is no guarantee
				if (SearchData.Key.CompareTo(FFindInBlueprintSearchTags::FiB_Name) == 0)
				{
					SearchData.Value = SearchName;
					break;
				}
			}
		}
	}
}

void UK2Node_EnumLiteral::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* InputPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, Enum, GetEnumInputPinName());
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(InputPin);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Byte, Enum, UEdGraphSchema_K2::PN_ReturnValue);

	Super::AllocateDefaultPins();
}

FSlateIcon UK2Node_EnumLiteral::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x");
	return Icon;
}

FText UK2Node_EnumLiteral::GetTooltipText() const
{
	if (Enum == nullptr)
	{
		return NSLOCTEXT("K2Node", "BadEnumLiteral_Tooltip", "Literal enum (bad enum)");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumName"), FText::FromName(Enum->GetFName()));
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "EnumLiteral_Tooltip", "Literal enum {EnumName}"), Args), this);
	}
	return CachedTooltip;	
}

FText UK2Node_EnumLiteral::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		return GetTooltipText();
	}
	return CachedTooltip;
}

void UK2Node_EnumLiteral::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	bool bSuccess = Enum != nullptr;
	if (bSuccess)
	{
		const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralByte);
		UK2Node_CallFunction* MakeLiteralByteNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		MakeLiteralByteNode->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(FunctionName));
		MakeLiteralByteNode->AllocateDefaultPins();

		UEdGraphPin* OrgInputPin = FindPinChecked(GetEnumInputPinName());
		UEdGraphPin* NewInputPin = MakeLiteralByteNode->FindPinChecked(TEXT("Value"));

		// read the enum value, convert to an integer:
		int32 Index = Enum->GetIndexByName(*OrgInputPin->DefaultValue);
		bSuccess = Index != INDEX_NONE;
		if (bSuccess)
		{
			NewInputPin->DefaultValue = FString::FromInt(IntCastChecked<int32, int64>(Enum->GetValueByIndex(Index)));
		}

		UK2Node_CastByteToEnum* CastByte = CompilerContext.SpawnIntermediateNode<UK2Node_CastByteToEnum>(this, SourceGraph);
		CastByte->Enum = Enum;
		CastByte->AllocateDefaultPins();

		UEdGraphPin* CastInputPin = CastByte->FindPinChecked(UK2Node_CastByteToEnum::ByteInputPinName);
		UEdGraphPin* NewReturnPin = MakeLiteralByteNode->GetReturnValuePin();
		check(nullptr != NewReturnPin);
		bSuccess &= CompilerContext.GetSchema()->TryCreateConnection(NewReturnPin, CastInputPin);

		UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		UEdGraphPin* CastReturnPin = CastByte->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *CastReturnPin);
	}

	if (!bSuccess)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "EnumLiteral_NullEnumError", "Undefined Enum in @@").ToString(), this);
	}
}

void UK2Node_EnumLiteral::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, FFieldVariant /*EnumField*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UK2Node_EnumLiteral* EnumNode = CastChecked<UK2Node_EnumLiteral>(NewNode);
			EnumNode->Enum = NonConstEnumPtr.Get();
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* InEnum)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UEnum*>(InEnum));
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(InEnum));
		NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

FText UK2Node_EnumLiteral::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Enum);
}

void UK2Node_EnumLiteral::ReloadEnum(class UEnum* InEnum)
{
	Enum = InEnum;
	CachedTooltip.MarkDirty();
}
