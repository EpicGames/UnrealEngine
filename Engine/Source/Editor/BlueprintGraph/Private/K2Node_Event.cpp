// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Event.h"

#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Engine/Blueprint.h"
#include "EngineLogs.h"
#include "EventEntryHandler.h"
#include "FindInBlueprints.h"
#include "GameFramework/Actor.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CreateDelegate.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_Self.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "ObjectTools.h"
#include "Serialization/Archive.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/Class.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectVersion.h"
#include "UObject/Script.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Event)

const FName UK2Node_Event::DelegateOutputName(TEXT("OutputDelegate"));

bool UK2Node_Event::IsCosmeticTickEvent() const
{
	// Special case for EventTick/ReceiveTick that is conditionally executed by a separate bool rather than function flag.
	static const FName EventTickName(TEXT("ReceiveTick"));
	if (EventReference.GetMemberName() == EventTickName)
	{
		const UBlueprint* Blueprint = GetBlueprint();
		if (Blueprint)
		{
			UClass* BPClass = Blueprint->GeneratedClass;
			const AActor* DefaultActor = BPClass ? Cast<const AActor>(BPClass->GetDefaultObject()) : nullptr;
			if (DefaultActor && !DefaultActor->AllowReceiveTickEventOnDedicatedServer())
			{
				return true;
			}
		}
	}

	return false;
}


#define LOCTEXT_NAMESPACE "K2Node_Event"

UK2Node_Event::UK2Node_Event(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FunctionFlags = 0;
}

void UK2Node_Event::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);
	// Fix up legacy nodes that may not yet have a delegate pin
	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE)
		{
			EventReference.SetExternalMember(EventSignatureName_DEPRECATED, EventSignatureClass_DEPRECATED);
		}

		if (Ar.CustomVer(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::OverridenEventReferenceFixup)
		{
			FixupEventReference();
		}
	}
}

void UK2Node_Event::PostLoad()
{
	UK2Node_EditablePinBase::PostLoad();

	// Fix up legacy nodes that may not yet have a delegate pin
	if (!FindPin(DelegateOutputName))
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Delegate, DelegateOutputName);
	}
}

void UK2Node_Event::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE)
	{
		FixupEventReference();
	}
}

FNodeHandlingFunctor* UK2Node_Event::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_EventEntry(CompilerContext);
}


FLinearColor UK2Node_Event::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FText UK2Node_Event::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (bOverrideFunction || (CustomFunctionName == NAME_None))
	{
		FText FunctionName = FText::FromName(EventReference.GetMemberName()); // If we fail to find the function, still want to write something on the node.

		if (UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
		{
			FunctionName = UEdGraphSchema_K2::GetFriendlySignatureName(Function);
		}

		FFormatNamedArguments Args;
		Args.Add(TEXT("FunctionName"), FunctionName);
		FText Title = FText::Format(NSLOCTEXT("K2Node", "Event_Name", "Event {FunctionName}"), Args);

		if(TitleType == ENodeTitleType::FullTitle && EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) != nullptr && EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->IsChildOf(UInterface::StaticClass()))
		{
			const FText SignatureClassAsText = FBlueprintEditorUtils::GetFriendlyClassDisplayName(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()));

			FFormatNamedArguments FullTitleArgs;
			FullTitleArgs.Add(TEXT("Title"), Title);
			FullTitleArgs.Add(TEXT("InterfaceClass"), SignatureClassAsText);

			Title = FText::Format(LOCTEXT("EventFromInterface", "{Title}\nFrom {InterfaceClass}"), FullTitleArgs);
		}

		return Title;
	}
	else
	{
		return FText::FromName(CustomFunctionName);
	}
}

FText UK2Node_Event::GetTooltipText() const
{
	UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	if (CachedTooltip.IsOutOfDate(this) && (Function != nullptr))
	{
		CachedTooltip.SetCachedText(FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function)), this);

		if (bOverrideFunction || (CustomFunctionName == NAME_None))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("FunctionTooltip"), (FText&)CachedTooltip);

			//@TODO: KISMETREPLICATION: Should do this for events with a custom function name, if it's a newly introduced replicating thingy
			if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic) || IsCosmeticTickEvent())
			{
				Args.Add(
					TEXT("ClientString"),
					NSLOCTEXT("K2Node", "ClientEvent", "\n\nCosmetic. This event is only for cosmetic, non-gameplay actions.")
				);
				// FText::Format() is slow, so we cache this to save on performance
				CachedTooltip.SetCachedText(FText::Format(LOCTEXT("Event_SubtitledTooltip", "{FunctionTooltip}\n\n{ClientString}"), Args), this);
			} 
			else if(Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
			{
				Args.Add(
					TEXT("ClientString"),
					NSLOCTEXT("K2Node", "ServerEvent", "Authority Only. This event only fires on the server.")
				);
				// FText::Format() is slow, so we cache this to save on performance
				CachedTooltip.SetCachedText(FText::Format(LOCTEXT("Event_SubtitledTooltip", "{FunctionTooltip}\n\n{ClientString}"), Args), this);
			}
			else if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
			{
				Args.Add(
					TEXT("LatentString"),
					NSLOCTEXT("K2Node", "LatentFunction", "Latent. This node will complete at a later time. Latent nodes can only be placed in event graphs.")
				);
				// FText::Format() is slow, so we cache this to save on performance
				CachedTooltip.SetCachedText(FText::Format(LOCTEXT("CallFunction_SubtitledTooltip", "{DefaultTooltip}\n\n{LatentString}"), Args), this);
			}
		}		
	}

	return CachedTooltip;
}

FText UK2Node_Event::GetKeywords() const
{
	FText Keywords;

	UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	if (Function != nullptr)
	{
		Keywords = UK2Node_CallFunction::GetKeywordsForFunction( Function );
	}

	return Keywords;
}

FString UK2Node_Event::GetDocumentationLink() const
{
	if ( UClass* EventSignatureClass = EventReference.GetMemberParentClass(GetBlueprintClassFromNode()))
	{
		return FString::Printf(TEXT("Shared/Types/%s%s"), EventSignatureClass->GetPrefixCPP(), *EventSignatureClass->GetName());
	}

	return FString();
}

FString UK2Node_Event::GetDocumentationExcerptName() const
{
	return EventReference.GetMemberName().ToString();
}

void UK2Node_Event::PostReconstructNode()
{
	UpdateDelegatePin();

	Super::PostReconstructNode();
}


void UK2Node_Event::FixupEventReference(bool bForce /*= false*/)
{
	if (bOverrideFunction && !HasAnyFlags(RF_Transient))
	{
		if (!EventReference.IsSelfContext())
		{
			UBlueprint* Blueprint = GetBlueprint();
			UClass* BlueprintType = (Blueprint != nullptr) ? *Blueprint->SkeletonGeneratedClass : nullptr;

			UClass* ParentType = EventReference.GetMemberParentClass();

			if (BlueprintType && (bForce || !ParentType || !(BlueprintType->IsChildOf(ParentType) || BlueprintType->ImplementsInterface(ParentType))))
			{
				FName EventName = EventReference.GetMemberName();

				const UFunction* OverriddenFunc = BlueprintType->FindFunctionByName(EventName);
				while (OverriddenFunc != nullptr)
				{
					if (UFunction* SuperFunc = OverriddenFunc->GetSuperFunction())
					{
						OverriddenFunc = SuperFunc;
					}
					else
					{
						break;
					}
				}

				if (OverriddenFunc != nullptr)
				{
					UClass* SuperClass = OverriddenFunc->GetOwnerClass();
					if (UBlueprint* SuperBlueprint = Cast<UBlueprint>(SuperClass->ClassGeneratedBy))
					{
						SuperClass = SuperBlueprint->GeneratedClass;
					}

					if (SuperClass != nullptr)
					{
						EventReference.SetExternalMember(EventName, SuperClass);
					}
				}
			}
		}
	}
}

void UK2Node_Event::UpdateDelegatePin(bool bSilent)
{
	UEdGraphPin* Pin = FindPinChecked(DelegateOutputName);
	checkSlow(EGPD_Output == Pin->Direction);

	const UObject* OldSignature = FMemberReference::ResolveSimpleMemberReference<UFunction>(Pin->PinType.PinSubCategoryMemberReference);
	if (!OldSignature)
	{
		OldSignature = Pin->PinType.PinSubCategoryObject.Get();
	}

	UFunction* NewSignature = nullptr;
	if(bOverrideFunction)
	{
		NewSignature = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	}
	else if(UBlueprint* Blueprint = GetBlueprint())
	{
		NewSignature = Blueprint->SkeletonGeneratedClass
			? Blueprint->SkeletonGeneratedClass->FindFunctionByName(CustomFunctionName)
			: nullptr;
	}

	Pin->PinType.PinSubCategoryObject = nullptr;
	FMemberReference::FillSimpleMemberReference<UFunction>(NewSignature, Pin->PinType.PinSubCategoryMemberReference);

	if ((OldSignature != NewSignature) && !bSilent)
	{
		PinTypeChanged(Pin);
	}
}

void UK2Node_Event::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if(Pin == FindPin(DelegateOutputName)) 
	{
		UpdateDelegatePin();
	}

	Super::PinConnectionListChanged(Pin);
}

FName UK2Node_Event::GetFunctionName() const
{
	return bOverrideFunction ? EventReference.GetMemberName() : CustomFunctionName;
}

UFunction* UK2Node_Event::FindEventSignatureFunction() const
{
	return EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
}

void UK2Node_Event::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Delegate, DelegateOutputName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	
	// If the function signature is currently invalid then try and fix the event reference
	// in case the function has moved to some other valid location from the last compile
	UFunction* Function = FindEventSignatureFunction();
	
	if (!Function)
	{
		FixupEventReference(/*bForce*/ true);
		Function = FindEventSignatureFunction();
	}

	if (Function)
	{
		CreatePinsForFunctionEntryExit(Function, /*bIsFunctionEntry=*/ true);
	}

	UpdateDelegatePin(true);

	Super::AllocateDefaultPins();
}

void UK2Node_Event::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UFunction* Function = nullptr;
	if (bOverrideFunction)
	{
		Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
		if (!Function)
		{
			// If we are overriding a function, but we can;t find the function we are overriding, that is a compile error
			MessageLog.Error(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingEventSig_ErrorFmt", "Missing Event '{0}' for @@"), FText::FromString(EventReference.GetMemberName().ToString())).ToString(), this);
		}
		else if(Function->HasAnyFunctionFlags(FUNC_Const|FUNC_BlueprintPure))
		{
			// The implicit writes of object parameters to the event graph frame can violate clustering rules, so this is 
			// noting some quite severe shortcomings in the blueprint compiler - but starting with a note while we address
			// those:
			MessageLog.Note(*FText::Format(NSLOCTEXT("KismetCompiler", "EventOverridingConst_Fmt", "@@: Event overriding '{0}', which is const, implicitly writes to this object. Please convert this to a function - ESPECIALLY if this event has Object parameters"), FText::FromString(EventReference.GetMemberName().ToString())).ToString(), this);
		}
	}
	else if (UBlueprint* Blueprint = GetBlueprint())
	{
		Function = Blueprint->SkeletonGeneratedClass
			? Blueprint->SkeletonGeneratedClass->FindFunctionByName(CustomFunctionName)
			: nullptr;
	}

	FKismetCompilerUtilities::DetectValuesReturnedByRef(Function, this, MessageLog);
}

bool UK2Node_Event::NodeCausesStructuralBlueprintChange() const
{
	// FBlueprintEditor::CanAddParentNode requires actual data in skel class
	return true;
}

void UK2Node_Event::GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const
{
	Super::GetRedirectPinNames(Pin, RedirectPinNames);

	if ( RedirectPinNames.Num() > 0 )
	{
		const FString OldPinName = RedirectPinNames[0];

		// first add functionname.param
		RedirectPinNames.Add(FString::Printf(TEXT("%s.%s"), *EventReference.GetMemberName().ToString(), *OldPinName));
		// if there is class, also add an option for class.functionname.param
		if ( EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()) != nullptr )
		{
			if(const UClass* EventSignatureClass = EventReference.GetMemberParentClass(GetBlueprintClassFromNode()))
			{
				RedirectPinNames.Add(FString::Printf(TEXT("%s.%s.%s"), *EventSignatureClass->GetName(), *EventReference.GetMemberName().ToString(), *OldPinName));
			}
		}
	}
}

bool UK2Node_Event::IsFunctionEntryCompatible(const UK2Node_FunctionEntry* EntryNode) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Copy a set of the pin references for both nodes, so we can pare down lists
	TArray<UEdGraphPin*> EventPins = Pins;
	TArray<UEdGraphPin*> EntryPins = EntryNode->Pins;

	// Prune the exec wires and inputs (delegate binding) from both sets
	for(int32 i = 0; i < EventPins.Num(); i++)
	{
		const UEdGraphPin* CurPin = EventPins[i];
		if( CurPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec 
			|| CurPin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self
			|| CurPin->PinName == DelegateOutputName
			|| CurPin->Direction == EGPD_Input
			|| CurPin->ParentPin != nullptr )
		{
			EventPins.RemoveAt(i, 1);
			i--;
		}
	}

	for(int32 i = 0; i < EntryPins.Num(); i++)
	{
		const UEdGraphPin* CurPin = EntryPins[i];
		if( CurPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec 
			|| CurPin->PinType.PinSubCategory == UEdGraphSchema_K2::PSC_Self
			|| CurPin->PinName == DelegateOutputName
			|| CurPin->Direction == EGPD_Input
			|| CurPin->ParentPin != nullptr )
		{
			EntryPins.RemoveAt(i, 1);
			i--;
		}
	}

	// Early out:  we don't have the same number of parameters
	if( EventPins.Num() != EntryPins.Num() )
	{
		return false;
	}

	// Now check through the event's pins, and check for compatible pins, removing them if we find a match.
	for( int32 i = 0; i < EventPins.Num(); i++ )
	{
		const UEdGraphPin* CurEventPin = EventPins[i];

		bool bMatchFound = false;
		for( int32 j = 0; j < EntryPins.Num(); j++ )
		{
			const UEdGraphPin* CurEntryPin = EntryPins[j];
			if( CurEntryPin->PinName == CurEventPin->PinName )
			{
				// Check to make sure pins are of the same type
				if( K2Schema->ArePinTypesCompatible(CurEntryPin->PinType, CurEventPin->PinType) )
				{
					// Found a match, remove it from the list
					bMatchFound = true;
					EntryPins.RemoveAt(j, 1);
					break;
				}
				else
				{
					// Found a pin, but the type has changed, bail.
					bMatchFound = false;
					break;
				}
			}
		}

		if( bMatchFound )
		{
			// Found a match, remove it from the event array
			EventPins.RemoveAt(i, 1);
			i--;
		}
		else
		{
			// Didn't find a match...bail!
			return false;
		}
	}

	// Checked for matches, if any pins remain in either array, they were unmatched.
	return (EventPins.Num() == 0) && (EntryPins.Num() == 0);
}

bool UK2Node_Event::IsInterfaceEventNode() const
{
	return (EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) != nullptr) && EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->IsChildOf(UInterface::StaticClass());
}

bool UK2Node_Event::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool bIsCompatible = Super::IsCompatibleWithGraph(TargetGraph);
	if (bIsCompatible)
	{
		EGraphType const GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
		bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);
	}
	
	return bIsCompatible;
}

bool UK2Node_Event::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bDisallowPaste = !Super::CanPasteHere(TargetGraph);
	if(!bDisallowPaste)
	{
		// Find the Blueprint that owns the target graph
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
		if(Blueprint && Blueprint->SkeletonGeneratedClass)
		{
			TSet<FName> ExistingNamesInUse;
			TArray<FString> ExcludedEventNames;
			TArray<UK2Node_Event*> ExistingEventNodes;
			TArray<UClass*> ImplementedInterfaceClasses;

			// Gather all names in use by the Blueprint class
			FBlueprintEditorUtils::GetFunctionNameList(Blueprint, ExistingNamesInUse);
			FBlueprintEditorUtils::GetClassVariableList(Blueprint, ExistingNamesInUse);

			// Gather all existing event nodes
			FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_Event>(Blueprint, ExistingEventNodes);

			// Gather any event names excluded by the Blueprint class
			const FString ExclusionListKeyName = TEXT("KismetHideOverrides");
			if(Blueprint->ParentClass->HasMetaData(*ExclusionListKeyName))
			{
				const FString& ExcludedEventNameString = Blueprint->ParentClass->GetMetaData(*ExclusionListKeyName);
				ExcludedEventNameString.ParseIntoArray(ExcludedEventNames, TEXT(","), true);
			}

			// Gather all interfaces implemented by the Blueprint class
			FBlueprintEditorUtils::FindImplementedInterfaces(Blueprint, true, ImplementedInterfaceClasses);

			// If this is an internal event, don't paste this event
			if(!bInternalEvent)
			{
				// If this is a function override
				if(bOverrideFunction)
				{
					// If the function name is hidden by the parent class, don't paste this event
					bDisallowPaste = EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) == Blueprint->ParentClass
						&& ExcludedEventNames.Contains(EventReference.GetMemberName().ToString());
					if(!bDisallowPaste)
					{
						TArray<UK2Node_Event*> DisabledEventNodesToStomp;
						// If the event function is already handled in this Blueprint, don't paste this event
						for(int32 i = 0; i < ExistingEventNodes.Num() && !bDisallowPaste; ++i)
						{
							bDisallowPaste = ExistingEventNodes[i]->bOverrideFunction /*&& ExistingEventNodes[i]->IsNodeEnabled() */&& AreEventNodesIdentical(this, ExistingEventNodes[i]);

							if (bDisallowPaste && !ExistingEventNodes[i]->IsNodeEnabled())
							{
								DisabledEventNodesToStomp.Add(ExistingEventNodes[i]);
								bDisallowPaste = false;
							}
						}

						// We need to also check for 'const' BPIE methods that might already be implemented as functions with a read-only 'self' context (these were previously implemented as events)
						if(!bDisallowPaste)
						{
							TArray<UBlueprint*> ParentBPStack;
							UBlueprint::GetBlueprintHierarchyFromClass(Blueprint->SkeletonGeneratedClass, ParentBPStack);
							for(auto BPStackIt = ParentBPStack.CreateConstIterator(); BPStackIt && !bDisallowPaste; ++BPStackIt)
							{
								TArray<UK2Node_FunctionEntry*> ExistingFunctionEntryNodes;
								FBlueprintEditorUtils::GetAllNodesOfClass<UK2Node_FunctionEntry>(*BPStackIt, ExistingFunctionEntryNodes);
								for(auto NodeIt = ExistingFunctionEntryNodes.CreateConstIterator(); NodeIt && !bDisallowPaste; ++NodeIt)
								{
									UK2Node_FunctionEntry* ExistingFunctionEntryNode = *NodeIt;
									bDisallowPaste = ExistingFunctionEntryNode->bEnforceConstCorrectness
										&& ExistingFunctionEntryNode->FunctionReference.GetMemberName() == EventReference.GetMemberName();
								}
							}
						}

						if(!bDisallowPaste)
						{
							// If the signature class is not implemented by the Blueprint parent class or an interface, don't paste this event
							bDisallowPaste = !Blueprint->ParentClass->IsChildOf(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()))
								&& !ImplementedInterfaceClasses.Contains(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()));
							if(bDisallowPaste)
							{
								UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the event signature class (%ls) is incompatible with this Blueprint.", *GetFName().ToString(), EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) ? *EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->GetFName().ToString() : TEXT("NONE"));
							}
						}
						else
						{
							UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the event function (%ls) is already handled.", *GetFName().ToString(), *EventReference.GetMemberName().ToString());
						}

						if (!bDisallowPaste)
						{
							for (UK2Node_Event* EventNode : DisabledEventNodesToStomp)
							{
								EventNode->DestroyNode();
							}
						}
					}
					else
					{
						UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the event function (%ls) is hidden by the Blueprint parent class (%ls).", *GetFName().ToString(), *EventReference.GetMemberName().ToString(), EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) ? *EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->GetFName().ToString() : TEXT("NONE"));
					}
				}
				else if(CustomFunctionName != NAME_None)
				{
					// If this name is already in use, we can't paste this event
					bDisallowPaste = ExistingNamesInUse.Contains(CustomFunctionName);

					if(!bDisallowPaste)
					{
						// Handle events that have a custom function name with an actual signature name/class that is not an override (e.g. AnimNotify events)
						if(EventReference.GetMemberName() != NAME_None)
						{
							// If the signature class is not implemented by the Blueprint parent class or an interface, don't paste this event
							bDisallowPaste = !Blueprint->ParentClass->IsChildOf(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()))
								&& !ImplementedInterfaceClasses.Contains(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()));
							if(bDisallowPaste)
							{
								UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the custom event function (%ls) with event signature name (%ls) has an event signature class (%ls) that is incompatible with this Blueprint.", *GetFName().ToString(), *CustomFunctionName.ToString(), *EventReference.GetMemberName().ToString(), EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) ? *EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->GetFName().ToString() : TEXT("NONE"));
							}
						}
					}
					else
					{
						UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the custom event function (%ls) is already handled.", *GetFName().ToString(), *CustomFunctionName.ToString());
					}
				}
				else
				{
					UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because the event configuration is not specifically handled (EventSignatureName=%ls, EventSignatureClass=%ls).", *GetFName().ToString(), *EventReference.GetMemberName().ToString(), EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) ? *EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->GetFName().ToString() : TEXT("NONE"));
				}
			}
			else
			{
				UE_LOGF(LogBlueprint, Log, "Cannot paste event node (%ls) directly because it is flagged as an internal event.", *GetFName().ToString());
			}
		}
	}

	return !bDisallowPaste;
}

FText UK2Node_Event::GetLocalizedNetString(uint32 FunctionFlags, bool Calling)
{
	FText RPCString;
	if (FunctionFlags & FUNC_Net)
	{
		if (FunctionFlags & FUNC_NetMulticast)
		{
			if (Calling)
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedMulticast", "Replicated To All (if server)");
			}
			else
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedMulticastFrom", "Replicated From Server\nExecutes On All");
			}
		}
		else if (FunctionFlags & FUNC_NetServer)
		{
			if (Calling)
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedServer", "Replicated To Server (if owning client)");
			}
			else
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedServerFrom", "Replicated From Client\nExecutes On Server");
			}
		}
		else if (FunctionFlags & FUNC_NetClient)
		{
			if (Calling)
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedClient", "Replicated To Owning Client (if server)");
			}
			else
			{
				RPCString = NSLOCTEXT("K2Node", "CustomEvent_ReplicatedClientFrom", "Replicated From Server\nExecutes on Owning Client");
			}
		}

		if (FunctionFlags & FUNC_NetReliable)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("RPCString"), RPCString);
			RPCString = FText::Format(NSLOCTEXT("K2Node", "CustomEvent_ReplicatedReliable", "RELIABLE {RPCString}"), Args);
		}
	}
	return RPCString;
}

void UK2Node_Event::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* OrgDelegatePin = FindPin(UK2Node_Event::DelegateOutputName);
	if (OrgDelegatePin && OrgDelegatePin->LinkedTo.Num() > 0)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

		const FName FunctionName = GetFunctionName();
		if(FunctionName == NAME_None)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("EventDelegateName_Error", "Event node @@ has no name of function.").ToString(), this);
		}

		UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
		SelfNode->AllocateDefaultPins();

		UK2Node_CreateDelegate* CreateDelegateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CreateDelegate>(this, SourceGraph);
		CreateDelegateNode->AllocateDefaultPins();
		CompilerContext.MovePinLinksToIntermediate(*OrgDelegatePin, *CreateDelegateNode->GetDelegateOutPin());
		Schema->TryCreateConnection(SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self), CreateDelegateNode->GetObjectInPin());
		// When called UFunction is defined in the same class, it wasn't created yet (previously the Skeletal class was checked). So no "CreateDelegateNode->HandleAnyChangeWithoutNotifying();" is called.
		CreateDelegateNode->SetFunction(FunctionName);
	}
}

FName UK2Node_Event::GetCornerIcon() const
{
	if (UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		if (bOverrideFunction || (CustomFunctionName == NAME_None))
		{
			//@TODO: KISMETREPLICATION: Should do this for events with a custom function name, if it's a newly introduced replicating thingy
			if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic) || IsCosmeticTickEvent())
			{
				return TEXT("Graph.Replication.ClientEvent");
			} 
			else if(Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
			{
				return TEXT("Graph.Replication.AuthorityOnly");
			}
			else if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
			{
				return TEXT("Graph.Latent.LatentIcon");
			}
		}
	}

	if(IsUsedByAuthorityOnlyDelegate())
	{
		return TEXT("Graph.Replication.AuthorityOnly");
	}

	if(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) != nullptr && EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->IsChildOf(UInterface::StaticClass()))
	{
		return TEXT("Graph.Event.InterfaceEventIcon");
	}

	return Super::GetCornerIcon();
}

FText UK2Node_Event::GetToolTipHeading() const
{
	FText Heading = Super::GetToolTipHeading();

	FText EventHeading = FText::GetEmpty();
	if (UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		if (bOverrideFunction || (CustomFunctionName == NAME_None))
		{
			if (Function->HasAllFunctionFlags(FUNC_BlueprintCosmetic) || IsCosmeticTickEvent())
			{
				EventHeading = LOCTEXT("ClinetOnlyEvent", "Client Only");
			}
			else if(Function->HasAllFunctionFlags(FUNC_BlueprintAuthorityOnly))
			{
				EventHeading = LOCTEXT("ServerOnlyEvent", "Server Only");
			}
			else if (Function->HasMetaData(FBlueprintMetadata::MD_Latent))
			{
				EventHeading = LOCTEXT("LatentEvent", "Latent");
			}
		}
	}

	if (EventHeading.IsEmpty() && IsUsedByAuthorityOnlyDelegate())
	{
		EventHeading = LOCTEXT("ServerOnlyEvent", "Server Only");
	}
	else if(EventHeading.IsEmpty() && IsInterfaceEventNode())
	{
		EventHeading = LOCTEXT("InterfaceEvent", "Interface Event");
	}

	FText CompleteHeading = Super::GetToolTipHeading();
	if (!CompleteHeading.IsEmpty() && !EventHeading.IsEmpty())
	{
		CompleteHeading = FText::Format(FText::FromString("{0}\n{1}"), EventHeading, CompleteHeading);
	}
	else if (!EventHeading.IsEmpty())
	{
		CompleteHeading = EventHeading;
	}
	return CompleteHeading;
}

void UK2Node_Event::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "Event" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetFunctionName().ToString() ));
}

FText UK2Node_Event::GetMenuCategory() const
{
	FText FunctionCategory = LOCTEXT("AddEventCategory", "Add Event");
	if (UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		FunctionCategory = UK2Node_CallFunction::GetDefaultCategoryForFunction(Function, FunctionCategory);
	}
	return FunctionCategory;
}

bool UK2Node_Event::HasDeprecatedReference() const
{
	if (UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		return Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction);
	}

	return false;
}

FEdGraphNodeDeprecationResponse UK2Node_Event::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		// Only warn on override usage.
		if (bOverrideFunction)
		{
			UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
			if (ensureMsgf(Function != nullptr, TEXT("This node should not be able to report having a deprecated reference if the event override cannot be resolved.")))
			{
				// Check the deprecation type to override the severity
				FString MessageType = Function->GetMetaData(FBlueprintMetadata::MD_DeprecatedFunction);
				Response.MessageType = FBlueprintEditorUtils::GetDeprecatedMessageType(MessageType);

				FText EventName = FText::FromName(GetFunctionName());
				FText DetailedMessage = FText::FromString(Function->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage));
				Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(EventName, DetailedMessage);
			}
		}
		else
		{
			// Allow the source event to be marked as deprecated in the class that defines it without warning, but use a note to visually indicate that the definition itself has been deprecated.
			Response.MessageType = EEdGraphNodeDeprecationMessageType::Note;
			Response.MessageText = LOCTEXT("DeprecatedEventMessage", "@@: This event has been marked as deprecated. It can be safely deleted if all references have been replaced or removed.");
		}
	}

	return Response;
}

UObject* UK2Node_Event::GetJumpTargetForDoubleClick() const
{
	if(EventReference.GetMemberParentClass(GetBlueprintClassFromNode()) != NULL && EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->ClassGeneratedBy != NULL && EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->ClassGeneratedBy->IsA(UBlueprint::StaticClass()))
	{
		UBlueprint* Blueprint = CastChecked<UBlueprint>(EventReference.GetMemberParentClass(GetBlueprintClassFromNode())->ClassGeneratedBy);
		TArray<UEdGraph*> Graphs;
		Blueprint->GetAllGraphs(Graphs);
		for(auto It(Graphs.CreateConstIterator()); It; It++)
		{
			if((*It)->GetFName() == EventReference.GetMemberName())
			{
				return *It;
			}
		}
	}

	return nullptr;
}

FSlateIcon UK2Node_Event::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	return Icon;
}

FString UK2Node_Event::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	// If searching by class member, try to construct search term from the UFunction.
	// This may fail if the function was not found or for whatever reason, its owning 
	// class is invalid. If it fails, proceed to search by name as fallback behavior.
	if (EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::UseSearchSyntax) && !EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::Legacy))
	{
		// Resolve the function
		if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
		{
			FString SearchTerm;
			if (FindInBlueprintsHelpers::ConstructSearchTermFromFunction(Function, SearchTerm))
			{
				return SearchTerm;
			}
		}
	}

	// Searching by just name. When overriding a function, try to find the function that it overrides to return its name.
	if (bOverrideFunction || (CustomFunctionName == NAME_None))
	{
		// Attempt to find the function
		if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
		{
			// Search by native name
			const FString FunctionNativeName = Function->GetName();
			return FString::Printf(TEXT("\"%s\""), *FunctionNativeName);
		}
		else
		{
			// If we fail to find the function, still want to search for its expected name, in quotes
			return FString::Printf(TEXT("\"%s\""), *EventReference.GetMemberName().ToString());
		}
	}
	else
	{
		// The function was not an override; its name is defined by this node.
		// Return that name in quotes (treating special characters as part of name)
		return FString::Printf(TEXT("\"%s\""), *CustomFunctionName.ToString());
	}
}

void UK2Node_Event::FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);
	UK2Node_Event* OtherFunction = Cast<UK2Node_Event>(OtherNode);

	if (OtherFunction)
	{
		if (FunctionFlags != OtherFunction->FunctionFlags)
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.DisplayString = LOCTEXT("DIF_EventFlags", "Event flags have changed");
			Diff.Category = EDiffType::MODIFICATION;

			Results.Add(Diff);
		}
	}
}

UEdGraphPin* UK2Node_Event::GetDelegatePin() const
{
	return FindPinChecked(DelegateOutputName);
}

bool UK2Node_Event::AreEventNodesIdentical(const UK2Node_Event* InNodeA, const UK2Node_Event* InNodeB)
{
	return InNodeA->EventReference.GetMemberName() == InNodeB->EventReference.GetMemberName()
		&& InNodeA->EventReference.GetMemberParentClass(InNodeA->GetBlueprintClassFromNode()) == InNodeB->EventReference.GetMemberParentClass(InNodeB->GetBlueprintClassFromNode());
}

bool UK2Node_Event::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	UFunction* Function = EventReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
	const UClass* SourceClass = Function ? Function->GetOwnerClass() : nullptr;
	const bool bResult = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(Function);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_Event::AddSearchMetaDataInfo(TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
	{
		// Index the native name of the function, this will be used in search queries rather than node title
		const FString FunctionNativeName = Function->GetName();
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(FunctionNativeName)));

		// Index the (ancestor) class or interface from which the function originates, can be self
		if (const UClass* FuncOriginClass = FindInBlueprintsHelpers::GetFunctionOriginClass(Function))
		{
			const FString FuncOriginClassName = FuncOriginClass->GetPathName();
			OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_FuncOriginClass, FText::FromString(FuncOriginClassName)));
		}
	}
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_Event::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2Event> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2Event(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

#undef LOCTEXT_NAMESPACE
