// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ExecutionSequence.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ExecutionSequence)

#define LOCTEXT_NAMESPACE "K2Node_MultiGate"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_ExecutionSequence

class FKCHandler_ExecutionSequence : public FNodeHandlingFunctor
{
public:
	FKCHandler_ExecutionSequence(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		// Make sure that the input pin is connected and valid for this block
		FEdGraphPinType ExpectedPinType;
		ExpectedPinType.PinCategory = UEdGraphSchema_K2::PC_Exec;

		UEdGraphPin* ExecTriggeringPin = Context.FindRequiredPinByName(Node, UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		if ((ExecTriggeringPin == nullptr) || !Context.ValidatePinType(ExecTriggeringPin, ExpectedPinType))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoValidExecutionPinForExecSeq_Error", "@@ must have a valid execution pin @@").ToString(), Node, ExecTriggeringPin);
			return;
		}
		else if (ExecTriggeringPin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Warning(*LOCTEXT("NodeNeverExecuted_Warning", "@@ will never be executed").ToString(), Node);
			return;
		}

		// Find the valid, connected output pins, and add them to the processing list
		TArray<UEdGraphPin*> OutputPins;
		for (UEdGraphPin* CurrentPin : Node->Pins)
		{
			if ((CurrentPin->Direction == EGPD_Output) && (CurrentPin->LinkedTo.Num() > 0) && (CurrentPin->PinName.ToString().StartsWith(UEdGraphSchema_K2::PN_Then.ToString())))
			{
				OutputPins.Add(CurrentPin);
			}
		}

		//@TODO: Sort the pins by the number appended to the pin!

		// Process the pins, if there are any valid entries
		if (OutputPins.Num() > 0)
		{
			if (Context.IsDebuggingOrInstrumentationRequired() && (OutputPins.Num() > 1))
			{
				const FString NodeComment = Node->NodeComment.IsEmpty() ? Node->GetName() : Node->NodeComment;

				// Assuming sequence X goes to A, B, C, we want to emit:
				//   X: push X1
				//      goto A
				//  X1: debug site
				//      push X2
				//      goto B
				//  X2: debug site
				//      goto C

				// A push statement we need to patch up on the next pass (e.g., push X1 before we know where X1 is)
				FBlueprintCompiledStatement* LastPushStatement = NULL;

				for (int32 i = 0; i < OutputPins.Num(); ++i)
				{
					// Emit the debug site and patch up the previous jump if we're on subsequent steps
					const bool bNotFirstIndex = i > 0;
					if (bNotFirstIndex)
					{
						// Emit a debug site
						FBlueprintCompiledStatement& DebugSiteAndJumpTarget = Context.AppendStatementForNode(Node);
						DebugSiteAndJumpTarget.Type = Context.GetBreakpointType();
						DebugSiteAndJumpTarget.Comment = NodeComment;
						DebugSiteAndJumpTarget.bIsJumpTarget = true;

						// Patch up the previous push jump target
						check(LastPushStatement);
						LastPushStatement->TargetLabel = &DebugSiteAndJumpTarget;
					}

					// Emit a push to get to the next step in the sequence, unless we're the last one or this is an instrumented build
					const bool bNotLastIndex = ((i + 1) < OutputPins.Num());
					if (bNotLastIndex)
					{
						FBlueprintCompiledStatement& PushExecutionState = Context.AppendStatementForNode(Node);
						PushExecutionState.Type = KCST_PushState;
						LastPushStatement = &PushExecutionState;
					}

					// Emit the goto to the actual state
					FBlueprintCompiledStatement& GotoSequenceLinkedState = Context.AppendStatementForNode(Node);
					GotoSequenceLinkedState.Type = KCST_UnconditionalGoto;
					Context.GotoFixupRequestMap.Add(&GotoSequenceLinkedState, OutputPins[i]);
				}

				check(LastPushStatement);
			}
			else
			{
				// Directly emit pushes to execute the remaining branches
				for (int32 i = OutputPins.Num() - 1; i > 0; i--)
				{
					FBlueprintCompiledStatement& PushExecutionState = Context.AppendStatementForNode(Node);
					PushExecutionState.Type = KCST_PushState;
					Context.GotoFixupRequestMap.Add(&PushExecutionState, OutputPins[i]);
				}

				// Immediately jump to the first pin
				FBlueprintCompiledStatement& NextExecutionState = Context.AppendStatementForNode(Node);
				NextExecutionState.Type = KCST_UnconditionalGoto;
				Context.GotoFixupRequestMap.Add(&NextExecutionState, OutputPins[0]);
			}
		}
		else
		{
			FBlueprintCompiledStatement& NextExecutionState = Context.AppendStatementForNode(Node);
			NextExecutionState.Type = KCST_EndOfThread;
		}
	}
};

UK2Node_ExecutionSequence::UK2Node_ExecutionSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_ExecutionSequence::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// Add two default pins
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, GetPinNameGivenIndex(0));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, GetPinNameGivenIndex(1));

	Super::AllocateDefaultPins();
}

FText UK2Node_ExecutionSequence::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "Sequence", "Sequence");
}

FSlateIcon UK2Node_ExecutionSequence::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Sequence_16x");
	return Icon;
}

FLinearColor UK2Node_ExecutionSequence::GetNodeTitleColor() const
{
	return FLinearColor::White;
}

FText UK2Node_ExecutionSequence::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "ExecutePinInOrder_Tooltip", "Executes a series of pins in order");
}

FName UK2Node_ExecutionSequence::GetUniquePinName()
{
	FName NewPinName;
	int32 i = 0;
	while (true)
	{
		NewPinName = GetPinNameGivenIndex(i++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}

	return NewPinName;
}

void UK2Node_ExecutionSequence::AddInputPin()
{
	Modify();
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, GetUniquePinName());
}

void UK2Node_ExecutionSequence::InsertPinIntoExecutionNode(UEdGraphPin* PinToInsertBefore, EPinInsertPosition Position)
{
	Modify();

	int32 DesiredPinIndex = Pins.Find(PinToInsertBefore);
	if (DesiredPinIndex != INDEX_NONE)
	{
		if (Position == EPinInsertPosition::After)
		{
			DesiredPinIndex = DesiredPinIndex + 1;
		}

		FCreatePinParams Params;
		Params.Index = DesiredPinIndex;
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, GetUniquePinName(), Params);

		// refresh names on the pin list:
		int32 ThenIndex = 0;
		for (int32 Idx = 0; Idx < Pins.Num(); ++Idx)
		{
			UEdGraphPin* PotentialPin = Pins[Idx];
			if (UEdGraphSchema_K2::IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output))
			{
				PotentialPin->PinName = GetPinNameGivenIndex(ThenIndex);
				++ThenIndex;
			}
		}
	}
}

void UK2Node_ExecutionSequence::RemovePinFromExecutionNode(UEdGraphPin* TargetPin) 
{
	UK2Node_ExecutionSequence* OwningSeq = Cast<UK2Node_ExecutionSequence>( TargetPin->GetOwningNode() );
	if (OwningSeq)
	{
		OwningSeq->Pins.Remove(TargetPin);
		TargetPin->MarkAsGarbage();

		// Renumber the pins so the numbering is compact
		int32 ThenIndex = 0;
		for (int32 i = 0; i < OwningSeq->Pins.Num(); ++i)
		{
			UEdGraphPin* PotentialPin = OwningSeq->Pins[i];
			if (UEdGraphSchema_K2::IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output))
			{
				PotentialPin->PinName = GetPinNameGivenIndex(ThenIndex);
				++ThenIndex;
			}
		}
	}
}

bool UK2Node_ExecutionSequence::CanRemoveExecutionPin() const
{
	int32 NumOutPins = 0;

	for (int32 i = 0; i < Pins.Num(); ++i)
	{
		UEdGraphPin* PotentialPin = Pins[i];
		if (UEdGraphSchema_K2::IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output))
		{
			NumOutPins++;
		}
	}

	return (NumOutPins > 2);
}

FName UK2Node_ExecutionSequence::GetPinNameGivenIndex(int32 Index) const
{
	return *FString::Printf(TEXT("%s_%d"), *UEdGraphSchema_K2::PN_Then.ToString(), Index);
}

void UK2Node_ExecutionSequence::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::AllocateDefaultPins();

	// Create the execution input pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// Create a new pin for each old execution output pin, and coerce the names to match on both sides
	int32 ExecOutPinCount = 0;
	for (int32 i = 0; i < OldPins.Num(); ++i)
	{
		UEdGraphPin* TestPin = OldPins[i];
		if (UEdGraphSchema_K2::IsExecPin(*TestPin) && (TestPin->Direction == EGPD_Output))
		{
			const FName NewPinName(GetPinNameGivenIndex(ExecOutPinCount));
			ExecOutPinCount++;

			// Make sure the old pin and new pin names match
			TestPin->PinName = NewPinName;

			// Create the new output pin to match
			CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, NewPinName);
		}
	}
}

UEdGraphPin* UK2Node_ExecutionSequence::GetThenPinGivenIndex(const int32 Index) 
{
	return FindPin(GetPinNameGivenIndex(Index));
}

FNodeHandlingFunctor* UK2Node_ExecutionSequence::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_ExecutionSequence(CompilerContext);
}

void UK2Node_ExecutionSequence::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_ExecutionSequence::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::FlowControl);
}

#undef LOCTEXT_NAMESPACE
