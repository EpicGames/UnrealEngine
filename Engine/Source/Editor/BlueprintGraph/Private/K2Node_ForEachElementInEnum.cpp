// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_ForEachElementInEnum.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFieldNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CastByteToEnum.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_GetNumEnumEntries.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_SwitchEnum.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetNodeHelperLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_ForEachElementInEnum)

class UBlueprintNodeSpawner;
struct FLinearColor;

#define LOCTEXT_NAMESPACE "K2Node"

struct FForExpandNodeHelper
{
	UEdGraphPin* StartLoopExecInPin;
	UEdGraphPin* InsideLoopExecOutPin;
	UEdGraphPin* LoopCompleteOutExecPin;

	UEdGraphPin* ArrayIndexOutPin;
	UEdGraphPin* LoopCounterOutPin;
	// for(LoopCounter = 0; LoopCounter < LoopCounterLimit; ++LoopCounter)
	UEdGraphPin* LoopCounterLimitInPin;

	FForExpandNodeHelper()
		: StartLoopExecInPin(nullptr)
		, InsideLoopExecOutPin(nullptr)
		, LoopCompleteOutExecPin(nullptr)
		, ArrayIndexOutPin(nullptr)
		, LoopCounterOutPin(nullptr)
		, LoopCounterLimitInPin(nullptr)
	{ }

	bool BuildLoop(UK2Node* Node, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEnum* Enum)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
		check(Node && SourceGraph && Schema);

		bool bResult = true;

		// Create int Loop Counter
		UK2Node_TemporaryVariable* LoopCounterNode = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(Node, SourceGraph);
		LoopCounterNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
		LoopCounterNode->AllocateDefaultPins();
		LoopCounterOutPin = LoopCounterNode->GetVariablePin();
		check(LoopCounterOutPin);

		// Initialize loop counter
		UK2Node_AssignmentStatement* LoopCounterInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
		LoopCounterInitialize->AllocateDefaultPins();
		LoopCounterInitialize->GetValuePin()->DefaultValue = TEXT("0");
		bResult &= Schema->TryCreateConnection(LoopCounterOutPin, LoopCounterInitialize->GetVariablePin());
		StartLoopExecInPin = LoopCounterInitialize->GetExecPin();
		check(StartLoopExecInPin);

		// Create int Array Index
		UK2Node_TemporaryVariable* ArrayIndexNode = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(Node, SourceGraph);
		ArrayIndexNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
		ArrayIndexNode->AllocateDefaultPins();
		ArrayIndexOutPin = ArrayIndexNode->GetVariablePin();
		check(ArrayIndexOutPin);

		// Initialize array index
		UK2Node_AssignmentStatement* ArrayIndexInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
		ArrayIndexInitialize->AllocateDefaultPins();
		ArrayIndexInitialize->GetValuePin()->DefaultValue = TEXT("0");
		bResult &= Schema->TryCreateConnection(ArrayIndexOutPin, ArrayIndexInitialize->GetVariablePin());
		bResult &= Schema->TryCreateConnection(LoopCounterInitialize->GetThenPin(), ArrayIndexInitialize->GetExecPin());

		// Do loop branch
		UK2Node_IfThenElse* Branch = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(Node, SourceGraph);
		Branch->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(ArrayIndexInitialize->GetThenPin(), Branch->GetExecPin());
		LoopCompleteOutExecPin = Branch->GetElsePin();
		check(LoopCompleteOutExecPin);

		// Do loop condition
		UK2Node_CallFunction* Condition = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Node, SourceGraph); 
		Condition->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Less_IntInt)));
		Condition->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(Condition->GetReturnValuePin(), Branch->GetConditionPin());
		bResult &= Schema->TryCreateConnection(Condition->FindPinChecked(TEXT("A")), LoopCounterOutPin);
		LoopCounterLimitInPin = Condition->FindPinChecked(TEXT("B"));
		check(LoopCounterLimitInPin);

		// Convert the Enum index to a value
		UK2Node_CallFunction* GetEnumeratorValueFromIndexCall = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Node, SourceGraph); 
		GetEnumeratorValueFromIndexCall->SetFromFunction(UKismetNodeHelperLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, GetEnumeratorValueFromIndex)));
		GetEnumeratorValueFromIndexCall->AllocateDefaultPins();
		Schema->TrySetDefaultObject(*GetEnumeratorValueFromIndexCall->FindPinChecked(TEXT("Enum")), Enum);
		bResult &= Schema->TryCreateConnection(GetEnumeratorValueFromIndexCall->FindPinChecked(TEXT("EnumeratorIndex")), LoopCounterOutPin);

		// Array Index assigned
		UK2Node_AssignmentStatement* ArrayIndexAssign = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
		ArrayIndexAssign->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(Branch->GetThenPin(), ArrayIndexAssign->GetExecPin());
		bResult &= Schema->TryCreateConnection(ArrayIndexAssign->GetVariablePin(), ArrayIndexOutPin);
		bResult &= Schema->TryCreateConnection(ArrayIndexAssign->GetValuePin(), GetEnumeratorValueFromIndexCall->GetReturnValuePin());

		// body sequence
		UK2Node_ExecutionSequence* Sequence = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(Node, SourceGraph);
		Sequence->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(ArrayIndexAssign->GetThenPin(), Sequence->GetExecPin());
		InsideLoopExecOutPin = Sequence->GetThenPinGivenIndex(0);
		check(InsideLoopExecOutPin);

		// Loop Counter increment
		UK2Node_CallFunction* Increment = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Node, SourceGraph); 
		Increment->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Add_IntInt)));
		Increment->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(Increment->FindPinChecked(TEXT("A")), LoopCounterOutPin);
		Increment->FindPinChecked(TEXT("B"))->DefaultValue = TEXT("1");

		// Loop Counter assigned
		UK2Node_AssignmentStatement* LoopCounterAssign = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(Node, SourceGraph);
		LoopCounterAssign->AllocateDefaultPins();
		bResult &= Schema->TryCreateConnection(LoopCounterAssign->GetExecPin(), Sequence->GetThenPinGivenIndex(1));
		bResult &= Schema->TryCreateConnection(LoopCounterAssign->GetVariablePin(), LoopCounterOutPin);
		bResult &= Schema->TryCreateConnection(LoopCounterAssign->GetValuePin(), Increment->GetReturnValuePin());
		bResult &= Schema->TryCreateConnection(LoopCounterAssign->GetThenPin(), Branch->GetExecPin());

		return bResult;
	}
};

UK2Node_ForEachElementInEnum::UK2Node_ForEachElementInEnum(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const FName UK2Node_ForEachElementInEnum::InsideLoopPinName(TEXT("LoopBody"));
const FName UK2Node_ForEachElementInEnum::EnumOuputPinName(TEXT("EnumValue"));
const FName UK2Node_ForEachElementInEnum::SkipHiddenPinName(TEXT("SkipHidden"));

void UK2Node_ForEachElementInEnum::AllocateDefaultPins()
{
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	if (Enum)
	{
		if (UEdGraphPin* SkipHiddenPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, SkipHiddenPinName))
		{
			// This is a non-standard option that likely won't need to be utilized much, so we make it advanced.
			SkipHiddenPin->bAdvancedView = true;
			AdvancedPinDisplay = ENodeAdvancedPins::Hidden;

			K2Schema->ConstructBasicPinTooltip(*SkipHiddenPin, LOCTEXT("SkipHiddenPinToolTip", "Controls whether or not the loop will skip over hidden enumeration values."), SkipHiddenPin->PinToolTip);
		}

		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, InsideLoopPinName);
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Byte, Enum, EnumOuputPinName);
	}

	if (UEdGraphPin* CompletedPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then))
	{
		CompletedPin->PinFriendlyName = LOCTEXT("Completed", "Completed");
	}
}

void UK2Node_ForEachElementInEnum::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!Enum)
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "ForEachElementInEnum_NoEnumError", "No Enum in @@").ToString(), this);
	}
}

FText UK2Node_ForEachElementInEnum::GetTooltipText() const
{
	return GetNodeTitle(ENodeTitleType::FullTitle);
}

FText UK2Node_ForEachElementInEnum::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Enum == nullptr)
	{
		return LOCTEXT("ForEachElementInUnknownEnum_Title", "ForEach UNKNOWN");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumName"), FText::FromName(Enum->GetFName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("ForEachElementInEnum_Title", "ForEach {EnumName}"), Args), this);
	}
	return CachedNodeTitle;
}

FSlateIcon UK2Node_ForEachElementInEnum::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Macro.Loop_16x");
	return Icon;
}

void UK2Node_ForEachElementInEnum::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (!Enum)
	{
		ValidateNodeDuringCompilation(CompilerContext.MessageLog);
		return;
	}

	FForExpandNodeHelper ForLoop;
	if (!ForLoop.BuildLoop(this, CompilerContext, SourceGraph, Enum))
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "ForEachElementInEnum_ForError", "For Expand error in @@").ToString(), this);
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UK2Node_GetNumEnumEntries* GetNumEnumEntries = CompilerContext.SpawnIntermediateNode<UK2Node_GetNumEnumEntries>(this, SourceGraph);
	GetNumEnumEntries->Enum = Enum;
	GetNumEnumEntries->AllocateDefaultPins();
	bool bResult = Schema->TryCreateConnection(GetNumEnumEntries->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), ForLoop.LoopCounterLimitInPin);

	UK2Node_CallFunction* Conv_Func = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	FName Conv_Func_Name = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Conv_IntToByte);
	Conv_Func->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(Conv_Func_Name));
	Conv_Func->AllocateDefaultPins();
	bResult &= Schema->TryCreateConnection(Conv_Func->FindPinChecked(TEXT("InInt")), ForLoop.ArrayIndexOutPin);

	UK2Node_CastByteToEnum* CastByteToEnum = CompilerContext.SpawnIntermediateNode<UK2Node_CastByteToEnum>(this, SourceGraph);
	CastByteToEnum->Enum = Enum;
	CastByteToEnum->bSafe = true;
	CastByteToEnum->AllocateDefaultPins();
	bResult &= Schema->TryCreateConnection(Conv_Func->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), CastByteToEnum->FindPinChecked(UK2Node_CastByteToEnum::ByteInputPinName));

	// Additional expansion logic to optionally exclude hidden values during runtime loop iteration
	UK2Node_ExecutionSequence* SwitchOutputSequence = nullptr;
	if (const UEdGraphPin* SkipHiddenValuesPin = FindPin(SkipHiddenPinName))
	{
		// Process only if the enum type contains at least one hidden value
		int32 EnumIndex = 0;
		bool bHasHiddenValues = false;
		while (!bHasHiddenValues && EnumIndex < Enum->NumEnums() - 1)
		{
			bHasHiddenValues = Enum->HasMetaData(TEXT("Hidden"), EnumIndex) || Enum->HasMetaData(TEXT("Spacer"), EnumIndex++);
		}

		if (bHasHiddenValues)
		{
			// Skip hidden values branch (only included if something is linked to the "skip hidden" input pin)
			UK2Node_IfThenElse* ShouldSkipHiddenBranch = nullptr;
			if (SkipHiddenValuesPin->LinkedTo.Num() > 0)
			{
				bResult &= ensure(SkipHiddenValuesPin->LinkedTo.Num() == 1);
				ShouldSkipHiddenBranch = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
				ShouldSkipHiddenBranch->AllocateDefaultPins();
				bResult &= Schema->TryCreateConnection(ForLoop.InsideLoopExecOutPin, ShouldSkipHiddenBranch->GetExecPin());
				bResult &= Schema->TryCreateConnection(SkipHiddenValuesPin->LinkedTo[0], ShouldSkipHiddenBranch->GetConditionPin());
			}

			// Enum switch node (only if we included a "should skip" test or if the "skip hidden" input pin default value is 'true')
			if (ShouldSkipHiddenBranch || SkipHiddenValuesPin->GetDefaultAsString().Equals(TEXT("true"), ESearchCase::IgnoreCase))
			{
				// The switch node will internally exclude any hidden enum values when constructed
				UK2Node_SwitchEnum* SwitchEnum = CompilerContext.SpawnIntermediateNode<UK2Node_SwitchEnum>(this, SourceGraph);
				SwitchEnum->SetEnum(Enum);
				SwitchEnum->bHasDefaultPin = false;
				SwitchEnum->AllocateDefaultPins();
				bResult &= Schema->TryCreateConnection(SwitchEnum->GetSelectionPin(), CastByteToEnum->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue));
				bResult &= Schema->TryCreateConnection(SwitchEnum->GetExecPin(), ShouldSkipHiddenBranch ? ShouldSkipHiddenBranch->GetThenPin() : ForLoop.InsideLoopExecOutPin);

				// Switch output execution sequence (direct all relevant output pins back to a single execution path)
				SwitchOutputSequence = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
				SwitchOutputSequence->AllocateDefaultPins();
				if (ShouldSkipHiddenBranch)
				{
					bResult &= Schema->TryCreateConnection(ShouldSkipHiddenBranch->GetElsePin(), SwitchOutputSequence->GetExecPin());
				}
				for (int32 SwitchCasePinIndex = 0; SwitchCasePinIndex < SwitchEnum->EnumEntries.Num() && bResult; ++SwitchCasePinIndex)
				{
					if (UEdGraphPin* SwitchCasePin = SwitchEnum->FindPin(SwitchEnum->EnumEntries[SwitchCasePinIndex]))
					{
						bResult &= Schema->TryCreateConnection(SwitchCasePin, SwitchOutputSequence->GetExecPin());
					}
				}
			}
		}
	}

	CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *ForLoop.StartLoopExecInPin);
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Then), *ForLoop.LoopCompleteOutExecPin);
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(InsideLoopPinName), SwitchOutputSequence ? *SwitchOutputSequence->GetThenPinGivenIndex(0) : *ForLoop.InsideLoopExecOutPin);
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(EnumOuputPinName), *CastByteToEnum->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue));

	if (!bResult)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "ForEachElementInEnum_ExpandError", "Expand error in @@").ToString(), this);
	}

	BreakAllNodeLinks();
}

void UK2Node_ForEachElementInEnum::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, FFieldVariant /*EnumField*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UK2Node_ForEachElementInEnum* EnumNode = CastChecked<UK2Node_ForEachElementInEnum>(NewNode);
			EnumNode->Enum = NonConstEnumPtr.Get();
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* InEnum)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UEnum*>(InEnum));
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(InEnum));
		NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

FText UK2Node_ForEachElementInEnum::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Enum);
}

void UK2Node_ForEachElementInEnum::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// Skip hidden enumeration values by default for new node placements.
	if (UEdGraphPin* SkipHiddenPin = FindPin(SkipHiddenPinName))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->SetPinAutogeneratedDefaultValue(SkipHiddenPin, TEXT("true"));
	}
}

void UK2Node_ForEachElementInEnum::ReloadEnum(class UEnum* InEnum)
{
	Enum = InEnum;
	CachedNodeTitle.MarkDirty();
}

#undef LOCTEXT_NAMESPACE
