// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_FormatText.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MakeArray.h"
#include "K2Node_MakeStruct.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet/KismetTextLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_FormatText)

#define LOCTEXT_NAMESPACE "K2Node_FormatText"

/////////////////////////////////////////////////////
// UK2Node_FormatText

struct FFormatTextNodeHelper
{
	static const FName FormatPinName;

	static const FName GetFormatPinName()
	{
		return FormatPinName;
	}
};

const FName FFormatTextNodeHelper::FormatPinName(TEXT("Format"));

UK2Node_FormatText::UK2Node_FormatText(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CachedFormatPin(NULL)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Builds a formatted string using available format argument values.\n  \u2022 Use {} to denote format arguments.\n  \u2022 Argument types may be Byte, Integer, Float, Text, String, Name, Boolean, Object or ETextGender.");
}

void UK2Node_FormatText::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CachedFormatPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Text, FFormatTextNodeHelper::GetFormatPinName());
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Text, TEXT("Result"));

	for (const FName& PinName : PinNames)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, PinName);
	}
}

void UK2Node_FormatText::SynchronizeArgumentPinType(UEdGraphPin* Pin)
{
	const UEdGraphPin* FormatPin = GetFormatPin();
	if (Pin != FormatPin && Pin->Direction == EGPD_Input)
	{
		const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());

		bool bPinTypeChanged = false;
		if (Pin->LinkedTo.Num() == 0)
		{
			static const FEdGraphPinType WildcardPinType = FEdGraphPinType(UEdGraphSchema_K2::PC_Wildcard, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());

			// Ensure wildcard
			if (Pin->PinType != WildcardPinType)
			{
				Pin->PinType = WildcardPinType;
				bPinTypeChanged = true;
			}
		}
		else
		{
			UEdGraphPin* ArgumentSourcePin = Pin->LinkedTo[0];

			// Take the type of the connected pin
			if (Pin->PinType != ArgumentSourcePin->PinType)
			{
				Pin->PinType = ArgumentSourcePin->PinType;
				bPinTypeChanged = true;
			}
		}

		if (bPinTypeChanged)
		{
			// Let the graph know to refresh
			GetGraph()->NotifyNodeChanged(this);

			UBlueprint* Blueprint = GetBlueprint();
			if (!Blueprint->bBeingCompiled)
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
			}
		}
	}
}

FText UK2Node_FormatText::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("FormatText_Title", "Format Text");
}

FText UK2Node_FormatText::GetPinDisplayName(const UEdGraphPin* Pin) const
{
	return FText::FromName(Pin->PinName);
}

FName UK2Node_FormatText::GetUniquePinName()
{
	FName NewPinName;
	int32 i = 0;
	while (true)
	{
		NewPinName = *FString::FromInt(i++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}
	return NewPinName;
}

void UK2Node_FormatText::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property  ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_FormatText, PinNames))
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_FormatText::PinConnectionListChanged(UEdGraphPin* Pin)
{
	UEdGraphPin* FormatPin = GetFormatPin();

	Modify();

	// Clear all pins.
	if(Pin == FormatPin && !FormatPin->DefaultTextValue.IsEmpty())
	{
		PinNames.Empty();
		GetSchema()->TrySetDefaultText(*FormatPin, FText::GetEmpty());

		bool bRemoved = false;
		for(auto It = Pins.CreateConstIterator(); It; ++It)
		{
			UEdGraphPin* CheckPin = *It;
			if(CheckPin != FormatPin && CheckPin->Direction == EGPD_Input)
			{
				CheckPin->Modify();
				CheckPin->MarkAsGarbage();
				Pins.Remove(CheckPin);
				bRemoved = true;
				--It;
			}
		}

		if (bRemoved)
		{
			GetGraph()->NotifyNodeChanged(this);
		}

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}

	// Potentially update an argument pin type
	SynchronizeArgumentPinType(Pin);
}

void UK2Node_FormatText::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	const UEdGraphPin* FormatPin = GetFormatPin();
	if(Pin == FormatPin && FormatPin->LinkedTo.Num() == 0)
	{
		TArray< FString > ArgumentParams;
		FText::GetFormatPatternParameters(FormatPin->DefaultTextValue, ArgumentParams);

		PinNames.Reset();

		for (const FString& Param : ArgumentParams)
		{
			const FName ParamName(*Param);
			if (!FindArgumentPin(ParamName))
			{
				CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, ParamName);
			}
			PinNames.Add(ParamName);
		}

		for (auto It = Pins.CreateIterator(); It; ++It)
		{
			UEdGraphPin* CheckPin = *It;
			if (CheckPin != FormatPin && CheckPin->Direction == EGPD_Input)
			{
				const bool bIsValidArgPin = ArgumentParams.ContainsByPredicate([&CheckPin](const FString& InPinName)
				{
					return InPinName.Equals(CheckPin->PinName.ToString(), ESearchCase::CaseSensitive);
				});

				if(!bIsValidArgPin)
				{
					CheckPin->MarkAsGarbage();
					It.RemoveCurrent();
				}
			}
		}

		GetGraph()->NotifyNodeChanged(this);
	}
}

void UK2Node_FormatText::PinTypeChanged(UEdGraphPin* Pin)
{
	// Potentially update an argument pin type
	SynchronizeArgumentPinType(Pin);

	Super::PinTypeChanged(Pin);
}

FText UK2Node_FormatText::GetTooltipText() const
{
	return NodeTooltip;
}

UEdGraphPin* FindOutputStructPinChecked(UEdGraphNode* Node)
{
	check(NULL != Node);
	UEdGraphPin* OutputPin = NULL;
	for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Node->Pins[PinIndex];
		if (Pin && (EGPD_Output == Pin->Direction))
		{
			OutputPin = Pin;
			break;
		}
	}
	check(NULL != OutputPin);
	return OutputPin;
}

void UK2Node_FormatText::PostReconstructNode()
{
	Super::PostReconstructNode();

	// We need to upgrade any non-connected argument pins with valid literal text data to use a "Make Literal Text" node as an input (argument pins used to be PC_Text and they're now PC_Wildcard)
	if (!IsTemplate())
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if (OuterGraph && OuterGraph->Schema)
		{
			int32 NumPinsFixedUp = 0;

			const UEdGraphPin* FormatPin = GetFormatPin();
			for (UEdGraphPin* CurrentPin : Pins)
			{
				if (CurrentPin != FormatPin && CurrentPin->Direction == EGPD_Input && CurrentPin->LinkedTo.Num() == 0 && !CurrentPin->DefaultTextValue.IsEmpty())
				{
					// Create a new "Make Literal Text" function and add it to the graph
					const FVector2D SpawnLocation = FVector2D(NodePosX - 300, NodePosY + (60 * (NumPinsFixedUp + 1)));
					UK2Node_CallFunction* MakeLiteralText = FEdGraphSchemaAction_K2NewNode::SpawnNode<UK2Node_CallFunction>(
						GetGraph(),
						SpawnLocation,
						EK2NewNodeFlags::None,
						[](UK2Node_CallFunction* NewInstance)
						{
							NewInstance->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralText)));
						}
					);

					// Set the new value and clear it on this pin to avoid it ever attempting this upgrade again (eg, if the "Make Literal Text" node was disconnected)
					UEdGraphPin* LiteralValuePin = MakeLiteralText->FindPinChecked(TEXT("Value"));
					LiteralValuePin->DefaultTextValue = CurrentPin->DefaultTextValue; // Note: Uses assignment rather than TrySetDefaultText to ensure we keep the existing localization identity
					CurrentPin->DefaultTextValue = FText::GetEmpty();

					// Connect the new node to the existing pin
					UEdGraphPin* LiteralReturnValuePin = MakeLiteralText->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
					GetSchema()->TryCreateConnection(LiteralReturnValuePin, CurrentPin);

					++NumPinsFixedUp;
				}

				// Potentially update an argument pin type
				SynchronizeArgumentPinType(CurrentPin);
			}

			if (NumPinsFixedUp > 0)
			{
				GetGraph()->NotifyNodeChanged(this);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
			}
		}
	}
}

void UK2Node_FormatText::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	/**
		At the end of this, the UK2Node_FormatText will not be a part of the Blueprint, it merely handles connecting
		the other nodes into the Blueprint.
	*/

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// Create a "Make Array" node to compile the list of arguments into an array for the Format function being called
	UK2Node_MakeArray* MakeArrayNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeArray>(this, SourceGraph);
	MakeArrayNode->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeArrayNode, this);

	UEdGraphPin* ArrayOut = MakeArrayNode->GetOutputPin();

	// This is the node that does all the Format work.
	UK2Node_CallFunction* CallFormatFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallFormatFunction->SetFromFunction(UKismetTextLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Format)));
	CallFormatFunction->AllocateDefaultPins();
	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFormatFunction, this);

	// Connect the output of the "Make Array" pin to the function's "InArgs" pin
	ArrayOut->MakeLinkTo(CallFormatFunction->FindPinChecked(TEXT("InArgs")));

	// This will set the "Make Array" node's type, only works if one pin is connected.
	MakeArrayNode->PinConnectionListChanged(ArrayOut);

	// For each argument, we will need to add in a "Make Struct" node.
	for(int32 ArgIdx = 0; ArgIdx < PinNames.Num(); ++ArgIdx)
	{
		UEdGraphPin* ArgumentPin = FindArgumentPin(PinNames[ArgIdx]);

		static UScriptStruct* FormatArgumentDataStruct = FindObjectChecked<UScriptStruct>(FindObjectChecked<UPackage>(nullptr, TEXT("/Script/Engine")), TEXT("FormatArgumentData"));

		// Spawn a "Make Struct" node to create the struct needed for formatting the text.
		UK2Node_MakeStruct* MakeFormatArgumentDataStruct = CompilerContext.SpawnIntermediateNode<UK2Node_MakeStruct>(this, SourceGraph);
		MakeFormatArgumentDataStruct->StructType = FormatArgumentDataStruct;
		MakeFormatArgumentDataStruct->AllocateDefaultPins();
		MakeFormatArgumentDataStruct->bMadeAfterOverridePinRemoval = true;
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(MakeFormatArgumentDataStruct, this);

		// Set the struct's "ArgumentName" pin literal to be the argument pin's name.
		MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentName)), ArgumentPin->PinName.ToString());

		UEdGraphPin* ArgumentTypePin = MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueType));

		// Move the connection of the argument pin to the correct argument value pin, and also set the correct argument type based on the pin that was hooked up.
		if (ArgumentPin->LinkedTo.Num() > 0)
		{
			const FName& ArgumentPinCategory = ArgumentPin->PinType.PinCategory;

			// Adds an implicit conversion node to this argument based on its function and pin name
			auto AddConversionNode = [&](const FName FuncName, const TCHAR* PinName)
			{
				// Set the default value if there was something passed in, or default to "Text"
				MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Text"));

				// Spawn conversion node based on the given function name
				UK2Node_CallFunction* ToTextFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				ToTextFunction->SetFromFunction(UKismetTextLibrary::StaticClass()->FindFunctionByName(FuncName));
				ToTextFunction->AllocateDefaultPins();
				CompilerContext.MessageLog.NotifyIntermediateObjectCreation(ToTextFunction, this);

				CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *ToTextFunction->FindPinChecked(PinName));

				ToTextFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue)->MakeLinkTo(MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValue)));
			};

			if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Int)
			{
				MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Int"));
				// Need a manual cast from int -> int64
				UK2Node_CallFunction* CallFloatToDoubleFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				CallFloatToDoubleFunction->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetMathLibrary, Conv_IntToInt64)));
				CallFloatToDoubleFunction->AllocateDefaultPins();
				CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFloatToDoubleFunction, this);

				// Move the byte output pin to the input pin of the conversion node
				CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *CallFloatToDoubleFunction->FindPinChecked(TEXT("InInt")));

				// Connect the int output pin to the argument value
				CallFloatToDoubleFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue)->MakeLinkTo(MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueInt)));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Real)
			{
				if (ArgumentPin->PinType.PinSubCategory == UEdGraphSchema_K2::PC_Float)
				{
					MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Float"));
					CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueFloat)));
				}
				else if (ArgumentPin->PinType.PinSubCategory == UEdGraphSchema_K2::PC_Double)
				{
					MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Double"));
					CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueDouble)));
				}
				else
				{
					check(false);
				}
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Int64)
			{
				MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Int64"));
				CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueInt)));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Text)
			{
				MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Text"));
				CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValue)));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Byte && !ArgumentPin->PinType.PinSubCategoryObject.IsValid())
			{
				MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Int"));

				// Need a manual cast from byte -> int
				UK2Node_CallFunction* CallByteToIntFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				CallByteToIntFunction->SetFromFunction(UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetMathLibrary, Conv_ByteToInt64)));
				CallByteToIntFunction->AllocateDefaultPins();
				CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallByteToIntFunction, this);

				// Move the byte output pin to the input pin of the conversion node
				CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *CallByteToIntFunction->FindPinChecked(TEXT("InByte")));

				// Connect the int output pin to the argument value
				CallByteToIntFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue)->MakeLinkTo(MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueInt)));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Byte || ArgumentPinCategory == UEdGraphSchema_K2::PC_Enum)
			{
				static UEnum* TextGenderEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.ETextGender"), EFindObjectFlags::ExactClass);
				if (ArgumentPin->PinType.PinSubCategoryObject == TextGenderEnum)
				{
					MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Gender"));
					CompilerContext.MovePinLinksToIntermediate(*ArgumentPin, *MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValueGender)));
				}
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Boolean)
			{
				AddConversionNode(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_BoolToText), TEXT("InBool"));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Name)
			{
				AddConversionNode(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_NameToText), TEXT("InName"));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_String)
			{
				AddConversionNode(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_StringToText), TEXT("InString"));
			}
			else if (ArgumentPinCategory == UEdGraphSchema_K2::PC_Object)
			{
				AddConversionNode(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_ObjectToText), TEXT("InObj"));
			}
			else
			{
				// Unexpected pin type!
				CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("Error_UnexpectedPinType", "Pin '{0}' has an unexpected type: {1}"), FText::FromName(PinNames[ArgIdx]), FText::FromName(ArgumentPinCategory)).ToString());
			}
		}
		else
		{
			// No connected pin - just default to an empty text
			MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultValue(*ArgumentTypePin, TEXT("Text"));
			MakeFormatArgumentDataStruct->GetSchema()->TrySetDefaultText(*MakeFormatArgumentDataStruct->FindPinChecked(GET_MEMBER_NAME_STRING_CHECKED(FFormatArgumentData, ArgumentValue)), FText::GetEmpty());
		}

		// The "Make Array" node already has one pin available, so don't create one for ArgIdx == 0
		if(ArgIdx > 0)
		{
			MakeArrayNode->AddInputPin();
		}

		// Find the input pin on the "Make Array" node by index.
		const FString PinName = FString::Printf(TEXT("[%d]"), ArgIdx);
		UEdGraphPin* InputPin = MakeArrayNode->FindPinChecked(PinName);

		// Find the output for the pin's "Make Struct" node and link it to the corresponding pin on the "Make Array" node.
		FindOutputStructPinChecked(MakeFormatArgumentDataStruct)->MakeLinkTo(InputPin);
	}

	// Move connection of FormatText's "Result" pin to the call function's return value pin.
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(TEXT("Result")), *CallFormatFunction->GetReturnValuePin());
	// Move connection of FormatText's "Format" pin to the call function's "InPattern" pin
	CompilerContext.MovePinLinksToIntermediate(*GetFormatPin(), *CallFormatFunction->FindPinChecked(TEXT("InPattern")));

	BreakAllNodeLinks();
}

UEdGraphPin* UK2Node_FormatText::FindArgumentPin(const FName InPinName) const
{
	const UEdGraphPin* FormatPin = GetFormatPin();
	for (UEdGraphPin* Pin : Pins)
	{
		if( Pin != FormatPin && Pin->Direction != EGPD_Output && Pin->PinName.ToString().Equals(InPinName.ToString(), ESearchCase::CaseSensitive) )
		{
			return Pin;
		}
	}

	return nullptr;
}

UK2Node::ERedirectType UK2Node_FormatText::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType RedirectType = ERedirectType_None;

	// if the pin names do match
	if (NewPin->PinName.ToString().Equals(OldPin->PinName.ToString(), ESearchCase::CaseSensitive))
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if( OuterGraph && OuterGraph->Schema )
		{
			const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
			if( !K2Schema || K2Schema->IsSelfPin(*NewPin) || K2Schema->ArePinTypesCompatible(OldPin->PinType, NewPin->PinType) )
			{
				RedirectType = ERedirectType_Name;
			}
			else
			{
				RedirectType = ERedirectType_None;
			}
		}
	}
	else
	{
		// try looking for a redirect if it's a K2 node
		if (UK2Node* Node = Cast<UK2Node>(NewPin->GetOwningNode()))
		{	
			// if you don't have matching pin, now check if there is any redirect param set
			TArray<FString> OldPinNames;
			GetRedirectPinNames(*OldPin, OldPinNames);

			FName NewPinName;
			RedirectType = ShouldRedirectParam(OldPinNames, /*out*/ NewPinName, Node);

			// make sure they match
			if ((RedirectType != ERedirectType_None) && (!NewPin->PinName.ToString().Equals(NewPinName.ToString(), ESearchCase::CaseSensitive)))
			{
				RedirectType = ERedirectType_None;
			}
		}
	}

	return RedirectType;
}

bool UK2Node_FormatText::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	// Argument input pins may only be connected to Byte, Integer, Float, Text, and ETextGender pins...
	const UEdGraphPin* FormatPin = GetFormatPin();
	if (MyPin != FormatPin && MyPin->Direction == EGPD_Input)
	{
		const UEdGraphSchema_K2* K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
		const FName& OtherPinCategory = OtherPin->PinType.PinCategory;

		bool bIsValidType = false;
		if (OtherPinCategory == UEdGraphSchema_K2::PC_Int || OtherPinCategory == UEdGraphSchema_K2::PC_Real || OtherPinCategory == UEdGraphSchema_K2::PC_Text ||
			(OtherPinCategory == UEdGraphSchema_K2::PC_Byte && !OtherPin->PinType.PinSubCategoryObject.IsValid()) ||
			OtherPinCategory == UEdGraphSchema_K2::PC_Boolean || OtherPinCategory == UEdGraphSchema_K2::PC_String || OtherPinCategory == UEdGraphSchema_K2::PC_Name || OtherPinCategory == UEdGraphSchema_K2::PC_Object ||
			OtherPinCategory == UEdGraphSchema_K2::PC_Wildcard || OtherPinCategory == UEdGraphSchema_K2::PC_Int64)
		{
			bIsValidType = true;
		}
		else if (OtherPinCategory == UEdGraphSchema_K2::PC_Byte || OtherPinCategory == UEdGraphSchema_K2::PC_Enum)
		{
			static UEnum* TextGenderEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.ETextGender"), EFindObjectFlags::ExactClass);
			if (OtherPin->PinType.PinSubCategoryObject == TextGenderEnum)
			{
				bIsValidType = true;
			}
		}

		if (!bIsValidType)
		{
			OutReason = LOCTEXT("Error_InvalidArgumentType", "Format arguments may only be Byte, Integer, Int64, Float, Double, Text, String, Name, Boolean, Object, Wildcard or ETextGender.").ToString();
			return true;
		}
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

FText UK2Node_FormatText::GetArgumentName(int32 InIndex) const
{
	if (InIndex < PinNames.Num())
	{
		return FText::FromName(PinNames[InIndex]);
	}
	return FText::GetEmpty();
}

void UK2Node_FormatText::AddArgumentPin()
{
	const FScopedTransaction Transaction( NSLOCTEXT("Kismet", "AddArgumentPin", "Add Argument Pin") );
	Modify();

	const FName PinName(GetUniquePinName());
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, PinName);
	PinNames.Add(PinName);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_FormatText::RemoveArgument(int32 InIndex)
{
	const FScopedTransaction Transaction( NSLOCTEXT("Kismet", "RemoveArgumentPin", "Remove Argument Pin") );
	Modify();

	if (UEdGraphPin* ArgumentPin = FindArgumentPin(PinNames[InIndex]))
	{
		Pins.Remove(ArgumentPin);
		ArgumentPin->MarkAsGarbage();
	}
	PinNames.RemoveAt(InIndex);

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_FormatText::SetArgumentName(int32 InIndex, FName InName)
{
	PinNames[InIndex] = InName;

	ReconstructNode();

	FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
}

void UK2Node_FormatText::SwapArguments(int32 InIndexA, int32 InIndexB)
{
	check(InIndexA < PinNames.Num());
	check(InIndexB < PinNames.Num());
	PinNames.Swap(InIndexA, InIndexB);

	ReconstructNode();
	GetGraph()->NotifyNodeChanged(this);

	FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
}

UEdGraphPin* UK2Node_FormatText::GetFormatPin() const
{
	if (!CachedFormatPin)
	{
		const_cast<UK2Node_FormatText*>(this)->CachedFormatPin = FindPinChecked(FFormatTextNodeHelper::GetFormatPinName());
	}
	return CachedFormatPin;
}

void UK2Node_FormatText::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_FormatText::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Text);
}

#undef LOCTEXT_NAMESPACE
