// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_FunctionEntry.h"

#include "Animation/AnimBlueprint.h"
#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "CoreGlobals.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "FindInBlueprintManager.h"
#include "FindInBlueprints.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_FunctionResult.h"
#include "K2Node_MakeArray.h"
#include "K2Node_MakeVariable.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "ObjectTools.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/Class.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/Field.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ObjectVersion.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_FunctionEntry)

#define LOCTEXT_NAMESPACE "K2Node_FunctionEntry"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_FunctionEntry

class FKCHandler_FunctionEntry : public FNodeHandlingFunctor
{
public:
	FKCHandler_FunctionEntry(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	void RegisterFunctionInput(FKismetFunctionContext& Context, UEdGraphPin* Net, UFunction* Function)
	{
		// This net is a parameter into the function
		FBPTerminal* Term = new FBPTerminal();
		Context.Parameters.Add(Term);
		Term->CopyFromPin(Net, Net->PinName);

		// Flag pass by reference parameters specially
		//@TODO: Still doesn't handle/allow users to declare new pass by reference, this only helps inherited functions
		if( Function )
		{
			if (FProperty* ParentProperty = FindFProperty<FProperty>(Function, Net->PinName))
			{
				if (ParentProperty->HasAnyPropertyFlags(CPF_ReferenceParm))
				{
					Term->bPassedByReference = true;
				}
			}
		}


		Context.NetMap.Add(Net, Term);
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_FunctionEntry* EntryNode = CastChecked<UK2Node_FunctionEntry>(Node);

		UFunction* Function = EntryNode->FunctionReference.ResolveMember<UFunction>(EntryNode->GetBlueprintClassFromNode());
		// if this function has a predefined signature (like for inherited/overridden 
		// functions), then we want to make sure to account for the output 
		// parameters - this is normally handled by the FunctionResult node, but 
		// we're not guaranteed that one is connected to the entry node 
		if (Function && Function->HasAnyFunctionFlags(FUNC_HasOutParms))
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			for (TFieldIterator<FProperty> ParamIt(Function, EFieldIteratorFlags::ExcludeSuper); ParamIt; ++ParamIt)
			{
				FProperty* ParamProperty = *ParamIt;

				// mirrored from UK2Node_FunctionResult::CreatePinsForFunctionEntryExit()
				const bool bIsFunctionInput = !ParamProperty->HasAnyPropertyFlags(CPF_OutParm) || ParamProperty->HasAnyPropertyFlags(CPF_ReferenceParm);
				if (bIsFunctionInput)
				{
					// 
					continue;
				}

				FEdGraphPinType ParamType;
				if (K2Schema->ConvertPropertyToPinType(ParamProperty, ParamType))
				{
					FString ParamName = ParamProperty->GetName();

					bool bTermExists = false;
					// check to see if this terminal already exists (most 
					// likely added by a FunctionResult node) - if so, then 
					// we don't need to add it ourselves
					for (const FBPTerminal& ResultTerm : Context.Results)
					{
						if (ResultTerm.Name == ParamName && ResultTerm.Type == ParamType)
						{
							bTermExists = true;
							break;
						}
					}

					if (!bTermExists)
					{
						// create a terminal that represents a output param 
						// for this function; if there is a FunctionResult 
						// node wired into our function graph, know that it
						// will first check to see if this already exists 
						// for it to use (rather than creating one of its own)
						FBPTerminal* ResultTerm = new FBPTerminal();
						Context.Results.Add(ResultTerm);
						ResultTerm->Name = ParamName;

						ResultTerm->Type = ParamType;
						ResultTerm->bPassedByReference = ParamType.bIsReference;
						ResultTerm->SetContextTypeStruct(ParamType.PinCategory == UEdGraphSchema_K2::PC_Struct && Cast<UScriptStruct>(ParamType.PinSubCategoryObject.Get()));
					}
				}
			}
		}

		for (UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin->ParentPin == nullptr && !CompilerContext.GetSchema()->IsMetaPin(*Pin))
			{
				UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(Pin);

				if (Context.NetMap.Find(Net) == nullptr)
				{
					// New net, resolve the term that will be used to construct it
					FBPTerminal* Term = nullptr;

					check(Net->Direction == EGPD_Output);

					RegisterFunctionInput(Context, Pin, Function);
				}
			}
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_FunctionEntry* EntryNode = CastChecked<UK2Node_FunctionEntry>(Node);
		//check(EntryNode->SignatureName != NAME_None);
		if (EntryNode->FunctionReference.GetMemberName() == UEdGraphSchema_K2::FN_ExecuteUbergraphBase)
		{
			UEdGraphPin* EntryPointPin = Node->FindPin(UEdGraphSchema_K2::PN_EntryPoint);
			FBPTerminal** pTerm = Context.NetMap.Find(EntryPointPin);
			if ((EntryPointPin != nullptr) && (pTerm != nullptr))
			{
				FBlueprintCompiledStatement& ComputedGotoStatement = Context.AppendStatementForNode(Node);
				ComputedGotoStatement.Type = KCST_ComputedGoto;
				ComputedGotoStatement.LHS = *pTerm;
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("NoEntryPointPin_Error", "Expected a pin named EntryPoint on @@").ToString(), Node);
			}
		}
		else
		{
			// Generate the output impulse from this node
			GenerateSimpleThenGoto(Context, *Node);
		}
	}

	virtual bool RequiresRegisterNetsBeforeScheduling() const override
	{
		return true;
	}
};

struct FFunctionEntryHelper
{
	static const FName& GetWorldContextPinName()
	{
		static const FName WorldContextPinName(TEXT("__WorldContext"));
		return WorldContextPinName;
	}

	static bool RequireWorldContextParameter(const UK2Node_FunctionEntry* Node)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		return K2Schema->IsStaticFunctionGraph(Node->GetGraph());
	}
};

UK2Node_FunctionEntry::UK2Node_FunctionEntry(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Enforce const-correctness by default
	bEnforceConstCorrectness = true;
	bUpdatedDefaultValuesOnLoad = false;
	bCanRenameNode = bIsEditable;
}

void UK2Node_FunctionEntry::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	const UBlueprint* Blueprint = HasValidBlueprint() ? GetBlueprint() : nullptr;
	if (Blueprint && LocalVariables.Num() > 0)
	{
		// Forcibly fixup defaults before we save
		UpdateLoadedDefaultValues(true);
	}
}

void UK2Node_FunctionEntry::PostLoad()
{
	Super::PostLoad();

	if (GIsEditor)
	{
		// In the editor, we need to handle processing function default values at load time so they get picked up properly by the cooker
		// This normally won't do anything because it gets called during the duplicate save during BP compilation, but if compilation gets skipped we need to make sure they get updated

		UpdateLoadedDefaultValues();
	}

	// fix deprecated state.
	if (HasAllExtraFlags(FUNC_Const | FUNC_Static))
	{
		// static functions can't be marked const
		SetExtraFlags(GetFunctionFlags() & ~FUNC_Const);
	}
}

void UK2Node_FunctionEntry::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FBlueprintsObjectVersion::GUID);

	if (Ar.IsSaving())
	{		
		if (Ar.IsObjectReferenceCollector() || Ar.Tell() < 0)
		{
			// If this is explicitly a reference collector, or it's a save with no backing archive, then we want to use the function variable cache if it exists
			// It's not safe to regenerate the cache at this point as we could be in GIsSaving
			if (FunctionVariableCache.IsValid() && FunctionVariableCache->IsValid())
			{
				UStruct* Struct = const_cast<UStruct*>(FunctionVariableCache->GetStruct());
				Struct->SerializeBin(Ar, FunctionVariableCache->GetStructMemory());

				// Copy back into defaults as they may have changed
				UpdateDefaultsFromVariableStruct(FunctionVariableCache->GetStruct(), FunctionVariableCache->GetStructMemory());
			}
		}
	}
	else if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::LocalVariablesBlueprintVisible)
		{
			for (FBPVariableDescription& LocalVariable : LocalVariables)
			{
				LocalVariable.PropertyFlags |= CPF_BlueprintVisible;
			}
		}

		if (Ar.UEVer() < VER_UE4_BLUEPRINT_ENFORCE_CONST_IN_FUNCTION_OVERRIDES
			|| ((Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::EnforceConstInAnimBlueprintFunctionGraphs) && GetBlueprint()->IsA<UAnimBlueprint>()))
		{
			// Allow legacy implementations to violate const-correctness
			bEnforceConstCorrectness = false;
		}

		if (Ar.CustomVer(FBlueprintsObjectVersion::GUID) < FBlueprintsObjectVersion::CleanBlueprintFunctionFlags)
		{
			// Flags we explicitly use ExtraFlags for (at the time this fix was made):
			//     FUNC_Public, FUNC_Protected, FUNC_Private, 
			//     FUNC_Static, FUNC_Const,
			//     FUNC_BlueprintPure, FUNC_BlueprintCallable, FUNC_BlueprintEvent, FUNC_BlueprintAuthorityOnly,
			//     FUNC_Net, FUNC_NetMulticast, FUNC_NetServer, FUNC_NetClient, FUNC_NetReliable
			// 
			// FUNC_Exec, FUNC_Event, & FUNC_BlueprintCosmetic are all inherited 
			// in FKismetCompilerContext::PrecompileFunction()
			static const uint32 InvalidExtraFlagsMask = FUNC_Final | FUNC_RequiredAPI | FUNC_BlueprintCosmetic |
				FUNC_NetRequest | FUNC_Exec | FUNC_Native | FUNC_Event | FUNC_NetResponse | FUNC_MulticastDelegate |
				FUNC_Delegate | FUNC_HasOutParms | FUNC_HasDefaults | FUNC_DLLImport | FUNC_NetValidate;
			ExtraFlags &= ~InvalidExtraFlagsMask;
		}

		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::ChangeAssetPinsToString)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			// Prior to this version, changing the type of a local variable would lead to corrupt default value strings
			for (FBPVariableDescription& LocalVar : LocalVariables)
			{
				FString UseDefaultValue;
				TObjectPtr<UObject> UseDefaultObject = nullptr;
				FText UseDefaultText;

				if (!LocalVar.DefaultValue.IsEmpty())
				{
					K2Schema->GetPinDefaultValuesFromString(LocalVar.VarType, this, LocalVar.DefaultValue, UseDefaultValue, UseDefaultObject, UseDefaultText);
					FString ErrorMessage;

					if (!K2Schema->DefaultValueSimpleValidation(LocalVar.VarType, LocalVar.VarName, UseDefaultValue, UseDefaultObject, UseDefaultText, &ErrorMessage))
					{
						const UBlueprint* Blueprint = GetBlueprint();
						UE_LOGF(LogBlueprint, Log, "Clearing invalid default value for local variable %ls on blueprint %ls: %ls", *LocalVar.VarName.ToString(), Blueprint ? *Blueprint->GetName() : TEXT("Unknown"), *ErrorMessage);

						LocalVar.DefaultValue.Reset();
					}
				}
			}
		}
	}
}

void UK2Node_FunctionEntry::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UK2Node_FunctionEntry* This = CastChecked<UK2Node_FunctionEntry>(InThis);
	TSharedPtr<FStructOnScope> FunctionVariableCache = This->FunctionVariableCache;
	if (FunctionVariableCache.IsValid())
	{
		TWeakObjectPtr<const UStruct> StructPtr = FunctionVariableCache->GetStructPtr();
		if (StructPtr.IsValid())
		{
			Collector.AddPropertyReferences(StructPtr.Get(), FunctionVariableCache->GetStructMemory());
		}
	}
	return UK2Node_FunctionTerminator::AddReferencedObjects(InThis, Collector);
}

FText UK2Node_FunctionEntry::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraph* Graph = GetGraph();
	FGraphDisplayInfo DisplayInfo;
	Graph->GetSchema()->GetGraphDisplayInformation(*Graph, DisplayInfo);

	return DisplayInfo.DisplayName;
}

void UK2Node_FunctionEntry::OnRenameNode(const FString& NewName)
{
	// Note: RenameGraph() will handle the rename operation for this node as well.
	FBlueprintEditorUtils::RenameGraph(GetGraph(), NewName);
}

TSharedPtr<class INameValidatorInterface> UK2Node_FunctionEntry::MakeNameValidator() const
{
	if (CustomGeneratedFunctionName.IsNone())
	{
		FText TextName = GetNodeTitle(ENodeTitleType::Type::EditableTitle);
		return MakeShareable(new FKismetNameValidator(GetBlueprint(), *TextName.ToString()));
	}
	else
	{
		return MakeShareable(new FKismetNameValidator(GetBlueprint(), CustomGeneratedFunctionName));
	}
}

bool UK2Node_FunctionEntry::GetCanRenameNode() const
{
	UEdGraph* const Graph = GetGraph();

	return (Graph && (Graph->bAllowDeletion || Graph->bAllowRenaming) && (bCanRenameNode || bIsEditable));
}

void UK2Node_FunctionEntry::AllocateDefaultPins()
{
	// Update our default values before copying them into pins
	UpdateLoadedDefaultValues();

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// Find any pins inherited from parent
	if (UFunction* Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		CreatePinsForFunctionEntryExit(Function, /*bIsFunctionEntry=*/ true);
	}

	Super::AllocateDefaultPins();

	if (FFunctionEntryHelper::RequireWorldContextParameter(this) 
		&& !FindPin(FFunctionEntryHelper::GetWorldContextPinName()))
	{
		UEdGraphPin* WorldContextPin = CreatePin(
			EGPD_Output,
			UEdGraphSchema_K2::PC_Object,
			UObject::StaticClass(),
			FFunctionEntryHelper::GetWorldContextPinName());
		WorldContextPin->bHidden = true;
	}
}

UEdGraphPin* UK2Node_FunctionEntry::GetAutoWorldContextPin() const
{
	return FindPin(FFunctionEntryHelper::GetWorldContextPinName());
}

void UK2Node_FunctionEntry::RemoveOutputPin(UEdGraphPin* PinToRemove)
{
	UK2Node_FunctionEntry* OwningSeq = Cast<UK2Node_FunctionEntry>( PinToRemove->GetOwningNode() );
	if (OwningSeq)
	{
		PinToRemove->MarkAsGarbage();
		OwningSeq->Pins.Remove(PinToRemove);
	}
}

bool UK2Node_FunctionEntry::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	bool bResult = Super::CanCreateUserDefinedPin(InPinType, InDesiredDirection, OutErrorMessage);
	if (bResult)
	{
		if(InDesiredDirection == EGPD_Input)
		{
			OutErrorMessage = LOCTEXT("AddInputPinError", "Cannot add input pins to function entry node!");
			bResult = false;
		}
	}
	return bResult;
}

UEdGraphPin* UK2Node_FunctionEntry::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	// Make sure that if this is an exec node we are allowed one.
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if (NewPinInfo->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec && !CanModifyExecutionWires())
	{
		return nullptr;
	}

	UEdGraphPin* NewPin = CreatePin(EGPD_Output, NewPinInfo->PinType, NewPinInfo->PinName);
	Schema->SetPinAutogeneratedDefaultValue(NewPin, NewPinInfo->PinDefaultValue);
	return NewPin;
}

TSharedPtr<FStructOnScope> UK2Node_FunctionEntry::GetFunctionVariableCache(bool bForceRefresh)
{
	if (bForceRefresh && FunctionVariableCache.IsValid())
	{
		// On force refresh, delete old one if it exists
		FunctionVariableCache.Reset();
	}

	if (!FunctionVariableCache.IsValid() && HasValidBlueprint() && LocalVariables.Num() > 0)
	{
		// Locate the UFunction object in the class hierarchy, starting at the current class. Note that local
		// variables are generated as fields (properties) within the function context that contains this node,
		// so for parent class/interface function overrides we want to make sure we're looking at the most-
		// derived UFunction object. Also note that FunctionFromNode() looks at the skeleton class rather than
		// the [authoritative] generated class, since the skeleton class is implicitly recompiled after e.g. adding
		// an input/output argument or local variable, whereas the generated class must be explicitly recompiled.
		if (UFunction* const Function = FFunctionFromNodeHelper::FunctionFromNode(this))
		{
			FunctionVariableCache = MakeShared<FStructOnScope>(Function);
			FunctionVariableCache->SetPackage(GetOutermost());

			RefreshFunctionVariableCache();
		}
	}
	
	return FunctionVariableCache;
}

bool UK2Node_FunctionEntry::RefreshFunctionVariableCache()
{
	GetFunctionVariableCache(false);

	if (FunctionVariableCache.IsValid())
	{
		// Update the cache if it was created
		return UpdateVariableStructFromDefaults(FunctionVariableCache->GetStruct(), FunctionVariableCache->GetStructMemory());
	}
	return false;
}

bool UK2Node_FunctionEntry::UpdateLoadedDefaultValues(bool bForceRefresh)
{
	// If we don't have a cache or it's force refresh, create one
	if (!bUpdatedDefaultValuesOnLoad || bForceRefresh)
	{
		GetFunctionVariableCache(bForceRefresh);

		bUpdatedDefaultValuesOnLoad = true;

		if (FunctionVariableCache.IsValid())
		{
			// Now copy back into the default value strings
			return UpdateDefaultsFromVariableStruct(FunctionVariableCache->GetStruct(), FunctionVariableCache->GetStructMemory());
		}
		else
		{
			// No variable cache created
			return true;
		}
	}

	return false;
}

void UK2Node_FunctionEntry::ClearCachedBlueprintData(UBlueprint* Blueprint)
{
	FunctionVariableCache.Reset();
}

bool UK2Node_FunctionEntry::UpdateVariableStructFromDefaults(const UStruct* VariableStruct, uint8* VariableStructData)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	if (!VariableStruct || !VariableStructData)
	{
		return false;
	}

	for (FBPVariableDescription& LocalVariable : LocalVariables)
	{
		if (!LocalVariable.DefaultValue.IsEmpty())
		{
			FProperty* PinProperty = VariableStruct->FindPropertyByName(LocalVariable.VarName);

			if (PinProperty && (!PinProperty->HasAnyPropertyFlags(CPF_OutParm) || PinProperty->HasAnyPropertyFlags(CPF_ReferenceParm)))
			{
				FEdGraphPinType PinType;
				K2Schema->ConvertPropertyToPinType(PinProperty, /*out*/ PinType);

				if (PinType != LocalVariable.VarType)
				{
					//UE_LOGF(LogBlueprint, Log, "Pin type for local variable %ls does not match type on struct %ls during UpdateVariableStructFromDefaults, ignoring old default", *LocalVariable.VarName.ToString(), *VariableStruct->GetName());
				}
				else
				{
					FBlueprintEditorUtils::PropertyValueFromString(PinProperty, LocalVariable.DefaultValue, VariableStructData, this);
				}
			}
			else
			{
				//UE_LOGF(LogBlueprint, Log, "Could not find local variable property %ls on struct %ls during UpdateVariableStructFromDefaults", *LocalVariable.VarName.ToString(), *VariableStruct->GetName());
			}
		}
	}

	return true;
}

bool UK2Node_FunctionEntry::UpdateDefaultsFromVariableStruct(const UStruct* VariableStruct, uint8* VariableStructData)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	if (!VariableStruct || !VariableStructData)
	{
		return false;
	}

	for (FBPVariableDescription& LocalVariable : LocalVariables)
	{
		if (!LocalVariable.DefaultValue.IsEmpty())
		{
			// We don't want to write out fields that were empty before, as they were guaranteed to not have actual real data
			FProperty* PinProperty = VariableStruct->FindPropertyByName(LocalVariable.VarName);

			if (PinProperty && (!PinProperty->HasAnyPropertyFlags(CPF_OutParm) || PinProperty->HasAnyPropertyFlags(CPF_ReferenceParm)))
			{
				FEdGraphPinType PinType;
				K2Schema->ConvertPropertyToPinType(PinProperty, /*out*/ PinType);

				if (PinType != LocalVariable.VarType)
				{
					//UE_LOGF(LogBlueprint, Log, "Pin type for local variable %ls does not match type on struct %ls during UpdateDefaultsFromVariableStruct, ignoring old default", *LocalVariable.VarName.ToString(), *VariableStruct->GetName());
				}
				else
				{
					FString NewValue;
					FBlueprintEditorUtils::PropertyValueToString(PinProperty, VariableStructData, NewValue, this);
					if (NewValue != LocalVariable.DefaultValue)
					{
						LocalVariable.DefaultValue = NewValue;
					}
				}
			}
			else
			{
				//UE_LOGF(LogBlueprint, Log, "Could not find local variable property %ls on struct %ls during UpdateDefaultsFromVariableStruct", *LocalVariable.VarName.ToString(), *VariableStruct->GetName());
			}
		}
	}

	return true;
}


FNodeHandlingFunctor* UK2Node_FunctionEntry::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_FunctionEntry(CompilerContext);
}

void UK2Node_FunctionEntry::GetRedirectPinNames(const UEdGraphPin& Pin, TArray<FString>& RedirectPinNames) const
{
	Super::GetRedirectPinNames(Pin, RedirectPinNames);

	if(RedirectPinNames.Num() > 0)
	{
		const FString OldPinName = RedirectPinNames[0];

		
		// first add functionname.param
		const FName SignatureName = FunctionReference.GetMemberName();
		RedirectPinNames.Add(FString::Printf(TEXT("%s.%s"), *SignatureName.ToString(), *OldPinName));
		// if there is class, also add an option for class.functionname.param
		if(UClass const* SignatureClass = FunctionReference.GetMemberParentClass())
		{
			RedirectPinNames.Add(FString::Printf(TEXT("%s.%s.%s"), *SignatureClass->GetName(), *SignatureName.ToString(), *OldPinName));
		}
	}
}

bool UK2Node_FunctionEntry::HasDeprecatedReference() const
{
	// We only show deprecated for inherited functions
	if (UFunction* const Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		return Function->HasMetaData(FBlueprintMetadata::MD_DeprecatedFunction);
	}
	else
	{
		return MetaData.bIsDeprecated;
	}
}

FEdGraphNodeDeprecationResponse UK2Node_FunctionEntry::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		// Only warn on non-editable (i.e. override) usage.
		if (!IsEditable())
		{
			UFunction* const Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode());
			if (ensureMsgf(Function != nullptr, TEXT("This node should not be able to report having a deprecated reference if the override function cannot be resolved.")))
			{
				// Check the deprecation type to override the severity
				FString MessageType = Function->GetMetaData(FBlueprintMetadata::MD_DeprecatedFunction);
				Response.MessageType = FBlueprintEditorUtils::GetDeprecatedMessageType(MessageType);

				FText FunctionName = FText::FromName(FunctionReference.GetMemberName());
				FText DetailedMessage = FText::FromString(Function->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage));
				Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(FunctionName, DetailedMessage);
			}
		}
		else
		{
			// Allow the function to be marked as deprecated in the class that defines it without warning, but use a note to visually indicate that the definition itself has been deprecated.
			Response.MessageType = EEdGraphNodeDeprecationMessageType::Note;
			Response.MessageText = LOCTEXT("DeprecatedFunctionMessage", "@@: This function has been marked as deprecated. It can be safely deleted if all references have been replaced or removed.");
		}
	}

	return Response;
}

FText UK2Node_FunctionEntry::GetTooltipText() const
{
	if (UFunction* const Function = FindSignatureFunction())
	{
		return FText::FromString(ObjectTools::GetDefaultTooltipForFunction(Function));
	}
	return Super::GetTooltipText();
}

void UK2Node_FunctionEntry::FindDiffs(UEdGraphNode* OtherNode, struct FDiffResults& Results)
{
	Super::FindDiffs(OtherNode, Results);
	UK2Node_FunctionEntry* OtherFunction = Cast<UK2Node_FunctionEntry>(OtherNode);

	if (OtherFunction)
	{
		if (ExtraFlags != OtherFunction->ExtraFlags)
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.DisplayString = LOCTEXT("DIF_FunctionFlags", "Function flags have changed");
			Diff.Category = EDiffType::MODIFICATION;

			Results.Add(Diff);
		}

		if (!FKismetUserDeclaredFunctionMetadata::StaticStruct()->CompareScriptStruct(&MetaData, &OtherFunction->MetaData, 0))
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.DisplayString = LOCTEXT("DIF_FunctionMetadata", "Function metadata has changed");
			Diff.Category = EDiffType::MODIFICATION;

			Results.Add(Diff);
		}

		// constructs a map from variable guid to index in the provided array
		auto MapVarGuidToArrayIndex = [](const TArray<FBPVariableDescription>& Variables)
		{
			TMap<FGuid, int32> Map;
			for (int32 i = 0; i < Variables.Num(); i++)
			{
				Map.Add(Variables[i].VarGuid, i);
			}
			return MoveTemp(Map);
		};

		auto AddDiff = [this, OtherNode, &Results](const FText& DisplayString)
		{
			FDiffSingleResult Diff;
			Diff.Diff = EDiffType::NODE_PROPERTY;
			Diff.Node1 = this;
			Diff.Node2 = OtherNode;
			Diff.DisplayString = DisplayString;
			Results.Add(Diff);
		};

		// using maps so that order doesn't matter and it's easy to diff
		const TMap<FGuid, int32> VarGuidToIndex = MapVarGuidToArrayIndex(LocalVariables);
		const TMap<FGuid, int32> OtherVarGuidToIndex = MapVarGuidToArrayIndex(OtherFunction->LocalVariables);

		FDiffSingleResult Diff;
		Diff.Diff = EDiffType::NODE_PROPERTY;
		Diff.Node1 = this;
		Diff.Node2 = OtherNode;
		const FText NodeName = FText::FromName(FunctionReference.GetMemberName());
		for (const auto& [VarGuid,Index] : VarGuidToIndex)
		{
			const FBPVariableDescription& ThisVar = LocalVariables[Index];
			const FText OldVarName = FText::FromName(ThisVar.VarName);
			
			if (const int32* OtherIndex = OtherVarGuidToIndex.Find(VarGuid))
			{
				Diff.Category = EDiffType::MODIFICATION;
				const FBPVariableDescription& OtherVar = OtherFunction->LocalVariables[*OtherIndex];
				const FText NewVarName = FText::FromName(OtherVar.VarName);
				if (ThisVar.VarName != OtherVar.VarName)
				{
					// variable name changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableNameChange", "Local variable name changed from {0}::{1} to {0}::{2}"),
						NodeName,
						OldVarName,
						NewVarName
					));
				}
				if (ThisVar.VarType != OtherVar.VarType)
				{
					// type changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableTypeChange", "Local variable {0}::{1} was a {2} but is now a {3}"),
						NodeName,
						NewVarName,
						UEdGraphSchema_K2::TypeToText(ThisVar.VarType),
						UEdGraphSchema_K2::TypeToText(OtherVar.VarType)
					));
				}
				if (!ThisVar.Category.EqualTo(OtherVar.Category))
				{
					// category changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableCategoryChange", "Local variable {0}::{1} changed category from {2} to {3}"),
						NodeName,
						NewVarName,
						ThisVar.Category,
						OtherVar.Category
					));
				}
				if (ThisVar.PropertyFlags != OtherVar.PropertyFlags)
				{
					// property flags changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableFlagChange", "Local variable {0}::{1} changed property flags"),
						NodeName,
						NewVarName
					));
				}
				if (ThisVar.RepNotifyFunc != OtherVar.RepNotifyFunc)
				{
					// replication notify function changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableRepNotifyFuncChange", "Local variable {0}::{1} changed RepNotifyFunc"),
						NodeName,
						NewVarName
					));
				}
				if (ThisVar.ReplicationCondition != OtherVar.ReplicationCondition)
				{
					// replication condition changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableRepConditionChange", "Local variable {0}::{1} changed ReplicationCondition"),
						NodeName,
						NewVarName
					));
				}
				if (ThisVar.DefaultValue != OtherVar.DefaultValue)
				{
					// replication condition changed
					AddDiff(FText::Format(
						LOCTEXT("DIF_FunctionLocalVariableDefaultValueChange", "Local variable {0}::{1} changed default value from '{2}' to '{3}'"),
						NodeName,
						NewVarName,
						FText::FromString(ThisVar.DefaultValue),
						FText::FromString(OtherVar.DefaultValue)
					));
				}
			}
			else
			{
				// local variable is in this node but not in other node
				Diff.Category = EDiffType::SUBTRACTION;
				AddDiff(FText::Format(
				LOCTEXT("DIF_FunctionLocalVariableRemoved", "Local variable {0}::{1} removed"),
					NodeName,
					OldVarName
				));
			}
		}
		
		for (const auto& [VarGuid,Index] : OtherVarGuidToIndex)
		{
			const FBPVariableDescription& OtherVar = OtherFunction->LocalVariables[Index];
			if (!VarGuidToIndex.Contains(VarGuid))
			{
				const FText NewVarName = FText::FromName(OtherVar.VarName);
				
				// local variable is in other node but not in this node
				Diff.Category = EDiffType::ADDITION;
				AddDiff(FText::Format(
				LOCTEXT("DIF_FunctionLocalVariableAdded", "Local variable {0}::{1} added"),
					NodeName,
					NewVarName
				));
			}
		}
	}
}

bool UK2Node_FunctionEntry::IsCompatibleWithGraph(const UEdGraph* InGraph) const
{
	if (CanCreateUnderSpecifiedSchema(InGraph->GetSchema()))
	{
		if (InGraph->GetSchema()->GetGraphType(InGraph) == GT_Function)
		{
			TArray<UK2Node_FunctionEntry*> Nodes;
			InGraph->GetNodesOfClass<UK2Node_FunctionEntry>(Nodes);
			return Nodes.Num() == 0;
		}
	}

	return false;
}

void UK2Node_FunctionEntry::PostPasteNode()
{
	Super::PostPasteNode();

	// If a function entry is being pasted, it should be editable in it's new graph
	bIsEditable = true;

	// ensure there are UserDefinedPins for all pins except the 'then' pin
	for (int32 PinIdx = 1; PinIdx < Pins.Num(); ++PinIdx)
	{
		UEdGraphPin* Pin = Pins[PinIdx];
		if (Pin && !UserDefinedPinExists(Pin->GetFName()))
		{
			UserDefinedPins.Add(MakeShared<FUserPinInfo>(*Pin));
		}
	}

	ReconstructNode();
}

void UK2Node_FunctionEntry::AddSearchMetaDataInfo(TArray<FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddSearchMetaDataInfo(OutTaggedMetaData);

	if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
	{
		// Index the native name of the function, this will be used in search queries rather than node title
		const FString FunctionNativeName = Function->GetName();
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(FunctionNativeName)));

		// Index the (ancestor) class or interface from which the function originates, can be self
		if (const UClass* FuncOriginClass = FindInBlueprintsHelpers::GetFunctionOriginClass(Function))
		{
			const FString FuncOriginClassName = FuncOriginClass->GetPathName();
			OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_FuncOriginClass, FText::FromString(FuncOriginClassName)));
		}
	}
}

int32 UK2Node_FunctionEntry::GetFunctionFlags() const
{
	int32 ReturnFlags = 0;

	if (UFunction* const Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		ReturnFlags = Function->FunctionFlags;
	}
	return ReturnFlags | ExtraFlags;
}

namespace UE::BlueprintGraph::Private
{
	bool GGenerateFieldNotifyBroadcastForOnRepFunction = true;
	static FAutoConsoleVariableRef CVarGenerateFieldNotifyBroadcastForOnRepFunction(
		TEXT("bp.GenerateFieldNotifyBroadcastForOnRepFunction"),
		GGenerateFieldNotifyBroadcastForOnRepFunction,
		TEXT("When needed, generate a Broadcast FieldNotification node when the OnRep function is called."),
		ECVF_Default
	);
}

void UK2Node_FunctionEntry::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	
	UEdGraphPin* OldStartExecPin = nullptr;

	if(Pins[0]->LinkedTo.Num())
	{
		OldStartExecPin = Pins[0]->LinkedTo[0];
	}
	
	UEdGraphPin* LastActiveOutputPin = Pins[0];
	bool bLinkLastPin = false;

	// Only look for FunctionEntry nodes who were duplicated and have a source object
	if ( UK2Node_FunctionEntry* OriginalNode = Cast<UK2Node_FunctionEntry>(CompilerContext.MessageLog.FindSourceObject(this)) )
	{
		check(OriginalNode->GetOuter());

		// Find the associated UFunction
		UFunction* Function = FindUField<UFunction>(CompilerContext.Blueprint->SkeletonGeneratedClass, *OriginalNode->GetOuter()->GetName());

		for (TFieldIterator<FProperty> It(Function); It; ++It)
		{
			if (const FProperty* Property = *It)
			{
				const FStructProperty* PotentialUDSProperty = CastField<const FStructProperty>(Property);

				for (const FBPVariableDescription& LocalVar : LocalVariables)
				{
					if (LocalVar.VarName == Property->GetFName() && !LocalVar.DefaultValue.IsEmpty())
					{
						// Add a variable set node for the local variable and hook it up immediately following the entry node or the last added local variable
						UK2Node_VariableSet* VariableSetNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableSet>(this, SourceGraph);
						VariableSetNode->SetFromProperty(Property, false, Property->GetOwnerClass());
						Schema->ConfigureVarNode(VariableSetNode, LocalVar.VarName, Function, CompilerContext.Blueprint);
						VariableSetNode->AllocateDefaultPins();

						if(UEdGraphPin* SetPin = VariableSetNode->FindPin(Property->GetFName()))
						{
							if(LocalVar.VarType.IsArray())
							{
								TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(Function));
								FBlueprintEditorUtils::PropertyValueFromString(Property, LocalVar.DefaultValue, StructData->GetStructMemory());

								// Create a Make Array node to setup the array's defaults
								UK2Node_MakeArray* MakeArray = CompilerContext.SpawnIntermediateNode<UK2Node_MakeArray>(this, SourceGraph);
								MakeArray->AllocateDefaultPins();
								MakeArray->GetOutputPin()->MakeLinkTo(SetPin);
								MakeArray->PostReconstructNode();

								const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
								check(ArrayProperty);

								FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, StructData->GetStructMemory());
								FScriptArrayHelper_InContainer DefaultArrayHelper(ArrayProperty, StructData->GetStructMemory());

								// Go through each element in the array to set the default value
								for( int32 ArrayIndex = 0 ; ArrayIndex < ArrayHelper.Num() ; ArrayIndex++ )
								{
									uint8* PropData = ArrayHelper.GetRawPtr(ArrayIndex);

									// Retrieve the element's default value
									FString DefaultValue;
									FBlueprintEditorUtils::PropertyValueToString(ArrayProperty->Inner, PropData, DefaultValue);

									if(ArrayIndex > 0)
									{
										MakeArray->AddInputPin();
									}

									// Add one to the index for the pin to set the default on to skip the output pin
									Schema->TrySetDefaultValue(*MakeArray->Pins[ArrayIndex + 1], DefaultValue);
								}
							}
							else if(LocalVar.VarType.IsSet() || LocalVar.VarType.IsMap())
							{
								UK2Node_MakeVariable* MakeVariableNode = CompilerContext.SpawnIntermediateNode<UK2Node_MakeVariable>(this, SourceGraph);
								MakeVariableNode->SetupVariable(LocalVar, SetPin, CompilerContext, Function, Property);
							}
							else
							{
								// Set the default value
								Schema->TrySetDefaultValue(*SetPin, LocalVar.DefaultValue);
							}
						}

						LastActiveOutputPin->BreakAllPinLinks();
						LastActiveOutputPin->MakeLinkTo(VariableSetNode->Pins[0]);
						LastActiveOutputPin = VariableSetNode->Pins[1];
					}
				}
			}
		}

		bLinkLastPin = true;
	}

	if (UE::BlueprintGraph::Private::GGenerateFieldNotifyBroadcastForOnRepFunction)
	{
		if (FProperty** RepNotifyProperty = CompilerContext.RepNotifyFunctionMap.Find(FunctionReference.GetMemberName()))
		{
			if ((*RepNotifyProperty)->HasMetaData(FBlueprintMetadata::MD_FieldNotify))
			{
				// Generate node to broadcast when the value is changed from the network
				TTuple<UEdGraphPin*, UEdGraphPin*> ExecThenPins = FKismetCompilerUtilities::GenerateBroadcastFieldNotificationNode(CompilerContext, SourceGraph, this, *RepNotifyProperty);

				LastActiveOutputPin->BreakAllPinLinks();
				LastActiveOutputPin->MakeLinkTo(ExecThenPins.Get<0>());
				LastActiveOutputPin = ExecThenPins.Get<1>();

				bLinkLastPin = true;
			}
		}
	}

	// Finally, hook up the last node to the old node the function entry node was connected to
	if (bLinkLastPin && OldStartExecPin)
	{
		LastActiveOutputPin->MakeLinkTo(OldStartExecPin);
	}
}

void UK2Node_FunctionEntry::PostReconstructNode()
{
	Super::PostReconstructNode();
}

void UK2Node_FunctionEntry::FixupPinStringDataReferences(FArchive* SavingArchive)
{
	Super::FixupPinStringDataReferences(SavingArchive);
	if (SavingArchive)
	{
		UpdateUserDefinedPinDefaultValues();
	}
}

ERenamePinResult UK2Node_FunctionEntry::RenameUserDefinedPinImpl(FName OldName, FName NewName, bool bTest)
{
	ERenamePinResult Result = Super::RenameUserDefinedPinImpl(OldName, NewName, bTest);

	// We don't actually rename our local variables via this method,
	// but we do need to verify that no name collisions will occur when testing a pin rename.

	const bool bCheckLocalVariablesNameCollision =
		(Result == ERenamePinResult::ERenamePinResult_Success) &&
		bTest
	;

	if (bCheckLocalVariablesNameCollision)
	{
		for (const FBPVariableDescription& Entry : LocalVariables)
		{
			if (NewName == Entry.VarName)
			{
				Result = ERenamePinResult_NameCollision;
				break;
			}
		}
	}

	return Result;
}

bool UK2Node_FunctionEntry::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(PinInfo, NewDefaultValue))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);

		RefreshFunctionVariableCache();

		return true;
	}
	return false;
}

bool UK2Node_FunctionEntry::ShouldUseConstRefParams() const
{
	// Interface functions with no outputs will be implemented as events. As with native interface functions with no outputs, the entry
	// node is expected to use 'const Type&' for input parameters that are passed by reference. See UEditablePinBase::PostLoad() for details.
	if (const UEdGraph* OwningGraph = GetGraph())
	{
		const UBlueprint* OwningBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(OwningGraph);
		if (OwningBlueprint && OwningBlueprint->BlueprintType == BPTYPE_Interface)
		{
			// Find paired result node and check for outputs.
			for (UEdGraphNode* Node : OwningGraph->Nodes)
			{
				if (UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(Node))
				{
					// This might be called from the super's Serialize() method for older assets, so make sure the result node's pins have been loaded.
					ResultNode->ConditionalPreload();

					return ResultNode->UserDefinedPins.Num() == 0;
				}
			}

			// No result node, so there are no outputs.
			return true;
		}
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
