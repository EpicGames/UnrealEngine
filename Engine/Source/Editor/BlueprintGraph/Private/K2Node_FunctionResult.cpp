// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_FunctionResult.h"
#include "Misc/CoreMisc.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_FunctionEntry.h"

#include "BPTerminal.h"
#include "KismetCompiledFunctionContext.h"
#include "VariableSetHandler.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_FunctionResult)

struct FFillDefaultPinValueHelper
{
private:
	static void FillInner(const UEdGraphSchema_K2* K2Schema, UEdGraphPin* Pin)
	{
		if(K2Schema && Pin)
		{
			const bool bValuePin = (Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec);
			const bool bNotConnected = (Pin->Direction == EEdGraphPinDirection::EGPD_Input) && (0 == Pin->LinkedTo.Num());
			const bool bNeedToResetDefaultValue = (Pin->DefaultValue.IsEmpty() && Pin->DefaultObject == nullptr && Pin->DefaultTextValue.IsEmpty()) || !(K2Schema->IsPinDefaultValid(Pin, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue).IsEmpty());
			if (bValuePin && bNotConnected && bNeedToResetDefaultValue)
			{
				K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
			}
		}
	}

public:
	static void Fill(UEdGraphPin* Pin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		if(K2Schema)
		{
			FillInner(K2Schema, Pin);
		}
	}

	static  void FillAll(UK2Node_FunctionResult* Node)
	{
		if(Node)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			if(K2Schema)
			{
				for (int32 PinIdx = 0; PinIdx < Node->Pins.Num(); PinIdx++)
				{
					FillInner(K2Schema, Node->Pins[PinIdx]);
				}
			}
		}
	}
};

//////////////////////////////////////////////////////////////////////////
// FKCHandler_FunctionResult

class FKCHandler_FunctionResult : public FKCHandler_VariableSet
{
public:
	FKCHandler_FunctionResult(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_VariableSet(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override
	{
		// Do not register as a default any Pin that comes from being Split
		if (Net->ParentPin == nullptr)
		{
			FString NetPinName = Net->PinName.ToString();
			for (FBPTerminal& ResultTerm : Context.Results)
			{
				if ((ResultTerm.Name == NetPinName) && (ResultTerm.Type == Net->PinType))
				{
					Context.NetMap.Add(Net, &ResultTerm);
					return;
				}
			}
			FBPTerminal* Term = new FBPTerminal();
			Context.Results.Add(Term);
			Term->CopyFromPin(Net, MoveTemp(NetPinName));
			Context.NetMap.Add(Net, Term);
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		static const FBoolConfigValueHelper ExecutionAfterReturn(TEXT("Kismet"), TEXT("bExecutionAfterReturn"), GEngineIni);

		if (ExecutionAfterReturn)
		{
			// for backward compatibility only
			FKCHandler_VariableSet::Compile(Context, Node);
		}
		else
		{
			GenerateAssigments(Context, Node);

			if (Context.IsDebuggingOrInstrumentationRequired() && Node)
			{
				FBlueprintCompiledStatement& TraceStatement = Context.AppendStatementForNode(Node);
				TraceStatement.Type = Context.GetWireTraceType();
				TraceStatement.Comment = Node->NodeComment.IsEmpty() ? Node->GetName() : Node->NodeComment;
			}

			// always go to return
			FBlueprintCompiledStatement& GotoStatement = Context.AppendStatementForNode(Node);
			GotoStatement.Type = KCST_GotoReturn;
		}
	}

	virtual bool RequiresRegisterNetsBeforeScheduling() const override
	{
		return true;
	}

protected:
	virtual bool UsesVariablePinAsKey() const override { return true; }
};

#if WITH_EDITORONLY_DATA
namespace
{
	void GatherFunctionResultNodeForLocalization(const UObject* const Object, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
	{
		const UK2Node_FunctionResult* const FunctionResultNode = CastChecked<UK2Node_FunctionResult>(Object);

		// Function Result (aka, Return) nodes always report their values as being the default
		// but we still need to gather them as they are the only place the values are defined
		const FString PathToObject = FunctionResultNode->GetPathName();
		for (const UEdGraphPin* Pin : FunctionResultNode->Pins)
		{
			if (!Pin->DefaultTextValue.IsEmpty())
			{
				PropertyLocalizationDataGatherer.GatherTextInstance(Pin->DefaultTextValue, FString::Printf(TEXT("%s.%s"), *PathToObject, *Pin->GetName()), /*bIsEditorOnly*/true);
			}
		}

		PropertyLocalizationDataGatherer.GatherLocalizationDataFromObject(FunctionResultNode, GatherTextFlags);
	}
}
#endif

UK2Node_FunctionResult::UK2Node_FunctionResult(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	{ static const FAutoRegisterLocalizationDataGatheringCallback AutomaticRegistrationOfLocalizationGatherer(UK2Node_FunctionResult::StaticClass(), &GatherFunctionResultNodeForLocalization); }
#endif
}

FText UK2Node_FunctionResult::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (ENodeTitleType::MenuTitle == TitleType)
	{
		return NSLOCTEXT("K2Node", "ReturnNodeMenuTitle", "Add Return Node...");
	}
	return NSLOCTEXT("K2Node", "ReturnNode", "Return Node");
}

void UK2Node_FunctionResult::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	if (UFunction* const Function = FunctionReference.ResolveMember<UFunction>(GetBlueprintClassFromNode()))
	{
		CreatePinsForFunctionEntryExit(Function, /*bIsFunctionEntry=*/ false);
	}

	Super::AllocateDefaultPins();

	FFillDefaultPinValueHelper::FillAll(this);
}

bool UK2Node_FunctionResult::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	bool bResult = Super::CanCreateUserDefinedPin(InPinType, InDesiredDirection, OutErrorMessage);
	if (bResult)
	{
		if(InDesiredDirection == EGPD_Output)
		{
			OutErrorMessage = NSLOCTEXT("K2Node", "AddOutputPinError", "Cannot add output pins to function result node!");
			bResult = false;
		}
	}
	return bResult;
}

UEdGraphPin* UK2Node_FunctionResult::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	UEdGraphPin* Pin = CreatePin(EGPD_Input, NewPinInfo->PinType, NewPinInfo->PinName);
	CastChecked<UEdGraphSchema_K2>(GetSchema())->SetPinAutogeneratedDefaultValue(Pin, NewPinInfo->PinDefaultValue);
	return Pin;
}

void UK2Node_FunctionResult::FixupPinStringDataReferences(FArchive* SavingArchive)
{
	Super::FixupPinStringDataReferences(SavingArchive);
	if (SavingArchive)
	{
		UpdateUserDefinedPinDefaultValues();
	}
}

ERenamePinResult UK2Node_FunctionResult::RenameUserDefinedPinImpl(FName OldName, FName NewName, bool bTest)
{
	ERenamePinResult Result = Super::RenameUserDefinedPinImpl(OldName, NewName, bTest);

	const bool bCheckLocalVariablesNameCollision =
		(Result == ERenamePinResult::ERenamePinResult_Success) &&
		bTest;
	
	if (bCheckLocalVariablesNameCollision)
	{
		if (UFunction* const Function = FindSignatureFunction())
		{
			const FProperty* ExistingProperty = FindFProperty<const FProperty>(Function, NewName);
			if (ExistingProperty)
			{
				Result = ERenamePinResult_NameCollision;
			}
		}
	}

	return Result;
}

FNodeHandlingFunctor* UK2Node_FunctionResult::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_FunctionResult(CompilerContext);
}

FText UK2Node_FunctionResult::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "ReturnNodeTooltip", "The node terminates the function's execution. It returns output parameters.");
}

void UK2Node_FunctionResult::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

bool UK2Node_FunctionResult::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(Graph ? Graph->GetSchema() : nullptr);
	const bool bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(Graph) : false;
	const bool bIsCompatible = (K2Schema != nullptr) ? (EGraphType::GT_Function == K2Schema->GetGraphType(Graph)) : false;
	return bIsCompatible && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
}

TArray<UK2Node_FunctionResult*> UK2Node_FunctionResult::GetAllResultNodes() const
{
	TArray<UK2Node_FunctionResult*> AllResultNodes;
	if (const UEdGraph* Graph = GetGraph())
	{
		Graph->GetNodesOfClass(AllResultNodes);
	}
	return AllResultNodes;
}

void UK2Node_FunctionResult::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// adhere to the function's inherited signature (if there is one)
	SyncWithEntryNode();
	// reflect any user added outputs (tracked by pre-existing result nodes)
	SyncWithPrimaryResultNode();
}

void UK2Node_FunctionResult::PostPasteNode()
{
	Super::PostPasteNode();

	// adhere to the function's inherited signature (if there is one)
	SyncWithEntryNode();
	// reflect any user added outputs (tracked by pre-existing result nodes)
	SyncWithPrimaryResultNode();
	// reflect editability of node in pins
	MakePinsEditable();
}

bool UK2Node_FunctionResult::CanUserDeleteNode() const
{
	bool bCanDelete = true;
	if (!bIsEditable)
	{
		if (UEdGraph* Graph = GetGraph())
		{
			bCanDelete = false;
			for (UEdGraphNode* Node : Graph->Nodes)
			{
				UK2Node_FunctionResult* ResultNode = Cast<UK2Node_FunctionResult>(Node);
				if (ResultNode && ResultNode != this)
				{
					bCanDelete = true;
					break;
				}
			}
		}
	}
	return bCanDelete;
}

void UK2Node_FunctionResult::SyncWithEntryNode()
{
	bool bWasSignatureMismatched = false;
	if (UEdGraph* Graph = GetGraph())
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			if (UK2Node_FunctionEntry* EntryNode = Cast<UK2Node_FunctionEntry>(Node))
			{
				bWasSignatureMismatched = !EntryNode->FunctionReference.IsSameReference(FunctionReference) || (!EntryNode->bIsEditable && UserDefinedPins.Num() > 0);

				// If the entry is editable, so is the result
				bIsEditable = EntryNode->bIsEditable;
				FunctionReference = EntryNode->FunctionReference;
				break;
			}
		}
	}

	if (bWasSignatureMismatched)
	{
		// to handle pasting of a result node from one function into another;
		// if the new function is not editable (like for one that is overidden), 
		// then we shouldn't have userdefined pins
		if (!bIsEditable)
		{
			// iterate backwards so we can remove items from the list as we go
			for (int32 UserPinIndex = UserDefinedPins.Num() - 1; UserPinIndex >= 0; --UserPinIndex)
			{
				RemoveUserDefinedPin(UserDefinedPins[UserPinIndex]);
			}
		}
		
		ReconstructNode();
	}
}

void UK2Node_FunctionResult::SyncWithPrimaryResultNode()
{
	const UK2Node_FunctionResult* PrimaryNode = nullptr;
	TArray<UK2Node_FunctionResult*> AllResultNodes = GetAllResultNodes();
	for (const UK2Node_FunctionResult* ResultNode : AllResultNodes)
	{
		if (ResultNode && (this != ResultNode))
		{
			PrimaryNode = ResultNode;
			break;
		}
	}

	if (PrimaryNode)
	{
		FunctionReference = PrimaryNode->FunctionReference;
		bIsEditable = PrimaryNode->bIsEditable;

		// Temporary array that will contain our list of Old Pins that are no longer part of the return signature
		TArray<TSharedPtr<FUserPinInfo>> OldPins = UserDefinedPins;

		// Temporary array that will contain our list of Signature Pins that need to be added
		TArray<TSharedPtr<FUserPinInfo>> SignaturePins = PrimaryNode->UserDefinedPins;

		for (int OldIndex = OldPins.Num() - 1; OldIndex >= 0; --OldIndex)
		{
			TSharedPtr<FUserPinInfo> OldPin = OldPins[OldIndex];

			if (!OldPin.IsValid())
			{
				OldPins.RemoveAt(OldIndex);
			}
			else
			{
				for (int SignatureIndex = SignaturePins.Num() - 1; SignatureIndex >= 0; --SignatureIndex)
				{
					TSharedPtr<FUserPinInfo> SignaturePin = SignaturePins[SignatureIndex];
					if (!SignaturePin.IsValid())
					{
						SignaturePins.RemoveAt(SignatureIndex);
					}
					else if (OldPin->PinName == SignaturePin->PinName &&
						OldPin->PinType == SignaturePin->PinType &&
						OldPin->DesiredPinDirection == SignaturePin->DesiredPinDirection)
					{
						// We have a match between our Signature pins and our Old Pins,
						// so we can leave the old pin as is by removing it from both temporary lists.
						OldPins.RemoveAt(OldIndex);
						SignaturePins.RemoveAt(SignatureIndex);
						break;
					}
				}
			}
		}

		// Remove old pins that are not part of the primary node signature
		for (TSharedPtr<FUserPinInfo> OldPinToRemove : OldPins)
		{
			RemoveUserDefinedPin(OldPinToRemove);
		}

		// Add pins that don't exist yet but are part of the primary node signature
		for (TSharedPtr<FUserPinInfo> SignaturePinToAdd : SignaturePins)
		{
			TSharedPtr<FUserPinInfo> NewPinInfo = MakeShareable(new FUserPinInfo());
			NewPinInfo->PinName = SignaturePinToAdd->PinName;
			NewPinInfo->PinType = SignaturePinToAdd->PinType;
			NewPinInfo->DesiredPinDirection = SignaturePinToAdd->DesiredPinDirection;
			UserDefinedPins.Add(NewPinInfo);
		}

		ReconstructNode();
	}
}

void UK2Node_FunctionResult::MakePinsEditable()
{
	// only do this step if this node is editable
	if (IsEditable())
	{
		// for each pin, excluding the 'exec' pin
		for (int PinIdx = 1; PinIdx < Pins.Num(); ++PinIdx)
		{
			UEdGraphPin* Pin = Pins[PinIdx];
			if (!UserDefinedPinExists(Pin->GetFName()))
			{
				UserDefinedPins.Add(MakeShared<FUserPinInfo>(*Pin));
			}
		}
	}
}

void UK2Node_FunctionResult::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	TArray<UK2Node_FunctionResult*> AllResultNodes = GetAllResultNodes();
	const UK2Node_FunctionResult* OtherResult = AllResultNodes.Num() ? AllResultNodes[0] : nullptr;
	if (OtherResult && (OtherResult != this))
	{
		for (UEdGraphPin* Pin : Pins)
		{
			check(Pin);
			UEdGraphPin* OtherPin = OtherResult->FindPin(Pin->PinName);
			if (!OtherPin || (OtherPin->PinType != Pin->PinType))
			{
				MessageLog.Error(*NSLOCTEXT("K2Node", "FunctionResult_DifferentReturnError", "Return nodes don't match each other: @@, @@").ToString(), this, OtherResult);
				break;
			}
		}
	}
}

void UK2Node_FunctionResult::PromoteFromInterfaceOverride(bool bIsPrimaryTerminator/* = true*/)
{
	// For non-primary terminators, we want to sync with the primary one and reconstruct.
	if (bIsPrimaryTerminator)
	{
		Super::PromoteFromInterfaceOverride();
	}
	else
	{
		FunctionReference.SetSelfMember(FunctionReference.GetMemberName());
		SyncWithPrimaryResultNode();
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		Schema->ReconstructNode(*this, true);
	}
}
