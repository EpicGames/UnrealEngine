// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_FunctionTerminator.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "FindInBlueprints.h"
#include "GraphEditorSettings.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_FunctionTerminator)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_FunctionTerminator::UK2Node_FunctionTerminator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_FunctionTerminator::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::FunctionTerminatorNodesUseMemberReference)
		{
			FunctionReference.SetExternalMember(SignatureName_DEPRECATED, SignatureClass_DEPRECATED);
		}
	}
}

FLinearColor UK2Node_FunctionTerminator::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->FunctionTerminatorNodeTitleColor;
}

FString UK2Node_FunctionTerminator::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	if (EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::UseSearchSyntax))
	{
		// Resolve the function
		if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
		{
			// Attempt to construct an advanced search syntax query from the function
			FString SearchTerm;
			if (FindInBlueprintsHelpers::ConstructSearchTermFromFunction(Function, SearchTerm))
			{
				return SearchTerm;
			}
			else
			{
				// Fallback behavior: function was found but failed to construct a search term from it
				// Just search for the function's friendly name
				return UEdGraphSchema_K2::GetFriendlySignatureName(Function).ToString();
			}
		}
	}
	else
	{
		// When searching by name, return function native name in quotes.
		// The quotes guarantee that the whole function name is used as single search term.
		// This avoids function names with special characters being interpreted as operators.
		if (const UFunction* Function = FFunctionFromNodeHelper::FunctionFromNode(this))
		{
			const FString NativeName = Function->GetName();
			return FString::Printf(TEXT("\"%s\""), *NativeName);
		}
	}

	// Fallback behavior: function was not resolved
	return Super::GetFindReferenceSearchString_Impl(InFlags);
}

FName UK2Node_FunctionTerminator::CreateUniquePinName(FName InSourcePinName) const
{
	const UFunction* FoundFunction = FFunctionFromNodeHelper::FunctionFromNode(this);

	FName ResultName = InSourcePinName;
	int UniqueNum = 0;
	// Prevent the unique name from being the same as another of the UFunction's properties
	while(FindPin(ResultName) || FindFProperty<const FProperty>(FoundFunction, ResultName) != nullptr)
	{
		ResultName = *FString::Printf(TEXT("%s%d"), *InSourcePinName.ToString(), ++UniqueNum);
	}
	return ResultName;
}

bool UK2Node_FunctionTerminator::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	const bool bIsNodeEditable = IsEditable();

	// Make sure that if this is an exec node we are allowed one.
	if (bIsNodeEditable && InPinType.PinCategory == UEdGraphSchema_K2::PC_Exec && !CanModifyExecutionWires())
	{
		OutErrorMessage = LOCTEXT("MultipleExecPinError", "Cannot support more exec pins!");
		return false;
	}
	else if (!bIsNodeEditable)
	{
		OutErrorMessage = LOCTEXT("NotEditableError", "Cannot edit this node!");
	}

	return bIsNodeEditable;
}

bool UK2Node_FunctionTerminator::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	UClass* SourceClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());
	bool bResult = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}

	// All structures, that are required for the BP compilation, should be gathered
	for (auto Pin : Pins)
	{
		UStruct* DepStruct = Pin ? Cast<UStruct>(Pin->PinType.PinSubCategoryObject.Get()) : nullptr;

		UClass* DepClass = Cast<UClass>(DepStruct);
		if (DepClass && (DepClass->ClassGeneratedBy.Get() == SourceBlueprint))
		{
			//Don't include self
			continue;
		}

		if (DepStruct && !DepStruct->IsNative())
		{
			if (OptionalOutput)
			{
				OptionalOutput->AddUnique(DepStruct);
			}
			bResult = true;
		}
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_FunctionTerminator::PostPasteNode()
{
	Super::PostPasteNode();

	UEdGraph* Graph = GetGraph();
	if (ensure(Graph))
	{
		FunctionReference.SetExternalMember(Graph->GetFName(), nullptr);
	}
}

void UK2Node_FunctionTerminator::PromoteFromInterfaceOverride(bool bIsPrimaryTerminator)
{
	// Remove the signature class, that is not relevant.
	FunctionReference.SetSelfMember(FunctionReference.GetMemberName());
	
	// For every pin that has been defined, make it a user defined pin if we can
	for (UEdGraphPin* const Pin : Pins)
	{
		if (Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec && !UserDefinedPinExists(Pin->PinName))
		{
			TSharedPtr<FUserPinInfo> NewPinInfo = MakeShareable(new FUserPinInfo());
			NewPinInfo->PinName = Pin->PinName;
			NewPinInfo->PinType = Pin->PinType;
			NewPinInfo->DesiredPinDirection = Pin->Direction;
			UserDefinedPins.Add(NewPinInfo);
		}
	}
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	Schema->ReconstructNode(*this, true);
}

UFunction* UK2Node_FunctionTerminator::FindSignatureFunction() const
{
	UClass* FoundClass = GetBlueprintClassFromNode();
	UFunction* FoundFunction = FunctionReference.ResolveMember<UFunction>(FoundClass);

	if (!FoundFunction && FoundClass && GetOuter())
	{
		// The resolve will fail if this is a locally-created function, so search using the event graph name
		FoundFunction = FindUField<UFunction>(FoundClass, *GetOuter()->GetName());
	}

	return FoundFunction;
}

void UK2Node_FunctionTerminator::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->PinType.bIsWeakPointer && !Pin->PinType.IsContainer())
		{
			const FString ErrorString = FText::Format(
				LOCTEXT("WeakPtrNotSupportedErrorFmt", "Weak pointers are not supported as function parameters. Pin '{0}' @@"),
				FText::FromString(Pin->GetName())
			).ToString();
			MessageLog.Error(*ErrorString, this);
		}
	}
}

#undef LOCTEXT_NAMESPACE
