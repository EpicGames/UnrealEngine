// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GeneratedBoundEvent.h"

#include "Containers/Array.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/GeneratedBlueprintDelegateBinding.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GeneratedBoundEvent)

#define LOCTEXT_NAMESPACE "K2Node"

FText UK2Node_GeneratedBoundEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText::FromName(CustomFunctionName);
}

void UK2Node_GeneratedBoundEvent::ReconstructNode()
{
	// We need to fixup our event reference as it may have changed or been redirected
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();

	// If we couldn't find the target delegate, then try to find it in the property remap table
	if (!TargetDelegateProp)
	{
		FMulticastDelegateProperty* NewProperty = FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
		if (NewProperty)
		{
			// Found a remapped property, update the node
			TargetDelegateProp = NewProperty;
			DelegatePropertyName = NewProperty->GetFName();
		}
	}

	if (TargetDelegateProp && TargetDelegateProp->SignatureFunction)
	{
		EventReference.SetFromField<UFunction>(TargetDelegateProp->SignatureFunction, false);
	}

	Super::ReconstructNode();
}

void UK2Node_GeneratedBoundEvent::InitializeGeneratedBoundEventParams(const FMulticastDelegateProperty* InDelegateProperty)
{
	if (InDelegateProperty)
	{
		DelegatePropertyName = InDelegateProperty->GetFName();
		DelegateOwnerClass = CastChecked<UClass>(InDelegateProperty->GetOwner<UObject>())->GetAuthoritativeClass();

		EventReference.SetFromField<UFunction>(InDelegateProperty->SignatureFunction, /* bIsConsideredSelfContext */ false);

		CustomFunctionName = EventReference.GetMemberName();
		bOverrideFunction = false;
		bInternalEvent = true;
	}
}

UClass* UK2Node_GeneratedBoundEvent::GetDynamicBindingClass() const
{
	return UGeneratedBlueprintBinding::StaticClass();
}

void UK2Node_GeneratedBoundEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UGeneratedBlueprintBinding* GeneratedBindingObject = CastChecked<UGeneratedBlueprintBinding>(BindingObject);

	FGeneratedBlueprintDelegateBinding Binding;
	Binding.DelegatePropertyName = DelegatePropertyName;
	Binding.FunctionNameToBind = CustomFunctionName;

	GeneratedBindingObject->GeneratedBlueprintBindings.Add(Binding);
}

void UK2Node_GeneratedBoundEvent::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	// Note: We skip this because our node's data is generated after the regular node validation
	Super::ValidateNodeDuringCompilation(MessageLog);
}

bool UK2Node_GeneratedBoundEvent::IsUsedByAuthorityOnlyDelegate() const
{
	FMulticastDelegateProperty* TargetDelegateProp = GetTargetDelegateProperty();
	return (TargetDelegateProp && TargetDelegateProp->HasAnyPropertyFlags(CPF_BlueprintAuthorityOnly));
}

FMulticastDelegateProperty* UK2Node_GeneratedBoundEvent::GetTargetDelegateProperty() const
{
	return FindFProperty<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName);
}

FText UK2Node_GeneratedBoundEvent::GetTargetDelegateDisplayName() const
{
	FMulticastDelegateProperty* Prop = GetTargetDelegateProperty();
	return Prop ? Prop->GetDisplayNameText() : FText::FromName(DelegatePropertyName);
}

bool UK2Node_GeneratedBoundEvent::IsDelegateValid() const
{
	const UBlueprint* const BP = GetBlueprint();
	// Validate that the property has not been renamed or deleted via the SCS tree
	return BP
		// Validate that the actual declaration for this event has not been deleted 
		// either from a native base class or a BP multicast delegate. The Delegate could have been 
		// renamed/redirected, so also check for a remapped field if we need to
		&& (GetTargetDelegateProperty() || FMemberReference::FindRemappedField<FMulticastDelegateProperty>(DelegateOwnerClass, DelegatePropertyName));
}


#undef LOCTEXT_NAMESPACE
