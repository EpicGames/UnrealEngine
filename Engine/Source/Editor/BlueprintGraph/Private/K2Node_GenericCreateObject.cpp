// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GenericCreateObject.h"

#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Self.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectBaseUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GenericCreateObject)

#define LOCTEXT_NAMESPACE "K2Node_GenericCreateObject"

struct FK2Node_GenericCreateObject_Utils
{
	static bool CanSpawnObjectOfClass(TSubclassOf<UObject> ObjectClass, bool bAllowAbstract)
	{
		// Initially include types that meet the basic requirements.
		// Note: CLASS_Deprecated is an inherited class flag, so any subclass of an explicitly-deprecated class also cannot be spawned.
		bool bCanSpawnObject = (nullptr != *ObjectClass)
			&& (bAllowAbstract || !ObjectClass->HasAnyClassFlags(CLASS_Abstract))
			&& !ObjectClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists);

		// UObject is a special case where if we are allowing abstract we are going to allow it through even though it doesn't have BlueprintType on it
		if (bCanSpawnObject && (!bAllowAbstract || (*ObjectClass != UObject::StaticClass())))
		{
			static const FName BlueprintTypeName(TEXT("BlueprintType"));
			static const FName NotBlueprintTypeName(TEXT("NotBlueprintType"));
			static const FName DontUseGenericSpawnObjectName(TEXT("DontUseGenericSpawnObject"));

			auto IsClassAllowedLambda = [](const UClass* InClass)
			{
				return InClass != AActor::StaticClass()
					&& InClass != UActorComponent::StaticClass();
			};

			// Exclude all types in the initial set by default.
			bCanSpawnObject = false;
			const UClass* CurrentClass = ObjectClass;

			// Climb up the class hierarchy and look for "BlueprintType." If "NotBlueprintType" is seen first, or if the class is not allowed, then stop searching.
			while (!bCanSpawnObject && CurrentClass != nullptr && !CurrentClass->GetBoolMetaData(NotBlueprintTypeName) && IsClassAllowedLambda(CurrentClass))
			{
				// Include any type that either includes or inherits 'BlueprintType'
				bCanSpawnObject = CurrentClass->GetBoolMetaData(BlueprintTypeName);

				// Stop searching if we encounter 'BlueprintType' with 'DontUseGenericSpawnObject'
				if (bCanSpawnObject && CurrentClass->GetBoolMetaData(DontUseGenericSpawnObjectName))
				{
					bCanSpawnObject = false;
					break;
				}

				CurrentClass = CurrentClass->GetSuperClass();
			}

			// If we validated the given class, continue walking up the hierarchy to make sure we exclude it if it's an Actor or ActorComponent derivative.
			while (bCanSpawnObject && CurrentClass != nullptr)
			{
				bCanSpawnObject &= IsClassAllowedLambda(CurrentClass);

				CurrentClass = CurrentClass->GetSuperClass();
			}
		}

		return bCanSpawnObject;
	}
};

void UK2Node_GenericCreateObject::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UK2Node_CallFunction* CallCreateNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallCreateNode->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UGameplayStatics, SpawnObject), UGameplayStatics::StaticClass());
	CallCreateNode->AllocateDefaultPins();
	
	// connect GenericCreateObject's self pin to the self node
	bool bSucceeded = ExpandDefaultToSelfPin(CompilerContext, SourceGraph, CallCreateNode);

	// store off the class to spawn before we mutate pin connections:
	UClass* ClassToSpawn = GetClassToSpawn();

	//connect exe
	{
		UEdGraphPin* SpawnExecPin = GetExecPin();
		UEdGraphPin* CallExecPin = CallCreateNode->GetExecPin();
		bSucceeded &= SpawnExecPin && CallExecPin && CompilerContext.MovePinLinksToIntermediate(*SpawnExecPin, *CallExecPin).CanSafeConnect();
	}

	UEdGraphPin* CallClassPin = nullptr;
	//connect class
	{
		UEdGraphPin* SpawnClassPin = GetClassPin();
		CallClassPin = CallCreateNode->FindPin(TEXT("ObjectClass"));
		bSucceeded &= SpawnClassPin && CallClassPin && CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallClassPin).CanSafeConnect();
	}
		
	//connect outer
	{
		UEdGraphPin* SpawnOuterPin = GetOuterPin();
		UEdGraphPin* CallOuterPin = CallCreateNode->FindPin(TEXT("Outer"));
		bSucceeded &= SpawnOuterPin && CallOuterPin && CompilerContext.MovePinLinksToIntermediate(*SpawnOuterPin, *CallOuterPin).CanSafeConnect();
	}

	UEdGraphPin* CallResultPin = nullptr;
	//connect result
	{
		UEdGraphPin* SpawnResultPin = GetResultPin();
		CallResultPin = CallCreateNode->GetReturnValuePin();

		// cast HACK. It should be safe. The only problem is native code generation.
		if (SpawnResultPin && CallResultPin)
		{
			CallResultPin->PinType = SpawnResultPin->PinType;
		}
		bSucceeded &= SpawnResultPin && CallResultPin && CompilerContext.MovePinLinksToIntermediate(*SpawnResultPin, *CallResultPin).CanSafeConnect();
	}

	//assign exposed values and connect then
	{
		UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes(CompilerContext, SourceGraph, CallCreateNode, this, CallResultPin, ClassToSpawn, CallClassPin);
		UEdGraphPin* SpawnNodeThen = GetThenPin();
		bSucceeded &= SpawnNodeThen && LastThen && CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *LastThen).CanSafeConnect();
	}

	BreakAllNodeLinks();

	if (!bSucceeded)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("GenericCreateObject_Error", "ICE: GenericCreateObject error @@").ToString(), this);
	}
}

void UK2Node_GenericCreateObject::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);
	UEdGraphPin* ClassPin = GetClassPin(&Pins);
	const bool bAllowAbstract = ClassPin && ClassPin->LinkedTo.Num();
	UClass* ClassToSpawn = GetClassToSpawn();
	if (!FK2Node_GenericCreateObject_Utils::CanSpawnObjectOfClass(ClassToSpawn, bAllowAbstract))
	{
		MessageLog.Error(*FText::Format(LOCTEXT("GenericCreateObject_WrongClassFmt", "Cannot construct objects of type '{0}' in @@"), FText::FromString(GetPathNameSafe(ClassToSpawn))).ToString(), this);
	}
}

bool UK2Node_GenericCreateObject::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	return UK2Node::IsCompatibleWithGraph(TargetGraph);
}

bool UK2Node_GenericCreateObject::ExpandDefaultToSelfPin(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UK2Node_CallFunction* CallCreateNode)
{
	// ensure valid input
	if (!SourceGraph || !CallCreateNode)
	{
		return false;
	}

	// Connect a self reference pin if there is a TScriptInterface default to self
	if (const UFunction* TargetFunc = CallCreateNode->GetTargetFunction())
	{
		const FString& MetaData = TargetFunc->GetMetaData(FBlueprintMetadata::MD_DefaultToSelf);
		if (!MetaData.IsEmpty())
		{
			// Get the Self Pin from this so we can default it correctly
			const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
			if (UEdGraphPin* DefaultToSelfPin = Schema->FindSelfPin(*this, EGPD_Input))
			{
				// If it has no links then spawn a new self node here
				if (DefaultToSelfPin->LinkedTo.IsEmpty())
				{
					UK2Node_Self* SelfNode = CompilerContext.SpawnIntermediateNode<UK2Node_Self>(this, SourceGraph);
					SelfNode->AllocateDefaultPins();
					UEdGraphPin* SelfPin = SelfNode->FindPinChecked(UEdGraphSchema_K2::PSC_Self);
					// Make a connection from this intermediate self pin to here
					return Schema->TryCreateConnection(DefaultToSelfPin, SelfPin);
				}
			}
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
