// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GenericToText.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetTextLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GenericToText)

#define LOCTEXT_NAMESPACE "K2Node_GenericToText"

/////////////////////////////////////////////////////
// K2Node_GenericConvToText

namespace UE::Blueprint::Private
{
	const FLazyName PC_Value = "Value";
	const TArrayView<const UScriptStruct*> GetToTextSupportedScriptStruct()
	{
		static const UScriptStruct* List[] = { TBaseStructure<FVector>::Get()
			, TBaseStructure<FVector2D>::Get()
			, TBaseStructure<FRotator>::Get()
			, TBaseStructure<FTransform>::Get()
			, TBaseStructure<FLinearColor>::Get()
			, TBaseStructure<FDateTime>::Get()
		};
		return MakeArrayView(List);
	}

	bool IsGenericNumericProperty(FName OtherPinCategory)
	{
		return OtherPinCategory == UEdGraphSchema_K2::PC_Boolean
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Byte
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Enum
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Int
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Int64
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Real
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Double
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Float;
	}
}

void UK2Node_GenericToText::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, UE::Blueprint::Private::PC_Value.Resolve());
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Text, UEdGraphSchema_K2::PN_ReturnValue);
}

void UK2Node_GenericToText::SynchronizeArgumentPinType()
{
	bool bPinTypeChanged = false;
	UEdGraphPin* InputPin = FindPinChecked(UE::Blueprint::Private::PC_Value.Resolve(), EEdGraphPinDirection::EGPD_Input);
	if (InputPin->LinkedTo.Num() == 0)
	{
		static const FEdGraphPinType WildcardPinType = FEdGraphPinType(UEdGraphSchema_K2::PC_Wildcard, NAME_None, nullptr, EPinContainerType::None, false, FEdGraphTerminalType());

		// Ensure wildcard
		if (InputPin->PinType != WildcardPinType)
		{
			InputPin->PinType = WildcardPinType;
			bPinTypeChanged = true;
		}
	}
	else
	{
		UEdGraphPin* SourcePin = InputPin->LinkedTo[0];

		// Take the type of the connected pin
		if (InputPin->PinType != SourcePin->PinType)
		{
			InputPin->PinType = SourcePin->PinType;
			bPinTypeChanged = true;
		}
	}

	if (bPinTypeChanged)
	{
		// Let the graph know to refresh
		GetGraph()->NotifyNodeChanged(this);

		UBlueprint* Blueprint = GetBlueprint();
		if (!Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

FText UK2Node_GenericToText::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("ToText_Title", "To Text");
}

void UK2Node_GenericToText::PinConnectionListChanged(UEdGraphPin* Pin)
{
	Modify();

	// Potentially update an argument pin type
	SynchronizeArgumentPinType();
}

void UK2Node_GenericToText::PinTypeChanged(UEdGraphPin* Pin)
{
	SynchronizeArgumentPinType();

	Super::PinTypeChanged(Pin);
}

void UK2Node_GenericToText::NodeConnectionListChanged()
{
	SynchronizeArgumentPinType();

	Super::NodeConnectionListChanged();
}

FText UK2Node_GenericToText::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Converts numeric value, enum, string, name and some basic structure to text.\n  \u2022 The basic type are Vector, Vector2D, Rotator, Transform, LinearColor, DateTime.");
}

void UK2Node_GenericToText::PostReconstructNode()
{
	Super::PostReconstructNode();

	if (!IsTemplate())
	{
		// Make sure we're not dealing with a menu node
		UEdGraph* OuterGraph = GetGraph();
		if (OuterGraph && OuterGraph->Schema)
		{
			// Potentially update an argument pin type
			SynchronizeArgumentPinType();
		}
	}
}

void UK2Node_GenericToText::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* InputPin = FindPinChecked(UE::Blueprint::Private::PC_Value.Resolve(), EEdGraphPinDirection::EGPD_Input);
	if (InputPin == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("Error_BadInputPin", "Invalid input pin.").ToString());
		return;
	}

	if (InputPin->LinkedTo.Num() != 1 || InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("Error_BadLinkedInputPin", "Input pin is not linked to a value.").ToString());
		return;
	}

	// Convert to the correct function call
	auto CreateFunctionCall = [Self = this, &CompilerContext, SourceGraph](FName FunctionName)
	{
		UK2Node_CallFunction* CallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(Self, SourceGraph);
		CallFunction->SetFromFunction(UKismetTextLibrary::StaticClass()->FindFunctionByName(FunctionName));
		check(CallFunction->IsNodePure());
		CallFunction->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(CallFunction, Self);
		return CallFunction;
	};

	UK2Node_CallFunction* CallFunction = nullptr;
	UEdGraphPin* CallFunctionInputValue = nullptr;
	if (UE::Blueprint::Private::IsGenericNumericProperty(InputPin->PinType.PinCategory))
	{
		CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_NumericPropertyToText));
		CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("Value"));
		// Set the generic type
		CallFunctionInputValue->PinType = InputPin->PinType;
	}
	else if (InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_String)
	{
		CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_StringToText));
		CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InString"));
	}
	else if (InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name)
	{
		CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_NameToText));
		CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InName"));
	}
	else if (InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		const UObject* PinSubCategoryObject = InputPin->PinType.PinSubCategoryObject.Get();
		if (PinSubCategoryObject == TBaseStructure<FVector>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_VectorToText));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InVec"));
		}
		else if (PinSubCategoryObject == TBaseStructure<FVector2D>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_Vector2dToText));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InVec"));
		}
		else if (PinSubCategoryObject == TBaseStructure<FRotator>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_RotatorToText));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InRot"));
		}
		else if (PinSubCategoryObject == TBaseStructure<FTransform>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_TransformToText));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InTrans"));
		}
		else if (PinSubCategoryObject == TBaseStructure<FLinearColor>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, Conv_ColorToText));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("InColor"));
		}
		else if (PinSubCategoryObject == TBaseStructure<FDateTime>::Get())
		{
			CallFunction = CreateFunctionCall(GET_MEMBER_NAME_CHECKED(UKismetTextLibrary, AsDateTime_DateTime));
			CallFunctionInputValue = CallFunction->FindPinChecked(TEXT("In"));
		}
		else
		{
			check(!UE::Blueprint::Private::GetToTextSupportedScriptStruct().Contains(Cast<UScriptStruct>(PinSubCategoryObject)));
		}
	}

	if (CallFunction == nullptr || CallFunctionInputValue == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("Error_BadPinCategory", "The input pin could not generate a valid ToText.").ToString());
		return;
	}

	// Move connection of ToText's to the generated function.
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue), *CallFunction->GetReturnValuePin());
	CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(TEXT("Value")), *CallFunctionInputValue);

	BreakAllNodeLinks();
}

bool UK2Node_GenericToText::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	UEdGraphPin* InputPin = FindPinChecked(UE::Blueprint::Private::PC_Value.Resolve(), EEdGraphPinDirection::EGPD_Input);
	if (InputPin == MyPin && MyPin->Direction == EGPD_Input)
	{
		const FName OtherPinCategory = OtherPin->PinType.PinCategory;	
		bool bIsValidType = UE::Blueprint::Private::IsGenericNumericProperty(OtherPinCategory)
			|| OtherPinCategory == UEdGraphSchema_K2::PC_String
			|| OtherPinCategory == UEdGraphSchema_K2::PC_Name;

		if (!bIsValidType)
		{
			const UObject* PinSubCategoryObject = OtherPin->PinType.PinSubCategoryObject.Get();
			const bool bIsScriptStruct = OtherPinCategory == UEdGraphSchema_K2::PC_Struct
				&& UE::Blueprint::Private::GetToTextSupportedScriptStruct().Contains(Cast<UScriptStruct>(PinSubCategoryObject));

			bIsValidType = bIsScriptStruct;
		}

		if (!bIsValidType)
		{
			OutReason = LOCTEXT("Error_InvalidArgumentType", "To Text arguments may only be Byte, Integer, Int64, Float, Double, Text, String, Name, Boolean, Enum, and basic struct.").ToString();
			return true;
		}
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_GenericToText::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GenericToText::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Text);
}

#undef LOCTEXT_NAMESPACE
