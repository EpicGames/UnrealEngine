// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GetArrayItem.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetArrayLibrary.h" // for Array_Get()
#include "Kismet2/BlueprintEditorUtils.h" // for MarkBlueprintAsModified()
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h" // for FKismetCompilerContext
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "SPinTypeSelector.h"
#include "ScopedTransaction.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Notifications/SNotificationList.h" // for FNotificationInfo

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetArrayItem)

class SWidget;
struct FLinearColor;


#define LOCTEXT_NAMESPACE "GetArrayItem"

/*******************************************************************************
*  FKCHandler_GetArrayItem
******************************************************************************/
class FKCHandler_GetArrayItem : public FNodeHandlingFunctor
{
public:
	FKCHandler_GetArrayItem(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_GetArrayItem* ArrayNode = CastChecked<UK2Node_GetArrayItem>(Node);

		// return inline term
		if (Context.NetMap.Find(Node->Pins[2]))
		{
			Context.MessageLog.Error(*LOCTEXT("Error_ReturnTermAlreadyRegistered", "ICE: Return term is already registered @@").ToString(), Node);
			return;
		}

		{
			FBPTerminal* Term = new FBPTerminal();
			Context.InlineGeneratedValues.Add(Term);
			Term->CopyFromPin(Node->Pins[2], Context.NetNameMap->MakeValidName(Node->Pins[2]));
			Context.NetMap.Add(Node->Pins[2], Term);
		}

		FNodeHandlingFunctor::RegisterNets(Context, Node);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_GetArrayItem* ArrayNode = CastChecked<UK2Node_GetArrayItem>(Node);

		FBlueprintCompiledStatement* ArrayGetFunction = new FBlueprintCompiledStatement();
		ArrayGetFunction->Type = KCST_ArrayGetByRef;

		UEdGraphPin* ArrayPinNet = FEdGraphUtilities::GetNetFromPin(Node->Pins[0]);
		UEdGraphPin* ReturnValueNet = FEdGraphUtilities::GetNetFromPin(Node->Pins[2]);

		FBPTerminal** ArrayTerm = Context.NetMap.Find(ArrayPinNet);

		UEdGraphPin* IndexPin = ArrayNode->GetIndexPin();
		check(IndexPin);
		UEdGraphPin* IndexPinNet = FEdGraphUtilities::GetNetFromPin(IndexPin);
		FBPTerminal** IndexTermPtr = Context.NetMap.Find(IndexPinNet);

		FBPTerminal** ReturnValue = Context.NetMap.Find(ReturnValueNet);
		FBPTerminal** ReturnValueOrig = Context.NetMap.Find(Node->Pins[2]);
		ArrayGetFunction->RHS.Add(*ArrayTerm);
		ArrayGetFunction->RHS.Add(*IndexTermPtr);

		(*ReturnValue)->InlineGeneratedParameter = ArrayGetFunction;
	}
};

/*******************************************************************************
 *  UK2Node_GetArrayItem
 ******************************************************************************/

namespace K2Node_GetArrayItem_Impl
{
	static bool SupportsReturnByRef(const FEdGraphPinType& PinType)
	{
		return !(PinType.PinCategory == UEdGraphSchema_K2::PC_Object ||
			PinType.PinCategory == UEdGraphSchema_K2::PC_Class ||
			PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject ||
			PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass ||
			PinType.PinCategory == UEdGraphSchema_K2::PC_Interface);
	}

	static bool SupportsReturnByRef(const UK2Node_GetArrayItem* Node)
	{
		return (Node->Pins.Num() == 0) || SupportsReturnByRef(Node->GetTargetArrayPin()->PinType);
	}

	static FText GetToggleTooltip(bool bIsOutputRef)
	{
		return bIsOutputRef ? LOCTEXT("ConvToValTooltip", "Changing this node to return a copy will make it so it returns a temporary duplicate of the item in the array (changes to this item will NOT be propagated back to the array)") :
			LOCTEXT("ConvToRefTooltip", "Changing this node to return by reference will make it so it returns the same item that's in the array (meaning you can operate directly on that item, and changes will be reflected in the array)");
	}
}

UK2Node_GetArrayItem::UK2Node_GetArrayItem(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bReturnByRefDesired(true)
{
}

void UK2Node_GetArrayItem::AllocateDefaultPins()
{
	// Create the input pins to create the arrays from
	UEdGraphNode::FCreatePinParams ArrayPinParams;
	ArrayPinParams.ContainerType = EPinContainerType::Array;
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, TEXT("Array"), ArrayPinParams);
	UEdGraphPin* IndexPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, TEXT("Dimension 1"));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(IndexPin);

	// Create the output pin
	UEdGraphNode::FCreatePinParams OutputPinParams;
	OutputPinParams.bIsReference = bReturnByRefDesired;
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, TEXT("Output"), OutputPinParams);
}

void UK2Node_GetArrayItem::PostReconstructNode()
{
	UEdGraphPin* ArrayPin = GetTargetArrayPin();
	if (ArrayPin->LinkedTo.Num() > 0 && ArrayPin->LinkedTo[0]->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard)
	{
		PropagatePinType(ArrayPin->LinkedTo[0]->PinType);
	}
	else
	{
		UEdGraphPin* ResultPin = GetResultPin();
		if (ResultPin->LinkedTo.Num() > 0)
		{
			PropagatePinType(ResultPin->LinkedTo[0]->PinType);
		}
	}
}

FText UK2Node_GetArrayItem::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::FullTitle)
	{
		return LOCTEXT("GetArrayItemByRef_FullTitle", "GET");
	}
	return IsSetToReturnRef() ? LOCTEXT("GetArrayItemByRef", "Get (a ref)") : LOCTEXT("GetArrayItemByVal", "Get (a copy)");
}

FText UK2Node_GetArrayItem::GetTooltipText() const
{
	return IsSetToReturnRef() 
		? LOCTEXT("RetRefTooltip", "Given an array and an index, returns the item in the array at that index (since\nit's a direct reference, you can operate directly on the item and changes made\nto it will be reflected back in the array)")
		: LOCTEXT("RetValTooltip", "Given an array and an index, returns a temporary copy of the item in the array\nat that index (since it's a copy, changes to this item will NOT be propagated\nback to the array)");
}

class FNodeHandlingFunctor* UK2Node_GetArrayItem::CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_GetArrayItem(CompilerContext);
}

TSharedPtr<SWidget> UK2Node_GetArrayItem::CreateNodeImage() const
{
	return SPinTypeSelector::ConstructPinTypeImage(GetTargetArrayPin());
}

void UK2Node_GetArrayItem::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	const bool bReturnIsRef = IsSetToReturnRef();
	FText ToggleTooltip = K2Node_GetArrayItem_Impl::GetToggleTooltip(bReturnIsRef);

	const bool bCannotReturnRef = !bReturnIsRef && bReturnByRefDesired;
	if (bCannotReturnRef)
	{
		UEdGraphPin* OutputPin = GetResultPin();
		ToggleTooltip = FText::Format(LOCTEXT("CannotToggleTooltip", "Cannot return by ref using '{0}' pins"), UEdGraphSchema_K2::TypeToText(OutputPin->PinType));
	}

	{
		FToolMenuSection& Section = Menu->AddSection("Array", LOCTEXT("ArrayHeader", "Array Get Node"));
		Section.AddMenuEntry(
			"ToggleReturnPin",
			bReturnIsRef ? LOCTEXT("ChangeNodeToRef", "Change to return a copy") : LOCTEXT("ChangeNodeToVal", "Change to return a reference"),
			ToggleTooltip,
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateUObject(const_cast<UK2Node_GetArrayItem*>(this), &UK2Node_GetArrayItem::ToggleReturnPin),
				FCanExecuteAction::CreateLambda([bCannotReturnRef]()->bool { return !bCannotReturnRef; }))
		);
	}
}

FSlateIcon UK2Node_GetArrayItem::GetIconAndTint(FLinearColor& OutColor) const
{
	// emulate the icon/color that we used when this was a UK2Node_CallArrayFunction node
	if (UFunction* WrappedFunction = FindUField<UFunction>(UKismetArrayLibrary::StaticClass(), GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get)))
	{
		return UK2Node_CallFunction::GetPaletteIconForFunction(WrappedFunction, OutColor);
	}
	return Super::GetIconAndTint(OutColor);
}

void UK2Node_GetArrayItem::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* RetRefNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(RetRefNodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(ActionKey, RetRefNodeSpawner);

		UBlueprintNodeSpawner* RetValNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		auto CustomizeToReturnByVal = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
		{
			UK2Node_GetArrayItem* ArrayGetNode = CastChecked<UK2Node_GetArrayItem>(NewNode);
			ArrayGetNode->SetDesiredReturnType(/*bAsReference =*/false);
		};
		check(RetValNodeSpawner != nullptr);
		RetValNodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeToReturnByVal);
		ActionRegistrar.AddBlueprintAction(ActionKey, RetValNodeSpawner);
	}
}

FBlueprintNodeSignature UK2Node_GetArrayItem::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();

	static const FName NodeRetByRefKey(TEXT("ReturnByRef"));
	NodeSignature.AddNamedValue(NodeRetByRefKey, IsSetToReturnRef() ? TEXT("true") : TEXT("false"));

	return NodeSignature;
}

FText UK2Node_GetArrayItem::GetMenuCategory() const
{
	return LOCTEXT("ArrayUtilitiesCategory", "Utilities|Array");;
}

void UK2Node_GetArrayItem::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	if (Pin != GetIndexPin() && Pin->ParentPin == nullptr)
	{
		UEdGraphPin* ArrayPin = GetTargetArrayPin();

		const bool bConnectionAdded = Pin->LinkedTo.Num() > 0;
		if (bConnectionAdded)
		{
			const bool bIsWildcard = (ArrayPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard);
			if (bIsWildcard)
			{
				FEdGraphPinType& NewType = Pin->LinkedTo[0]->PinType;
				if (NewType.PinCategory != UEdGraphSchema_K2::PC_Wildcard)
				{
					PropagatePinType(NewType);
					GetGraph()->NotifyNodeChanged(this);
				}
			}
		}
		else
		{
			const bool bIsArrayPin = (Pin == ArrayPin);
			UEdGraphPin* ResultPin = GetResultPin();
			UEdGraphPin* OtherPin  = bIsArrayPin ? ResultPin : ArrayPin;

			if (OtherPin->LinkedTo.Num() == 0)
			{
				ArrayPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				ArrayPin->PinType.PinSubCategory = NAME_None;
				ArrayPin->PinType.PinSubCategoryObject = nullptr;

				ResultPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				ResultPin->PinType.PinSubCategory = NAME_None;
				ResultPin->PinType.PinSubCategoryObject = nullptr;
				ResultPin->PinType.bIsReference = bReturnByRefDesired;

				ArrayPin->BreakAllPinLinks();
				ResultPin->BreakAllPinLinks();
			}
		}
	}
}

void UK2Node_GetArrayItem::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* SrcReturnPin = GetResultPin();
	if (!SrcReturnPin->PinType.bIsReference)
	{
		UK2Node_CallArrayFunction* CallArrayLibFunc = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
		CallArrayLibFunc->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetArrayLibrary, Array_Get), UKismetArrayLibrary::StaticClass());
		CallArrayLibFunc->AllocateDefaultPins();

		UEdGraphPin* SrcArrayPin = GetTargetArrayPin();
		UEdGraphPin* DstArrayPin = CallArrayLibFunc->GetTargetArrayPin();
		CallArrayLibFunc->PropagateArrayTypeInfo(SrcArrayPin);
		CompilerContext.MovePinLinksToIntermediate(*SrcArrayPin, *DstArrayPin);

		UEdGraphPin* SrcIndexPin = GetIndexPin();
		for (UEdGraphPin* DestPin : CallArrayLibFunc->Pins)
		{
			if (DestPin->Direction == EEdGraphPinDirection::EGPD_Output)
			{
				CompilerContext.MovePinLinksToIntermediate(*SrcReturnPin, *DestPin);
			}
			else if (DestPin != DstArrayPin && DestPin->PinName != UEdGraphSchema_K2::PN_Self)
			{
				DestPin->DefaultValue = SrcIndexPin->DefaultValue;
				CompilerContext.MovePinLinksToIntermediate(*SrcIndexPin, *DestPin);
			}
		}
	}
}

bool UK2Node_GetArrayItem::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	for (UEdGraphPin* Pin : Filter.Context.Pins)
	{
		if (bReturnByRefDesired && !K2Node_GetArrayItem_Impl::SupportsReturnByRef(Pin->PinType))
		{
			bIsFilteredOut = true;
			break;
		}
	}
	return bIsFilteredOut;
}

bool UK2Node_GetArrayItem::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (MyPin != GetIndexPin())
	{
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec)
		{
			OutReason = LOCTEXT("NoExecWarning", "Cannot have an array of execution pins.").ToString();
			return true;
		}
		else if (IsSetToReturnRef() && !K2Node_GetArrayItem_Impl::SupportsReturnByRef(OtherPin->PinType))
		{
			OutReason = LOCTEXT("ConnectionWillChangeNodeToVal", "Change the Get node to return a copy").ToString();
			// the connection is allowed, it will just change how the node behaves
			return false;
		}
	}
	return false;
}

void UK2Node_GetArrayItem::SetDesiredReturnType(bool bAsReference)
{
	if (bReturnByRefDesired != bAsReference)
	{
		bReturnByRefDesired = bAsReference;

		const bool bReconstruct = (Pins.Num() > 0) && (IsSetToReturnRef() == bAsReference);
		if (bReconstruct)
		{
			ReconstructNode();
			FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
		}
	}
}

void UK2Node_GetArrayItem::ToggleReturnPin()
{
	FText TransactionTitle;
	if (bReturnByRefDesired)
	{
		TransactionTitle = LOCTEXT("ToggleToVal", "Change to return a copy");
	}
	else
	{
		TransactionTitle = LOCTEXT("ToggleToRef", "Change to return a reference");
	}
	const FScopedTransaction Transaction(TransactionTitle);
	Modify();

	SetDesiredReturnType(!bReturnByRefDesired);
}

void UK2Node_GetArrayItem::PropagatePinType(FEdGraphPinType& InType)
{
	UBlueprint const* Blueprint = GetBlueprint();

	UClass const* CallingContext = NULL;
	if (Blueprint)
	{
		CallingContext = Blueprint->GeneratedClass;
		if (CallingContext == NULL)
		{
			CallingContext = Blueprint->ParentClass;
		}
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* ArrayPin = Pins[0];
	UEdGraphPin* ResultPin = Pins[2];

	ArrayPin->PinType = InType;
	ArrayPin->PinType.ContainerType = EPinContainerType::Array;
	ArrayPin->PinType.bIsReference = false;

	// IsSetToReturnRef() has to be called after the ArrayPin's type is set, since it uses that to determine the result
	const bool bMakeOutputRef = IsSetToReturnRef() ||
		// Honor bReturnByRefDesired for transient (intermediate) nodes during compilation:
		(Blueprint && Blueprint->bBeingCompiled && bReturnByRefDesired && HasAnyFlags(RF_Transient));
	if (!bMakeOutputRef && bReturnByRefDesired && ResultPin->PinType.bIsReference && Blueprint && !Blueprint->bIsRegeneratingOnLoad)
	{
		FNotificationInfo Warning(LOCTEXT("ConnectionAlteredOutput", "Array Get node altered. Now returning a copy."));
		Warning.ExpireDuration = 2.0f;
		Warning.bFireAndForget = true;
		Warning.bUseLargeFont  = false;
		Warning.Image = FCoreStyle::Get().GetBrush(TEXT("MessageLog.Warning"));
		FSlateNotificationManager::Get().AddNotification(Warning);
	}

	ResultPin->PinType = InType;
	ResultPin->PinType.ContainerType = EPinContainerType::None;
	ResultPin->PinType.bIsReference = bMakeOutputRef;
	ResultPin->PinType.bIsConst = false;


	// Verify that all previous connections to this pin are still valid with the new type
	for (TArray<UEdGraphPin*>::TIterator ConnectionIt(ArrayPin->LinkedTo); ConnectionIt; ++ConnectionIt)
	{
		UEdGraphPin* ConnectedPin = *ConnectionIt;
		if (!Schema->ArePinsCompatible(ArrayPin, ConnectedPin, CallingContext))
		{
			ArrayPin->BreakLinkTo(ConnectedPin);
		}
		else if (ConnectedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			if (UK2Node* ConnectedNode = Cast<UK2Node>(ConnectedPin->GetOwningNode()))
			{
				ConnectedNode->PinConnectionListChanged(ConnectedPin);
			}
		}
	}

	// Verify that all previous connections to this pin are still valid with the new type
	for (TArray<UEdGraphPin*>::TIterator ConnectionIt(ResultPin->LinkedTo); ConnectionIt; ++ConnectionIt)
	{
		UEdGraphPin* ConnectedPin = *ConnectionIt;
		if (!Schema->ArePinsCompatible(ResultPin, ConnectedPin, CallingContext))
		{
			ResultPin->BreakLinkTo(ConnectedPin);
		}
		else if (ConnectedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			if (UK2Node* ConnectedNode = Cast<UK2Node>(ConnectedPin->GetOwningNode()))
			{
				ConnectedNode->PinConnectionListChanged(ConnectedPin);
			}
		}
	}
}

bool UK2Node_GetArrayItem::IsSetToReturnRef() const
{
	return bReturnByRefDesired && K2Node_GetArrayItem_Impl::SupportsReturnByRef(this);
}

#undef LOCTEXT_NAMESPACE
