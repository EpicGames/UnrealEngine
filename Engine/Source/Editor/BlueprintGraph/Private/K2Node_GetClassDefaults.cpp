// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetClassDefaults.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_PureAssignmentStatement.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetClassDefaults)

#define LOCTEXT_NAMESPACE "UK2Node_GetClassDefaults"

FName UK2Node_GetClassDefaults::ClassPinName(TEXT("Class"));

namespace
{
	// Optional pin manager subclass.
	struct FClassDefaultsOptionalPinManager : public FOptionalPinManager
	{
		FClassDefaultsOptionalPinManager(UClass* InClass, bool bExcludeObjectContainers, bool bExcludeObjectArrays)
			:FOptionalPinManager()
		{
			SrcClass = InClass;
			bExcludeObjectArrayProperties = bExcludeObjectContainers | bExcludeObjectArrays;
			bExcludeObjectContainerProperties = bExcludeObjectContainers;
		}

		virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
		{
			FOptionalPinManager::GetRecordDefaults(TestProperty, Record);

			// Show pin unless the property is owned by a parent class.
			Record.bShowPin = TestProperty->GetOwnerClass() == SrcClass;
		}

		virtual bool CanTreatPropertyAsOptional(FProperty* TestProperty) const override
		{
			// Don't expose anything not marked BlueprintReadOnly/BlueprintReadWrite.
			if(!TestProperty || !TestProperty->HasAllPropertyFlags(CPF_BlueprintVisible))
			{
				return false;
			}
			
			if(FArrayProperty* TestArrayProperty = CastField<FArrayProperty>(TestProperty))
			{
				// We only use the Inner type if the flag is set. This is done for backwards-compatibility (some BPs may already rely on the previous behavior when the property value was allowed to be exposed).
				if(bExcludeObjectArrayProperties && TestArrayProperty->Inner)
				{
					TestProperty = TestArrayProperty->Inner;
				}
			}
			else if (FSetProperty* TestSetProperty = CastField<FSetProperty>(TestProperty))
			{
				if (bExcludeObjectContainerProperties && TestSetProperty->ElementProp)
				{
					TestProperty = TestSetProperty->ElementProp;
				}
			}
			else if (FMapProperty* TestMapProperty = CastField<FMapProperty>(TestProperty))
			{
				// Since we can't treat the key or value as read-only right now, we exclude any TMap that has a non-class UObject reference as its key or value type.
				return !(bExcludeObjectContainerProperties
					&& ((TestMapProperty->KeyProp && TestMapProperty->KeyProp->IsA<FObjectProperty>() && !TestMapProperty->KeyProp->IsA<FClassProperty>())
						|| (TestMapProperty->ValueProp && TestMapProperty->ValueProp->IsA<FObjectProperty>() && !TestMapProperty->ValueProp->IsA<FClassProperty>())));
			}
			
			// Don't expose object properties (except for those containing class objects).
			// @TODO - Could potentially expose object reference values if/when we have support for 'const' input pins.
			return !TestProperty->ContainsInstancedObjectProperty() || !TestProperty->IsA<FObjectProperty>() || TestProperty->IsA<FClassProperty>();
		}

		virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property = nullptr) const override
		{
			check(Pin);

			// Move into the advanced view if the property metadata is set.
			Pin->bAdvancedView = Property && Property->HasAnyPropertyFlags(CPF_AdvancedDisplay);
		}

	private:
		// Class type for which optional pins are being managed.
		UClass* SrcClass;

		// Indicates whether or not object array properties will be excluded (for backwards-compatibility).
		bool bExcludeObjectArrayProperties;

		// Indicates whether or not object container properties will be excluded (will supercede the array-specific flag when true).
		bool bExcludeObjectContainerProperties;
	};

	// Compilation handler subclass.
	class FKCHandler_GetClassDefaults : public FNodeHandlingFunctor
	{
	public:
		FKCHandler_GetClassDefaults(FKismetCompilerContext& InCompilerContext)
			: FNodeHandlingFunctor(InCompilerContext)
		{
		}

		virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
		{
			// Cast to the correct node type
			if(const UK2Node_GetClassDefaults* GetClassDefaultsNode = Cast<UK2Node_GetClassDefaults>(Node))
			{
				// Only if we have a valid class input pin
				if(UEdGraphPin* ClassPin = GetClassDefaultsNode->FindClassPin())
				{
					// Redirect to a linked pin if necessary
					UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(ClassPin);
					check(Net != nullptr);

					// Register a literal if necessary (e.g. there are no linked pins)
					if(ValidateAndRegisterNetIfLiteral(Context, Net))
					{
						// First check for a literal term in case one was created above
						FBPTerminal** FoundTerm = Context.LiteralHackMap.Find(Net);
						if(FoundTerm == nullptr)
						{
							// Otherwise, check for a linked term
							FoundTerm = Context.NetMap.Find(Net);
						}

						// If we did not find an input term, make sure we create one here
						FBPTerminal* ClassContextTerm = FoundTerm ? *FoundTerm : nullptr;
						if(ClassContextTerm == nullptr)
						{
							ClassContextTerm = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
							check(ClassContextTerm != nullptr);

							Context.NetMap.Add(Net, ClassContextTerm);
						}

						// Flag this as a "class context" term
						ClassContextTerm->SetContextTypeClass();

						// Infer the class type from the context term
						if(const UClass* ClassType = Cast<UClass>(ClassContextTerm->bIsLiteral ? ClassContextTerm->ObjectLiteral.Get() : ClassContextTerm->Type.PinSubCategoryObject.Get()))
						{
							// Create a local term for each output pin (class property)
							for(int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
							{
								UEdGraphPin* Pin = Node->Pins[PinIndex];
								if(Pin != nullptr && Pin->Direction == EGPD_Output)
								{
									FProperty* BoundProperty = FindFProperty<FProperty>(ClassType, Pin->PinName);
									if(BoundProperty != nullptr)
									{
										FBPTerminal* OutputTerm = Context.CreateLocalTerminalFromPinAutoChooseScope(Pin, Pin->PinName.ToString());
										check(OutputTerm != nullptr);

										// Set as a variable within the class context
										OutputTerm->AssociatedVarProperty = BoundProperty;
										OutputTerm->Context = ClassContextTerm;

										// Flag this as a "class default" variable term
										OutputTerm->bIsConst = true;
										OutputTerm->SetVarTypeDefault();

										// Add it to the lookup table
										Context.NetMap.Add(Pin, OutputTerm);
									}
									else
									{
										CompilerContext.MessageLog.Error(*LOCTEXT("UnmatchedOutputPinOnCompile", "Failed to find a class member to match @@").ToString(), Pin);
									}
								}
							}
						}
						else
						{
							CompilerContext.MessageLog.Error(*LOCTEXT("InvalidClassTypeOnCompile", "Missing or invalid input class type for @@").ToString(), Node);
						}
					}
				}
			}
		}
	};
}

UK2Node_GetClassDefaults::UK2Node_GetClassDefaults(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_GetClassDefaults::PostLoad()
{
	Super::PostLoad();

	// Ensure that a previously-set class object is no longer referenced as a default value if the class input pin is connected.
	if (UEdGraphPin* ClassPin = FindClassPin(Pins))
	{
		if (ClassPin->LinkedTo.Num() > 0 && !ClassPin->DoesDefaultValueMatchAutogenerated())
		{
			if (const UEdGraphSchema* Schema = GetSchema())
			{
				Schema->ResetPinToAutogeneratedDefaultValue(ClassPin, false);
			}
		}
	}
}


void UK2Node_GetClassDefaults::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);
	}
}

void UK2Node_GetClassDefaults::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		GetSchema()->ReconstructNode(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FText UK2Node_GetClassDefaults::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// @TODO - unique node icon needed as well?

	return LOCTEXT("NodeTitle", "Get Class Defaults");
}

void UK2Node_GetClassDefaults::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Create the class input type selector pin
	UEdGraphPin* ClassPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, UObject::StaticClass(), ClassPinName);
	K2Schema->ConstructBasicPinTooltip(*ClassPin, LOCTEXT("ClassPinDescription", "The class from which to access one or more default values."), ClassPin->PinToolTip);
}

void UK2Node_GetClassDefaults::PostPlacedNewNode()
{
	// Always exclude object container properties for new nodes.
	// @TODO - Could potentially expose object reference values if/when we have support for 'const' input pins.
	bExcludeObjectContainers = true;

	if(UEdGraphPin* ClassPin = FindClassPin(Pins))
	{
		// Default to the owner BP's generated class for "normal" BPs if this is a new node
		const UBlueprint* OwnerBlueprint = GetBlueprint();
		if(OwnerBlueprint != nullptr && OwnerBlueprint->BlueprintType == BPTYPE_Normal)
		{
			ClassPin->DefaultObject = OwnerBlueprint->GeneratedClass;
		}

		if(UClass* InputClass = GetInputClass(ClassPin))
		{
			CreateOutputPins(InputClass);
		}
	}
}

void UK2Node_GetClassDefaults::PinConnectionListChanged(UEdGraphPin* ChangedPin)
{
	Super::PinConnectionListChanged(ChangedPin);

	if(ChangedPin != nullptr && ChangedPin->PinName == ClassPinName && ChangedPin->Direction == EGPD_Input)
	{
		OnClassPinChanged();
	}
}

void UK2Node_GetClassDefaults::PinDefaultValueChanged(UEdGraphPin* ChangedPin) 
{
	Super::PinDefaultValueChanged(ChangedPin);

	if(ChangedPin != nullptr && ChangedPin->PinName == ClassPinName && ChangedPin->Direction == EGPD_Input)
	{
		OnClassPinChanged();
	}
}

void UK2Node_GetClassDefaults::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (const UClass* SourceClass = GetInputClass())
	{
		for (const UEdGraphPin* Pin : Pins)
		{
			// Emit a warning for existing connections to potentially unsafe array property defaults. We do this rather than just implicitly breaking the connection (for compatibility).
			if (Pin && Pin->Direction == EGPD_Output && Pin->LinkedTo.Num() > 0)
			{
				// Even though container property defaults are copied, the copy could still contain a reference to a non-class object that belongs to the CDO, which would potentially be unsafe to modify.
				bool bEmitWarning = false;
				const FProperty* TestProperty = SourceClass->FindPropertyByName(Pin->PinName);
				if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(TestProperty))
				{
					bEmitWarning = ArrayProperty->Inner && ArrayProperty->Inner->IsA<FObjectProperty>() && !ArrayProperty->Inner->IsA<FClassProperty>();
				}
				else if (const FSetProperty* SetProperty = CastField<FSetProperty>(TestProperty))
				{
					bEmitWarning = SetProperty->ElementProp && SetProperty->ElementProp->IsA<FObjectProperty>() && !SetProperty->ElementProp->IsA<FClassProperty>();
				}
				else if (const FMapProperty* MapProperty = CastField<FMapProperty>(TestProperty))
				{
					bEmitWarning = (MapProperty->KeyProp && MapProperty->KeyProp->IsA<FObjectProperty>() && !MapProperty->KeyProp->IsA<FClassProperty>())
						|| (MapProperty->ValueProp && MapProperty->ValueProp->IsA<FObjectProperty>() && !MapProperty->ValueProp->IsA<FClassProperty>());
				}

				if (bEmitWarning)
				{
					MessageLog.Warning(*LOCTEXT("UnsafeConnectionWarning", "@@ has an unsafe connection to the @@ output pin that is not fully supported at this time. It should be disconnected to avoid potentially corrupting class defaults at runtime. If you need to keep this connection, make sure you're not changing the state of any elements in the container. Also note that if you recreate this node, it will not include this output pin.").ToString(), this, Pin);
				}
			}
		}
	}
}

bool UK2Node_GetClassDefaults::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UClass* SourceClass = GetInputClass();
	UBlueprint* SourceBlueprint = GetBlueprint();
	const bool bResult = (SourceClass && (SourceClass->ClassGeneratedBy != SourceBlueprint));
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_GetClassDefaults::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	AllocateDefaultPins();

	// Recreate output pins based on the previous input class
	UEdGraphPin* OldClassPin = FindClassPin(OldPins);
	if(UClass* InputClass = GetInputClass(OldClassPin))
	{
		CreateOutputPins(InputClass);
	}

	RestoreSplitPins(OldPins);
}

FNodeHandlingFunctor* UK2Node_GetClassDefaults::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_GetClassDefaults(CompilerContext);
}

void UK2Node_GetClassDefaults::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UClass* ClassType = GetInputClass();

	// @TODO - Remove if/when we support 'const' input pins.
	// For container properties, return a local copy of the container so that the original cannot be modified.
	for(UEdGraphPin* OutputPin : Pins)
	{
		if(OutputPin != nullptr && OutputPin->Direction == EGPD_Output && OutputPin->LinkedTo.Num() > 0)
		{
			FProperty* BoundProperty = FindFProperty<FProperty>(ClassType, OutputPin->PinName);
			if(BoundProperty != nullptr && (BoundProperty->IsA<FArrayProperty>() || BoundProperty->IsA<FSetProperty>() || BoundProperty->IsA<FMapProperty>()))
			{
				UK2Node_TemporaryVariable* LocalVariable = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
				LocalVariable->VariableType = OutputPin->PinType;
				LocalVariable->VariableType.bIsReference = false;
				LocalVariable->AllocateDefaultPins();

				UK2Node_PureAssignmentStatement* CopyDefaultValue = CompilerContext.SpawnIntermediateNode<UK2Node_PureAssignmentStatement>(this, SourceGraph);
				CopyDefaultValue->AllocateDefaultPins();
				CopyDefaultValue->GetVariablePin()->PinType = OutputPin->PinType;
				CompilerContext.GetSchema()->TryCreateConnection(LocalVariable->GetVariablePin(), CopyDefaultValue->GetVariablePin());

				// Note: This must be done AFTER connecting the variable input, which sets the pin type.
				CompilerContext.MovePinLinksToIntermediate(*OutputPin, *CopyDefaultValue->GetOutputPin());
				CompilerContext.GetSchema()->TryCreateConnection(OutputPin, CopyDefaultValue->GetValuePin());
			}
		}
	}
}

void UK2Node_GetClassDefaults::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetClassDefaults::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Class);
}

UEdGraphPin* UK2Node_GetClassDefaults::FindClassPin(const TArray<UEdGraphPin*>& FromPins) const
{
	UEdGraphPin* const* FoundPin = FromPins.FindByPredicate([](const UEdGraphPin* CurPin)
	{
		return CurPin && CurPin->Direction == EGPD_Input && CurPin->PinName == ClassPinName;
	});

	return FoundPin ? *FoundPin : nullptr;
}

UClass* UK2Node_GetClassDefaults::GetInputClass(const UEdGraphPin* FromPin) const
{
	UClass* InputClass = nullptr;

	if (FromPin != nullptr)
	{
		check(FromPin->Direction == EGPD_Input);

		if (FromPin->DefaultObject != nullptr && FromPin->LinkedTo.Num() == 0)
		{
			InputClass = CastChecked<UClass>(FromPin->DefaultObject);
		}
		else if (FromPin->LinkedTo.Num() > 0)
		{
			if (UEdGraphPin* LinkedPin = FromPin->LinkedTo[0])
			{
				InputClass = Cast<UClass>(LinkedPin->PinType.PinSubCategoryObject.Get());
			}
		}
	}

	// Switch Blueprint Class types to use the generated skeleton class (if valid).
	if (InputClass)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InputClass->ClassGeneratedBy))
		{
			// Stick with the original (serialized) class if the skeleton class is not valid for some reason (e.g. the Blueprint hasn't been compiled on load yet).
			// Note: There's not a need to force it to be preloaded here in that case, because once it is loaded, we'll end up reconstructing this node again anyway.
			if (Blueprint->SkeletonGeneratedClass)
			{
				InputClass = Blueprint->SkeletonGeneratedClass;
			}
		}
	}

	return InputClass;
}

void UK2Node_GetClassDefaults::OnBlueprintClassModified(UBlueprint* TargetBlueprint)
{
	check(TargetBlueprint);
	UBlueprint* OwnerBlueprint = FBlueprintEditorUtils::FindBlueprintForNode(this); //GetBlueprint() will crash, when the node is transient, etc
	if (OwnerBlueprint)
	{
		// The Blueprint that contains this node may have finished 
		// regenerating (see bHasBeenRegenerated), but we still may be
		// in the midst of unwinding a cyclic load (dependent Blueprints);
		// this lambda could be triggered during the targeted 
		// Blueprint's regeneration - meaning we really haven't completed 
		// the load process. In this situation, we cannot "reset loaders" 
		// because it is not likely that all of the package's objects
		// have been post-loaded (meaning an assert will most likely  
		// fire from ReconstructNode). To guard against this, we flip this
		// Blueprint's bIsRegeneratingOnLoad (like in 
		// UBlueprintGeneratedClass::ConditionalRecompileClass), which
		// we use throughout Blueprints to keep us from reseting loaders 
		// on object Rename()
		const bool bOldIsRegeneratingVal = OwnerBlueprint->bIsRegeneratingOnLoad;
		OwnerBlueprint->bIsRegeneratingOnLoad = bOldIsRegeneratingVal || TargetBlueprint->bIsRegeneratingOnLoad;

		ReconstructNode();

		OwnerBlueprint->bIsRegeneratingOnLoad = bOldIsRegeneratingVal;
	}
}

void UK2Node_GetClassDefaults::CreateOutputPins(UClass* InClass)
{
	// Create the set of output pins through the optional pin manager
	FClassDefaultsOptionalPinManager OptionalPinManager(InClass, bExcludeObjectContainers, bExcludeObjectArrays_DEPRECATED);
	OptionalPinManager.RebuildPropertyList(ShowPinForProperties, InClass);

	OptionalPinManager.CreateVisiblePins(ShowPinForProperties, InClass, EGPD_Output, this);

	// Check for any advanced properties (outputs)
	bool bHasAdvancedPins = false;
	for(int32 PinIndex = 0; PinIndex < Pins.Num() && !bHasAdvancedPins; ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		check(Pin != nullptr);

		bHasAdvancedPins |= Pin->bAdvancedView;
	}

	// Toggle advanced display on/off based on whether or not we have any advanced outputs
	if(bHasAdvancedPins && AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
	else if(!bHasAdvancedPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::NoPins;
	}

	// Unbind OnChanged() delegate from a previous Blueprint, if valid.

	// If the class was generated for a Blueprint, bind delegates to handle any OnChanged() & OnCompiled() events.
	bool bShouldClearDelegate = true;
	if (InClass)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InClass->ClassGeneratedBy))
		{
			// only clear the delegate if the pin has changed:
			bShouldClearDelegate = BlueprintSubscribedTo != Blueprint;
		}
	}

	if (bShouldClearDelegate)
	{
		if (OnBlueprintChangedDelegate.IsValid())
		{
			if (BlueprintSubscribedTo)
			{
				BlueprintSubscribedTo->OnChanged().Remove(OnBlueprintChangedDelegate);
			}
			OnBlueprintChangedDelegate.Reset();
		}

		// Unbind OnCompiled() delegate from a previous Blueprint, if valid.
		if (OnBlueprintCompiledDelegate.IsValid())
		{
			if (BlueprintSubscribedTo)
			{
				BlueprintSubscribedTo->OnCompiled().Remove(OnBlueprintCompiledDelegate);
			}
			OnBlueprintCompiledDelegate.Reset();
		}
		// Associated Blueprint changed, clear the BlueprintSubscribedTo:
		BlueprintSubscribedTo = nullptr;
	}

	if (InClass && bShouldClearDelegate)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InClass->ClassGeneratedBy))
		{
			BlueprintSubscribedTo = Blueprint;
			OnBlueprintChangedDelegate  = Blueprint->OnChanged().AddUObject(this, &ThisClass::OnBlueprintClassModified);
			OnBlueprintCompiledDelegate = Blueprint->OnCompiled().AddUObject(this, &ThisClass::OnBlueprintClassModified);
		}
	}
}

void UK2Node_GetClassDefaults::OnClassPinChanged()
{
	Modify();

	TArray<UEdGraphPin*> OldOutputPins = MoveTemp(Pins);

	// Gather all current output pins
	for(int32 PinIndex = 0; PinIndex < OldOutputPins.Num(); ++PinIndex)
	{
		UEdGraphPin* OldPin = OldOutputPins[PinIndex];
		if(OldPin->Direction == EGPD_Input)
		{
			OldOutputPins.RemoveAt(PinIndex--, EAllowShrinking::No);
			Pins.Add(OldPin);
		}
	}

	// If the blueprint is currently compiling, we shouldn't be changing the showpins settings
	const bool bIsCompiling = GetBlueprint()->bBeingCompiled;

	if (!bIsCompiling)
	{
		// Clear the current output pin settings (so they don't carry over to the new set)
		ShowPinForProperties.Reset();
	}

	// Create output pins for the new class type
	UClass* InputClass = GetInputClass();
	CreateOutputPins(InputClass);

	// Restore the split pins and reconnect output pins to previously linked inputs
	RestoreSplitPins(OldOutputPins);
	RewireOldPinsToNewPins(OldOutputPins, Pins, nullptr);

	// Notify the graph that the node has been changed
	if(UEdGraph* Graph = GetGraph())
	{
		Graph->NotifyNodeChanged(this);
	}
}

#undef LOCTEXT_NAMESPACE
