// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetDataTableRow.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DataTableEditorUtils.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/DataTable.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "Kismet/DataTableFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetDataTableRow)

class UBlueprint;

#define LOCTEXT_NAMESPACE "K2Node_GetDataTableRow"

namespace GetDataTableRowHelper
{
	const FName DataTablePinName = "DataTable";
	const FName RowNotFoundPinName = "RowNotFound";
	const FName RowNamePinName = "RowName";
}

UK2Node_GetDataTableRow::UK2Node_GetDataTableRow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Attempts to retrieve a TableRow from a DataTable via it's RowName");
}

void UK2Node_GetDataTableRow::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	UEdGraphPin* RowFoundPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	RowFoundPin->PinFriendlyName = LOCTEXT("GetDataTableRow Row Found Exec pin", "Row Found");
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, GetDataTableRowHelper::RowNotFoundPinName);

	// Add DataTable pin
	UEdGraphPin* DataTablePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UDataTable::StaticClass(), GetDataTableRowHelper::DataTablePinName);
	SetPinToolTip(*DataTablePin, LOCTEXT("DataTablePinDescription", "The DataTable you want to retreive a row from"));

	// Row Name pin
	UEdGraphPin* RowNamePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Name, GetDataTableRowHelper::RowNamePinName);
	SetPinToolTip(*RowNamePin, LOCTEXT("RowNamePinDescription", "The name of the row to retrieve from the DataTable"));

	// Result pin
	UEdGraphPin* ResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UEdGraphSchema_K2::PN_ReturnValue);
	ResultPin->PinFriendlyName = LOCTEXT("GetDataTableRow Output Row", "Out Row");
	SetPinToolTip(*ResultPin, LOCTEXT("ResultPinDescription", "The returned TableRow, if found"));

	Super::AllocateDefaultPins();
}

void UK2Node_GetDataTableRow::SetPinToolTip(UEdGraphPin& MutatablePin, const FText& PinDescription) const
{
	MutatablePin.PinToolTip = UEdGraphSchema_K2::TypeToText(MutatablePin.PinType).ToString();

	UEdGraphSchema_K2 const* const K2Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	if (K2Schema != nullptr)
	{
		MutatablePin.PinToolTip += TEXT(" ");
		MutatablePin.PinToolTip += K2Schema->GetPinDisplayName(&MutatablePin).ToString();
	}

	MutatablePin.PinToolTip += FString(TEXT("\n")) + PinDescription.ToString();
}

void UK2Node_GetDataTableRow::RefreshOutputPinType()
{
	UScriptStruct* OutputType = GetDataTableRowStructType();
	SetReturnTypeForStruct(OutputType);
}

void UK2Node_GetDataTableRow::RefreshRowNameOptions()
{
	// When the DataTable pin gets a new value assigned, we need to update the Slate UI so that SGraphNodeCallParameterCollectionFunction will update the ParameterName drop down
	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}


void UK2Node_GetDataTableRow::SetReturnTypeForStruct(UScriptStruct* NewRowStruct)
{
	UScriptStruct* OldRowStruct = GetReturnTypeForStruct();
	if (NewRowStruct != OldRowStruct)
	{
		UEdGraphPin* ResultPin = GetResultPin();

		if (ResultPin->SubPins.Num() > 0)
		{
			GetSchema()->RecombinePin(ResultPin);
		}

		// NOTE: purposefully not disconnecting the ResultPin (even though it changed type)... we want the user to see the old
		//       connections, and incompatible connections will produce an error (plus, some super-struct connections may still be valid)
		ResultPin->PinType.PinSubCategoryObject = NewRowStruct;
		ResultPin->PinType.PinCategory = (NewRowStruct == nullptr) ? UEdGraphSchema_K2::PC_Wildcard : UEdGraphSchema_K2::PC_Struct;

		CachedNodeTitle.Clear();
	}
}

UScriptStruct* UK2Node_GetDataTableRow::GetReturnTypeForStruct()
{
	UScriptStruct* ReturnStructType = (UScriptStruct*)(GetResultPin()->PinType.PinSubCategoryObject.Get());

	return ReturnStructType;
}

UScriptStruct* UK2Node_GetDataTableRow::GetDataTableRowStructType() const
{
	UScriptStruct* RowStructType = nullptr;

	UEdGraphPin* DataTablePin = GetDataTablePin();
	if(DataTablePin && DataTablePin->DefaultObject != nullptr && DataTablePin->LinkedTo.Num() == 0)
	{
		if (const UDataTable* DataTable = Cast<const UDataTable>(DataTablePin->DefaultObject))
		{
			RowStructType = DataTable->RowStruct;
		}
	}

	if (RowStructType == nullptr)
	{
		UEdGraphPin* ResultPin = GetResultPin();
		if (ResultPin && ResultPin->LinkedTo.Num() > 0)
		{
			RowStructType = Cast<UScriptStruct>(ResultPin->LinkedTo[0]->PinType.PinSubCategoryObject.Get());
			if (RowStructType == nullptr && ResultPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard)
			{
				RowStructType = GetFallbackStruct();
			}
			for (int32 LinkIndex = 1; LinkIndex < ResultPin->LinkedTo.Num(); ++LinkIndex)
			{
				UEdGraphPin* Link = ResultPin->LinkedTo[LinkIndex];
				UScriptStruct* LinkType = Cast<UScriptStruct>(Link->PinType.PinSubCategoryObject.Get());

				if (RowStructType && RowStructType->IsChildOf(LinkType))
				{
					RowStructType = LinkType;
				}
			}
		}
	}
	return RowStructType;
}

void UK2Node_GetDataTableRow::OnDataTableRowListChanged(const UDataTable* DataTable)
{
	UEdGraphPin* DataTablePin = GetDataTablePin();
	if (DataTable && DataTablePin && DataTable == DataTablePin->DefaultObject)
	{
		UEdGraphPin* RowNamePin = GetRowNamePin();
		const bool TryRefresh = RowNamePin && !RowNamePin->LinkedTo.Num();
		const FName CurrentName = RowNamePin ? FName(*RowNamePin->GetDefaultAsString()) : NAME_None;
		if (TryRefresh && RowNamePin && !DataTable->GetRowNames().Contains(CurrentName))
		{
			if (UBlueprint* BP = GetBlueprint())
			{
				FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
			}
		}
	}
}

void UK2Node_GetDataTableRow::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	if (UEdGraphPin* DataTablePin = GetDataTablePin(&OldPins))
	{
		if (UDataTable* DataTable = Cast<UDataTable>(DataTablePin->DefaultObject))
		{
			// make sure to properly load the data-table object so that we can 
			// farm the "RowStruct" property from it (below, in GetDataTableRowStructType)
			PreloadObject(DataTable);
		}
	}
}

void UK2Node_GetDataTableRow::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetDataTableRow::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Utilities);
}

bool UK2Node_GetDataTableRow::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (MyPin == GetResultPin() && MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		bool bDisallowed = true;
		if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
		{
			if (UScriptStruct* ConnectionType = Cast<UScriptStruct>(OtherPin->PinType.PinSubCategoryObject.Get()))
			{
				bDisallowed = !FDataTableEditorUtils::IsValidTableStruct(ConnectionType);
			}
		}
		else if (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			bDisallowed = false;
		}

		if (bDisallowed)
		{
			OutReason = TEXT("Must be a struct that can be used in a DataTable");
		}
		return bDisallowed;
	}
	return false;
}

void UK2Node_GetDataTableRow::PinDefaultValueChanged(UEdGraphPin* ChangedPin) 
{
	if (ChangedPin && ChangedPin->PinName == GetDataTableRowHelper::DataTablePinName)
	{
		RefreshOutputPinType();

		UEdGraphPin* RowNamePin = GetRowNamePin();
		UDataTable*  DataTable = Cast<UDataTable>(ChangedPin->DefaultObject);
		if (RowNamePin)
		{
			if (DataTable && (RowNamePin->DefaultValue.IsEmpty() || !DataTable->GetRowMap().Contains(*RowNamePin->DefaultValue)))
			{
				if (auto Iterator = DataTable->GetRowMap().CreateConstIterator())
				{
					RowNamePin->DefaultValue = Iterator.Key().ToString();
				}
			}	

			RefreshRowNameOptions();
		}
	}
}

FText UK2Node_GetDataTableRow::GetTooltipText() const
{
	return NodeTooltip;
}

UEdGraphPin* UK2Node_GetDataTableRow::GetDataTablePin(const TArray<UEdGraphPin*>* InPinsToSearch /*= NULL*/) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;
    
	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* TestPin : *PinsToSearch)
	{
		if (TestPin && TestPin->PinName == GetDataTableRowHelper::DataTablePinName)
		{
			Pin = TestPin;
			break;
		}
	}
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_GetDataTableRow::GetRowNamePin() const
{
	UEdGraphPin* Pin = FindPinChecked(GetDataTableRowHelper::RowNamePinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_GetDataTableRow::GetRowNotFoundPin() const
{
	UEdGraphPin* Pin = FindPinChecked(GetDataTableRowHelper::RowNotFoundPinName);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

UEdGraphPin* UK2Node_GetDataTableRow::GetResultPin() const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* Pin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

FText UK2Node_GetDataTableRow::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("ListViewTitle", "Get Data Table Row");
	}
	else if (UEdGraphPin* DataTablePin = GetDataTablePin())
	{
		if (DataTablePin->LinkedTo.Num() > 0)
		{
			return NSLOCTEXT("K2Node", "DataTable_Title_Unknown", "Get Data Table Row");
		}
		else if (DataTablePin->DefaultObject == nullptr)
		{
			return NSLOCTEXT("K2Node", "DataTable_Title_None", "Get Data Table Row NONE");
		}
		else if (CachedNodeTitle.IsOutOfDate(this))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("DataTableName"), FText::FromString(DataTablePin->DefaultObject->GetName()));

			FText LocFormat = NSLOCTEXT("K2Node", "DataTable", "Get Data Table Row {DataTableName}");
			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
		}
	}
	else
	{
		return NSLOCTEXT("K2Node", "DataTable_Title_None", "Get Data Table Row NONE");
	}	
	return CachedNodeTitle;
}

void UK2Node_GetDataTableRow::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
    Super::ExpandNode(CompilerContext, SourceGraph);
    
    UEdGraphPin* OriginalDataTableInPin = GetDataTablePin();
    UDataTable* Table = (OriginalDataTableInPin != NULL) ? Cast<UDataTable>(OriginalDataTableInPin->DefaultObject) : NULL;
    if((nullptr == OriginalDataTableInPin) || (0 == OriginalDataTableInPin->LinkedTo.Num() && nullptr == Table))
    {
        CompilerContext.MessageLog.Error(*LOCTEXT("GetDataTableRowNoDataTable_Error", "GetDataTableRow must have a DataTable specified.").ToString(), this);
        // we break exec links so this is the only error we get
        BreakAllNodeLinks();
        return;
    }

	// FUNCTION NODE
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UDataTableFunctionLibrary, GetDataTableRowFromName);
	UK2Node_CallFunction* GetDataTableRowFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetDataTableRowFunction->FunctionReference.SetExternalMember(FunctionName, UDataTableFunctionLibrary::StaticClass());
	GetDataTableRowFunction->AllocateDefaultPins();
    CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *(GetDataTableRowFunction->GetExecPin()));

	// Connect the input of our GetDataTableRow to the Input of our Function pin
    UEdGraphPin* DataTableInPin = GetDataTableRowFunction->FindPinChecked(TEXT("Table"));
	if(OriginalDataTableInPin->LinkedTo.Num() > 0)
	{
		// Copy the connection
		CompilerContext.MovePinLinksToIntermediate(*OriginalDataTableInPin, *DataTableInPin);
	}
	else
	{
		// Copy literal
		DataTableInPin->DefaultObject = OriginalDataTableInPin->DefaultObject;
	}
	UEdGraphPin* RowNameInPin = GetDataTableRowFunction->FindPinChecked(TEXT("RowName"));
	CompilerContext.MovePinLinksToIntermediate(*GetRowNamePin(), *RowNameInPin);

	// Get some pins to work with
	UEdGraphPin* OriginalOutRowPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* FunctionOutRowPin = GetDataTableRowFunction->FindPinChecked(TEXT("OutRow"));
    UEdGraphPin* FunctionReturnPin = GetDataTableRowFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
    UEdGraphPin* FunctionThenPin = GetDataTableRowFunction->GetThenPin();
        
    // Set the type of the OutRow pin on this expanded mode to match original
    FunctionOutRowPin->PinType = OriginalOutRowPin->PinType;
	FunctionOutRowPin->PinType.PinSubCategoryObject = OriginalOutRowPin->PinType.PinSubCategoryObject;
        
    //BRANCH NODE
    UK2Node_IfThenElse* BranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
    BranchNode->AllocateDefaultPins();
    // Hook up inputs to branch
    FunctionThenPin->MakeLinkTo(BranchNode->GetExecPin());
    FunctionReturnPin->MakeLinkTo(BranchNode->GetConditionPin());
        
    // Hook up outputs
    CompilerContext.MovePinLinksToIntermediate(*GetThenPin(), *(BranchNode->GetThenPin()));
    CompilerContext.MovePinLinksToIntermediate(*GetRowNotFoundPin(), *(BranchNode->GetElsePin()));
    CompilerContext.MovePinLinksToIntermediate(*OriginalOutRowPin, *FunctionOutRowPin);

	BreakAllNodeLinks();
}

FSlateIcon UK2Node_GetDataTableRow::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = GetNodeTitleColor();
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon;
}

void UK2Node_GetDataTableRow::PostReconstructNode()
{
	Super::PostReconstructNode();

	RefreshOutputPinType();
}

void UK2Node_GetDataTableRow::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	const UEdGraphPin* DataTablePin = GetDataTablePin();
	const UEdGraphPin* RowNamePin = GetRowNamePin();
	if (!DataTablePin || !RowNamePin)
	{
		MessageLog.Error(*LOCTEXT("MissingPins", "Missing pins in @@").ToString(), this);
		return;
	}

	if (DataTablePin->LinkedTo.Num() == 0)
	{
		const UDataTable* DataTable = Cast<UDataTable>(DataTablePin->DefaultObject);
		if (!DataTable)
		{
			MessageLog.Error(*LOCTEXT("NoDataTable", "No DataTable in @@").ToString(), this);
			return;
		}

		if (!RowNamePin->LinkedTo.Num())
		{
			const FName CurrentName = FName(*RowNamePin->GetDefaultAsString());
			if (!DataTable->GetRowNames().Contains(CurrentName))
			{
				const FString Msg = FText::Format(
					LOCTEXT("WrongRowNameFmt", "'{0}' row name is not stored in '{1}'. @@"),
					FText::FromString(CurrentName.ToString()),
					FText::FromString(GetFullNameSafe(DataTable))
				).ToString();
				MessageLog.Error(*Msg, this);
				return;
			}
		}
	}	
}

void UK2Node_GetDataTableRow::PreloadRequiredAssets()
{
	if (UEdGraphPin* DataTablePin = GetDataTablePin())
	{
		if (UDataTable* DataTable = Cast<UDataTable>(DataTablePin->DefaultObject))
		{
			// make sure to properly load the data-table object so that we can 
			// farm the "RowStruct" property from it (below, in GetDataTableRowStructType)
			PreloadObject(DataTable);
		}
	}
	return Super::PreloadRequiredAssets();
}

void UK2Node_GetDataTableRow::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	if (Pin == GetResultPin())
	{
		UEdGraphPin* TablePin = GetDataTablePin();
		// this connection would only change the output type if the table pin is undefined
		const bool bIsTypeAuthority = (TablePin->LinkedTo.Num() > 0 || TablePin->DefaultObject == nullptr);
		if (bIsTypeAuthority)
		{
			RefreshOutputPinType();
		}		
	}
	else if (Pin == GetDataTablePin())
	{
		const bool bConnectionAdded = Pin->LinkedTo.Num() > 0;
		if (bConnectionAdded)
		{
			// if a connection was made, then we may need to rid ourselves of the row dropdown
			RefreshRowNameOptions();
			// if the output connection was previously, incompatible, it now becomes the authority on this node's output type
			RefreshOutputPinType();
		}
	}
}

#undef LOCTEXT_NAMESPACE
