// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GetEnumeratorName.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetNodeHelperLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetEnumeratorName)

struct FLinearColor;

FName UK2Node_GetEnumeratorName::EnumeratorPinName(TEXT("Enumerator"));

UK2Node_GetEnumeratorName::UK2Node_GetEnumeratorName(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_GetEnumeratorName::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, EnumeratorPinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Name, UEdGraphSchema_K2::PN_ReturnValue);
}

FText UK2Node_GetEnumeratorName::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "GetEnumeratorName_Tooltip", "Returns name of enumerator");
}

FText UK2Node_GetEnumeratorName::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "GetNode_Title", "Enum to Name");
}

FText UK2Node_GetEnumeratorName::GetCompactNodeTitle() const
{
	return NSLOCTEXT("K2Node", "CastSymbol", "\u2022");
}

UEnum* UK2Node_GetEnumeratorName::GetConnectedEnum() const
{
	const UEdGraphPin* InputPin = FindPinChecked(EnumeratorPinName);
	const UEdGraphPin* EnumPin = InputPin->LinkedTo.Num() ? InputPin->LinkedTo[0] : nullptr;
	return EnumPin ? Cast<UEnum>(EnumPin->PinType.PinSubCategoryObject.Get()) : nullptr;
}

void UK2Node_GetEnumeratorName::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	const UEdGraphPin* OutputPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue); 
	/*Don't validate isolated nodes */
	if (0 != OutputPin->LinkedTo.Num())
	{
		EarlyValidation(MessageLog);
	}
}

FSlateIcon UK2Node_GetEnumeratorName::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x");
	return Icon;
}

void UK2Node_GetEnumeratorName::EarlyValidation(class FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	const UEnum* Enum = GetConnectedEnum();
	if (Enum == nullptr)
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "GetNumEnumEntries_NoIntput_Error", "@@ Must have non-default Enum input").ToString(), this);
	}
}

bool UK2Node_GetEnumeratorName::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	const UEdGraphPin* InputPin = FindPinChecked(EnumeratorPinName);
	if((InputPin == MyPin) && OtherPin && (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte))
	{
		if(NULL == Cast<UEnum>(OtherPin->PinType.PinSubCategoryObject.Get()))
		{
			OutReason = NSLOCTEXT("K2Node", "GetNumEnumEntries_NotEnum_Msg", "Input is not an Enum.").ToString();
			return true;
		}
	}

	return false;
}

FName UK2Node_GetEnumeratorName::GetFunctionName() const
{
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, GetEnumeratorName);
	return FunctionName;
}

void UK2Node_GetEnumeratorName::PostReconstructNode()
{
	UEdGraphPin* EnumPin = FindPinChecked(EnumeratorPinName);
	EnumPin->PinType.PinSubCategoryObject = GetConnectedEnum();

	Super::PostReconstructNode();
}

void UK2Node_GetEnumeratorName::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	// The corresponding SGraphPinEnum will effectively "cache" enum info (eg: it assumes a specific enumerator count).
	// If our connected enum type changes for any reason, then we need to rebuild the widget.

	UEdGraphPin* EnumPin = FindPinChecked(EnumeratorPinName);
	UEnum* Enum = GetConnectedEnum();
	if (EnumPin->PinType.PinSubCategoryObject != Enum)
	{
		ReconstructNode();
	}
}

void UK2Node_GetEnumeratorName::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEnum* Enum = GetConnectedEnum();
	if (Enum == nullptr)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetEnumeratorNam_Error_MustHaveValidName", "@@ must have a valid enum defined").ToString(), this);
		return;
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
		
	const UFunction* Function = UKismetNodeHelperLibrary::StaticClass()->FindFunctionByName( GetFunctionName() );
	check(NULL != Function);
	UK2Node_CallFunction* CallGetName = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph); 
	CallGetName->SetFromFunction(Function);
	CallGetName->AllocateDefaultPins();
	check(CallGetName->IsNodePure());
		
	// OUTPUT PIN
	UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* NewReturnPin = CallGetName->GetReturnValuePin();
	check(NewReturnPin);
	CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *NewReturnPin);

	// ENUM PIN
	UEdGraphPin* EnumPin = CallGetName->FindPinChecked(TEXT("Enum"));
	Schema->TrySetDefaultObject(*EnumPin, Enum);
	check(EnumPin->DefaultObject == Enum);

	// VALUE PIN
	UEdGraphPin* OrgInputPin = FindPinChecked(EnumeratorPinName);
	UEdGraphPin* IndexPin = CallGetName->FindPinChecked(TEXT("EnumeratorValue"));
	check(EGPD_Input == IndexPin->Direction && UEdGraphSchema_K2::PC_Byte == IndexPin->PinType.PinCategory);
	CompilerContext.MovePinLinksToIntermediate(*OrgInputPin, *IndexPin);

	if (!IndexPin->LinkedTo.Num())
	{
		// MAKE LITERAL BYTE FROM LITERAL ENUM
		const FString EnumLiteral = IndexPin->GetDefaultAsString();
		const int32 NumericValue = IntCastChecked<int32, int64>(Enum->GetValueByName(*EnumLiteral));
		if (NumericValue == INDEX_NONE) 
		{
			CompilerContext.MessageLog.Error(*FText::Format(NSLOCTEXT("K2Node", "GetEnumeratorNam_Error_InvalidNameFmt", "@@ has invalid enum value '{0}'"), FText::FromString(EnumLiteral)).ToString(), this);
			return;
		}
		const FString DefaultByteValue = FString::FromInt(NumericValue);

		// LITERAL BYTE FUNCTION
		const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralByte);
		UK2Node_CallFunction* MakeLiteralByte = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph); 
		MakeLiteralByte->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(FunctionName));
		MakeLiteralByte->AllocateDefaultPins();

		UEdGraphPin* MakeLiteralByteReturnPin = MakeLiteralByte->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
		Schema->TryCreateConnection(MakeLiteralByteReturnPin, IndexPin);

		UEdGraphPin* MakeLiteralByteInputPin = MakeLiteralByte->FindPinChecked(TEXT("Value"));
		MakeLiteralByteInputPin->DefaultValue = DefaultByteValue;
	}

	BreakAllNodeLinks();
}

void UK2Node_GetEnumeratorName::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_GetEnumeratorName::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Name);
}
