// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GetEnumeratorNameAsString.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetNodeHelperLibrary.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetEnumeratorNameAsString)

struct FLinearColor;

UK2Node_GetEnumeratorNameAsString::UK2Node_GetEnumeratorNameAsString(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_GetEnumeratorNameAsString::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, EnumeratorPinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_String, UEdGraphSchema_K2::PN_ReturnValue);
}

FText UK2Node_GetEnumeratorNameAsString::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "GetEnumeratorNameAsString_Tooltip", "Returns user friendly name of enumerator");
}

FText UK2Node_GetEnumeratorNameAsString::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "GetEnumeratorNameAsString_Title", "Enum to String");
}

FSlateIcon UK2Node_GetEnumeratorNameAsString::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x");
	return Icon;
}

void UK2Node_GetEnumeratorNameAsString::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FName UK2Node_GetEnumeratorNameAsString::GetFunctionName() const
{
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, GetEnumeratorUserFriendlyName);
	return FunctionName;
}

FText UK2Node_GetEnumeratorNameAsString::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::String);
}
