// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetInputAxisKeyValue.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetInputAxisKeyValue)

struct FLinearColor;

#define LOCTEXT_NAMESPACE "K2Node_GetInputAxisKeyValue"

UK2Node_GetInputAxisKeyValue::UK2Node_GetInputAxisKeyValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
}

void UK2Node_GetInputAxisKeyValue::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* InputAxisKeyPin = FindPinChecked(TEXT("InputAxisKey"));
	InputAxisKeyPin->DefaultValue = InputAxisKey.ToString();
}

void UK2Node_GetInputAxisKeyValue::Initialize(const FKey AxisKey)
{
	InputAxisKey = AxisKey;
	SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, GetInputAxisKeyValue)));

	CachedTooltip.MarkDirty();
	CachedNodeTitle.MarkDirty();
}

FText UK2Node_GetInputAxisKeyValue::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return InputAxisKey.GetDisplayName();
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AxisKey"), InputAxisKey.GetDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "GetInputAxisKey_Name", "Get {AxisKey}"), Args), this);
	}

	return CachedNodeTitle;
}

FText UK2Node_GetInputAxisKeyValue::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("AxisKey"), InputAxisKey.GetDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "GetInputAxisKey_Tooltip", "Returns the current value of input axis key {AxisKey}.  If input is disabled for the actor the value will be 0."), Args), this);
	}
	return CachedTooltip;
}

bool UK2Node_GetInputAxisKeyValue::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

	UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
	bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(Graph) : false;

	return (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
}

void UK2Node_GetInputAxisKeyValue::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!InputAxisKey.IsValid())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Invalid_GetInputAxisKey_Warning", "GetInputAxisKey Value specifies invalid FKey'{0}' for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (!InputAxisKey.IsAxis1D())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotAxis_GetInputAxisKey_Warning", "GetInputAxisKey Value specifies FKey'{0}' which is not a float axis for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (InputAxisKey.IsDeprecated())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Deprecated_GetInputAxisKey_Warning", "GetInputAxisKey Value specifies FKey'{0}' which has been deprecated for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (!InputAxisKey.IsBindableInBlueprints())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotBindanble_GetInputAxisKey_Warning", "GetInputAxisKey Value specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
}

UClass* UK2Node_GetInputAxisKeyValue::GetDynamicBindingClass() const
{
	return UInputAxisKeyDelegateBinding::StaticClass();
}

void UK2Node_GetInputAxisKeyValue::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputAxisKeyDelegateBinding* InputAxisKeyBindingObject = CastChecked<UInputAxisKeyDelegateBinding>(BindingObject);

	FBlueprintInputAxisKeyDelegateBinding Binding;
	Binding.AxisKey = InputAxisKey;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;

	InputAxisKeyBindingObject->InputAxisKeyDelegateBindings.Add(Binding);
}

FSlateIcon UK2Node_GetInputAxisKeyValue::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), EKeys::GetMenuCategoryPaletteIcon(InputAxisKey.GetMenuCategory()));
}

void UK2Node_GetInputAxisKeyValue::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_GetInputAxisKeyValue* InputNode = CastChecked<UK2Node_GetInputAxisKeyValue>(NewNode);
		InputNode->Initialize(Key);
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (const FKey& Key : AllKeys)
		{
			if (!Key.IsBindableInBlueprints() || !Key.IsAxis1D())
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_GetInputAxisKeyValue::GetMenuCategory() const
{
	static TMap<FName, FNodeTextCache> CachedCategories;

	const FName KeyCategory = InputAxisKey.GetMenuCategory();
	const FText SubCategoryDisplayName = FText::Format(LOCTEXT("EventsCategory", "{0} Values"), EKeys::GetMenuCategoryDisplayName(KeyCategory));
	FNodeTextCache& NodeTextCache = CachedCategories.FindOrAdd(KeyCategory);

	if (NodeTextCache.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		NodeTextCache.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, SubCategoryDisplayName), this);
	}
	return NodeTextCache;
}

FBlueprintNodeSignature UK2Node_GetInputAxisKeyValue::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputAxisKey.ToString());

	return NodeSignature;
}

#undef LOCTEXT_NAMESPACE
