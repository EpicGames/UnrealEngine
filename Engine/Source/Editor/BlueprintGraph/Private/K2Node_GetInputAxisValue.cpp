// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetInputAxisValue.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisDelegateBinding.h"
#include "GameFramework/Actor.h"
#include "GameFramework/InputSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetInputAxisValue)

#define LOCTEXT_NAMESPACE "K2Node_GetInputAxisValue"

UK2Node_GetInputAxisValue::UK2Node_GetInputAxisValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
}

void UK2Node_GetInputAxisValue::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphPin* InputAxisNamePin = FindPinChecked(TEXT("InputAxisName"));
	InputAxisNamePin->DefaultValue = InputAxisName.ToString();
}

void UK2Node_GetInputAxisValue::Initialize(const FName AxisName)
{
	InputAxisName = AxisName;
	SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, GetInputAxisValue)));
}

FText UK2Node_GetInputAxisValue::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(InputAxisName);
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputAxisName"), FText::FromName(InputAxisName));

		FText LocFormat = NSLOCTEXT("K2Node", "GetInputAxis_Name", "Get {InputAxisName}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
	}

	return CachedNodeTitle;
}

FText UK2Node_GetInputAxisValue::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "GetInputAxis_Tooltip", "Returns the current value of input axis {0}.  If input is disabled for the actor the value will be 0."), FText::FromName(InputAxisName)), this);
	}
	return CachedTooltip;
}

bool UK2Node_GetInputAxisValue::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

	UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
	bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(Graph) : false;

	return (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
}

void UK2Node_GetInputAxisValue::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	TArray<FName> AxisNames;
	GetDefault<UInputSettings>()->GetAxisNames(AxisNames);
	if (!AxisNames.Contains(InputAxisName))
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingInputAxis_WarningFmt", "Get Input Axis references unknown Axis '{0}' for @@"), FText::FromString(InputAxisName.ToString())).ToString(), this);
	}
}

UClass* UK2Node_GetInputAxisValue::GetDynamicBindingClass() const
{
	return UInputAxisDelegateBinding::StaticClass();
}

void UK2Node_GetInputAxisValue::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputAxisDelegateBinding* InputAxisBindingObject = CastChecked<UInputAxisDelegateBinding>(BindingObject);

	FBlueprintInputAxisDelegateBinding Binding;
	Binding.InputAxisName = InputAxisName;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = false;

	InputAxisBindingObject->InputAxisDelegateBindings.Add(Binding);
}

void UK2Node_GetInputAxisValue::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FName> AxisNames;
	GetDefault<UInputSettings>()->GetAxisNames(AxisNames);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FName AxisName)
	{
		UK2Node_GetInputAxisValue* InputNode = CastChecked<UK2Node_GetInputAxisValue>(NewNode);
		InputNode->Initialize(AxisName);
	};

	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		auto RefreshClassActions = []()
		{
			FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass());
		};

		static bool bRegisterOnce = true;
		if(bRegisterOnce)
		{
			bRegisterOnce = false;
			FEditorDelegates::OnActionAxisMappingsChanged.AddStatic(RefreshClassActions);
		}

		for (const FName& AxisName : AxisNames)
		{
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, AxisName);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_GetInputAxisValue::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, LOCTEXT("ActionMenuCategory", "Axis Values")), this);
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_GetInputAxisValue::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputAxisName.ToString());

	return NodeSignature;
}

#undef LOCTEXT_NAMESPACE
