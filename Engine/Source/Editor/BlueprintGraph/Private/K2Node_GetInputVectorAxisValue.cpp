// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetInputVectorAxisValue.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "Engine/InputVectorAxisDelegateBinding.h"
#include "GameFramework/Actor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetInputVectorAxisValue)

UK2Node_GetInputVectorAxisValue::UK2Node_GetInputVectorAxisValue(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
}

void UK2Node_GetInputVectorAxisValue::Initialize(const FKey AxisKey)
{
	InputAxisKey = AxisKey;
	SetFromFunction(AActor::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(AActor, GetInputVectorAxisValue)));
}

FText UK2Node_GetInputVectorAxisValue::GetTooltipText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AxisKey"), InputAxisKey.GetDisplayName());
	return FText::Format(NSLOCTEXT("K2Node", "GetInputVectorAxis_Tooltip", "Returns the current value of input axis key {AxisKey}.  If input is disabled for the actor the value will be (0, 0, 0)."), Args);
}

void UK2Node_GetInputVectorAxisValue::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	// Skip GetInputKeyAxisValue validation
	UK2Node_CallFunction::ValidateNodeDuringCompilation(MessageLog);

	if (!InputAxisKey.IsValid())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Invalid_GetInputVectorAxis_Warning", "GetInputVectorAxis Value specifies invalid FKey'{0}' for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (!InputAxisKey.IsAxis2D() && !InputAxisKey.IsAxis3D())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotAxis_GetInputVectorAxis_Warning", "GetInputVectorAxis Value specifies FKey'{0}' which is not a vector axis for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (InputAxisKey.IsDeprecated())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Deprecated_GetInputVectorAxis_Warning", "GetInputVectorAxis Value specifies FKey'{0}' which has been deprecated for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
	else if (!InputAxisKey.IsBindableInBlueprints())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotBindanble_GetInputVectorAxis_Warning", "GetInputVectorAxis Value specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(InputAxisKey.ToString())).ToString(), this);
	}
}

UClass* UK2Node_GetInputVectorAxisValue::GetDynamicBindingClass() const
{
	return UInputVectorAxisDelegateBinding::StaticClass();
}

void UK2Node_GetInputVectorAxisValue::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputVectorAxisDelegateBinding* InputVectorAxisBindingObject = CastChecked<UInputVectorAxisDelegateBinding>(BindingObject);

	FBlueprintInputAxisKeyDelegateBinding Binding;
	Binding.AxisKey = InputAxisKey;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;

	InputVectorAxisBindingObject->InputAxisKeyDelegateBindings.Add(Binding);
}

void UK2Node_GetInputVectorAxisValue::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_GetInputVectorAxisValue* InputNode = CastChecked<UK2Node_GetInputVectorAxisValue>(NewNode);
		InputNode->Initialize(Key);
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (const FKey& Key : AllKeys)
		{
			if (!Key.IsBindableInBlueprints() || !(InputAxisKey.IsAxis2D() || InputAxisKey.IsAxis3D()))
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);
			NodeSpawner->DefaultMenuSignature.MenuName = FText::Format(NSLOCTEXT("K2Node_GetInputVectorAxisValue", "MenuName", "Get {0}"), Key.GetDisplayName());

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}
