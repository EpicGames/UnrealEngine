// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_GetNumEnumEntries.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFieldNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Field.h"
#include "UObject/LinkerLoad.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetNumEnumEntries)

class UBlueprintNodeSpawner;
struct FLinearColor;

UK2Node_GetNumEnumEntries::UK2Node_GetNumEnumEntries(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_GetNumEnumEntries::AllocateDefaultPins()
{
	// Create the return value pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Int, UEdGraphSchema_K2::PN_ReturnValue);

	Super::AllocateDefaultPins();
}

FText UK2Node_GetNumEnumEntries::GetTooltipText() const
{
	if (Enum == nullptr)
	{
		return NSLOCTEXT("K2Node", "GetNumEnumEntries_BadTooltip", "Returns (bad enum)_MAX value");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "GetNumEnumEntries_Tooltip", "Returns {0}_MAX value"), FText::FromName(Enum->GetFName())), this);
	}
	return CachedTooltip;
}

FText UK2Node_GetNumEnumEntries::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Enum == nullptr)
	{
		return NSLOCTEXT("K2Node", "GetNumEnumEntries_BadEnumTitle", "Get number of entries in (bad enum)");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumName"), FText::FromString(Enum->GetName()));
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "GetNumEnumEntries_Title", "Get number of entries in {EnumName}"), Args), this);
	}
	return CachedNodeTitle;
}

FSlateIcon UK2Node_GetNumEnumEntries::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Enum_16x");
	return Icon;
}

void UK2Node_GetNumEnumEntries::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if(NULL == Enum)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetNumEnumEntries_Error", "@@ must have a valid enum defined").ToString(), this);
		return;
	}

	// Force the enum to load its values if it hasn't already
	Enum->ConditionalPreload();

	//MAKE LITERAL
	const FName FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, MakeLiteralInt);
	UK2Node_CallFunction* MakeLiteralInt = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph); 
	MakeLiteralInt->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(FunctionName));
	MakeLiteralInt->AllocateDefaultPins();

	//OPUTPUT PIN
	UEdGraphPin* OrgReturnPin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* NewReturnPin = MakeLiteralInt->GetReturnValuePin();
	check(NULL != NewReturnPin);
	CompilerContext.MovePinLinksToIntermediate(*OrgReturnPin, *NewReturnPin);

	//INPUT PIN
	UEdGraphPin* InputPin = MakeLiteralInt->FindPinChecked(TEXT("Value"));
	check(EGPD_Input == InputPin->Direction);
	const FString DefaultValue = FString::FromInt(Enum->NumEnums() - 1);
	InputPin->DefaultValue = DefaultValue;

	BreakAllNodeLinks();
}

void UK2Node_GetNumEnumEntries::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, FFieldVariant /*EnumField*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UK2Node_GetNumEnumEntries* EnumNode = CastChecked<UK2Node_GetNumEnumEntries>(NewNode);
			EnumNode->Enum = NonConstEnumPtr.Get();
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* InEnum)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UEnum*>(InEnum));
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(InEnum));
		NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

FText UK2Node_GetNumEnumEntries::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Enum);
}

void UK2Node_GetNumEnumEntries::ReloadEnum(class UEnum* InEnum)
{
	Enum = InEnum;
	CachedTooltip.MarkDirty();
	CachedNodeTitle.MarkDirty();
}
