// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_GetSubsystem.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintEditorSettings.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorSubsystem.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GameFramework/PlayerController.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Preferences/UnrealEdOptions.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "Subsystems/EditorSubsystemBlueprintLibrary.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/Subsystem.h"
#include "Subsystems/SubsystemBlueprintLibrary.h"
#include "Subsystems/WorldSubsystem.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectHash.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealEdGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_GetSubsystem)

class FArchive;
class UEdGraph;

// ************************************************************************************
//    UK2Node_GetSubsystem
// ************************************************************************************

static FName SubsystemWorldContextPinName("WorldContext");

void UK2Node_GetSubsystem::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	if (CustomClass == nullptr)
	{
		if (UEdGraphPin* ClassPin = FindPin(TEXT("Class"), EGPD_Input))
		{
			ClassPin->SetSavePinIfOrphaned(false);
		}
	}
}

void UK2Node_GetSubsystem::Initialize( UClass* NodeClass )
{
	CustomClass = NodeClass;
}

void UK2Node_GetSubsystem::AllocateDefaultPins()
{
	// If required add the world context pin
	if (FBlueprintEditorUtils::ShouldShowWorldContextPin(GetBlueprint()))
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), SubsystemWorldContextPinName);
	}

	// Add blueprint pin
	if (!CustomClass)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, USubsystem::StaticClass(), TEXT("Class"));
	}

	// Result pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, (CustomClass ? (UClass*)CustomClass : USubsystem::StaticClass()), UEdGraphSchema_K2::PN_ReturnValue);

	Super::AllocateDefaultPins();
}

FString UK2Node_GetSubsystem::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData = Super::GetPinMetaData(InPinName, InKey);

	if (MetaData.IsEmpty())
	{
		if (InPinName == SubsystemWorldContextPinName && InKey == FBlueprintMetadata::MD_HidePinAssetPicker)
		{
			MetaData = TEXT("true");
		}
	}

	return MetaData;
}

bool UK2Node_GetSubsystem::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return Super::IsCompatibleWithGraph(TargetGraph) &&
		(!Blueprint || FBlueprintEditorUtils::FindUserConstructionScript(Blueprint) != TargetGraph);
}

FSlateIcon UK2Node_GetSubsystem::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = GetNodeTitleColor();
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.FunctionIcon");
	return Icon;
}

FLinearColor UK2Node_GetSubsystem::GetNodeTitleColor() const
{
	return FLinearColor(1.f, 0.078f, 0.576f, 1.f);
}

FText UK2Node_GetSubsystem::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CustomClass)
	{
		if (TitleType == ENodeTitleType::FullTitle)
		{
			return CustomClass->GetDisplayNameText();
		}

		const FString& ClassName = CustomClass->GetName();
		return FText::FormatNamed(NSLOCTEXT("K2Node", "GetSubsystem_NodeTitleFormat", "Get {ClassName}"), TEXT("ClassName"), FText::FromString(ClassName));
	}

	return GetTooltipText();
}

void UK2Node_GetSubsystem::GetNodeAttributes(TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes) const
{
	const FString ClassToSpawnStr = CustomClass ? CustomClass->GetName() : TEXT("InvalidClass");
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Type"), TEXT("GetSubsystems")));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Class"), GetClass()->GetName()));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("Name"), GetName()));
	OutNodeAttributes.Add(TKeyValuePair<FString, FString>(TEXT("ObjectClass"), ClassToSpawnStr));
}

void UK2Node_GetSubsystem::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	static const FName WorldContextObject_ParamName(TEXT("ContextObject"));
	static const FName Class_ParamName(TEXT("Class"));

	UK2Node_GetSubsystem* GetSubsystemNode = this;
	UEdGraphPin* SpawnWorldContextPin = GetSubsystemNode->GetWorldContextPin();
	UEdGraphPin* SpawnClassPin = GetSubsystemNode->GetClassPin();
	UEdGraphPin* SpawnNodeResult = GetSubsystemNode->GetResultPin();

	UClass* SpawnClass = (SpawnClassPin != nullptr) ? Cast<UClass>(SpawnClassPin->DefaultObject) : nullptr;
	if (SpawnClassPin && (SpawnClassPin->LinkedTo.Num() == 0) && !SpawnClass)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetSubsystemNode);
		GetSubsystemNode->BreakAllNodeLinks();
		return;
	}

	// Choose appropriate underlying Getter
	FName Get_FunctionName;
	if (CustomClass->IsChildOf<UGameInstanceSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetGameInstanceSubsystem);
	}
	else if (CustomClass->IsChildOf<UWorldSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetWorldSubsystem);
	}
	else if (CustomClass->IsChildOf<ULocalPlayerSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetLocalPlayerSubsystem);
	}
	else if (CustomClass->IsChildOf<UAudioEngineSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetAudioEngineSubsystem);
	}
	else
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetSubsystemNode);
		GetSubsystemNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'USubsystemBlueprintLibrary::Get[something]Subsystem' call node
	UK2Node_CallFunction* CallGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(GetSubsystemNode, SourceGraph);
	CallGetNode->FunctionReference.SetExternalMember(Get_FunctionName, USubsystemBlueprintLibrary::StaticClass());
	CallGetNode->AllocateDefaultPins();

	UEdGraphPin* CallCreateWorldContextPin = CallGetNode->FindPinChecked(WorldContextObject_ParamName);
	UEdGraphPin* CallCreateClassTypePin = CallGetNode->FindPinChecked(Class_ParamName);
	UEdGraphPin* CallCreateResult = CallGetNode->GetReturnValuePin();

	if (SpawnClassPin && SpawnClassPin->LinkedTo.Num() > 0)
	{
		// Copy the 'class' connection from the spawn node to 'USubsystemBlueprintLibrary::Get[something]Subsystem'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateClassTypePin);
	}
	else
	{
		// Copy class literal onto 'USubsystemBlueprintLibrary::Get[something]Subsystem' call
		CallCreateClassTypePin->DefaultObject = *CustomClass;
	}

	// Copy the world context connection from the spawn node to 'USubsystemBlueprintLibrary::Get[something]Subsystem' if necessary
	if (SpawnWorldContextPin)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnWorldContextPin, *CallCreateWorldContextPin);
	}

	// Move result connection from spawn node to 'USubsystemBlueprintLibrary::Get[something]Subsystem'
	CallCreateResult->PinType = SpawnNodeResult->PinType;
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////

	// Break any links to the expanded node
	GetSubsystemNode->BreakAllNodeLinks();
}

void UK2Node_GetSubsystem::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	if (!CustomClass)
	{
		if (auto ClassNode = GetClassPin(&OldPins))
		{
			CustomClass = Cast<UClass>(ClassNode->DefaultObject);
		}
	}

	AllocateDefaultPins();

	if (CustomClass)
	{
		UEdGraphPin* ResultPin = GetResultPin();
		ResultPin->PinType.PinSubCategoryObject = *CustomClass;
	}
}

class FNodeHandlingFunctor* UK2Node_GetSubsystem::CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_GetSubsystem::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	static TArray<UClass*> Subclasses;
	Subclasses.Reset();
	GetDerivedClasses(UGameInstanceSubsystem::StaticClass(), Subclasses);
	GetDerivedClasses(UWorldSubsystem::StaticClass(), Subclasses);
	GetDerivedClasses(ULocalPlayerSubsystem::StaticClass(), Subclasses);
	GetDerivedClasses(UAudioEngineSubsystem::StaticClass(), Subclasses);

	auto CustomizeCallback = [](UEdGraphNode* Node, bool bIsTemplateNode, UClass* Subclass)
	{
		auto TypedNode = CastChecked<UK2Node_GetSubsystem>(Node);
		TypedNode->Initialize(Subclass);
	};

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (UClass* Iter : Subclasses)
		{
			if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Iter, true))
			{
				continue;
			}

			UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(ActionKey);
			check(Spawner);

			Spawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCallback, Iter);
			ActionRegistrar.AddBlueprintAction(ActionKey, Spawner);
		}
	}
}

FText UK2Node_GetSubsystem::GetMenuCategory() const
{
	if (CustomClass->IsChildOf<UGameInstanceSubsystem>())
	{
		return NSLOCTEXT("K2Node", "GetSubsystem_GameInstanceSubsystemsMenuCategory", "GameInstance Subsystems");
	}
	else if (CustomClass->IsChildOf<ULocalPlayerSubsystem>())
	{
		return NSLOCTEXT("K2Node", "GetSubsystem_LocalPlayerSubsystemsMenuCategory", "LocalPlayer Subsystems");
	}
	else if (CustomClass->IsChildOf<UWorldSubsystem>())
	{
		return NSLOCTEXT("K2Node", "GetSubsystem_WorldSubsystemsMenuCategory", "World Subsystems");
	}
	else if (CustomClass->IsChildOf<UAudioEngineSubsystem>())
	{
		return NSLOCTEXT("K2Node", "GetSubsystem_AudioEngineSubsystemsMenuCategory", "AudioEngine Subsystems");
	}

	return NSLOCTEXT("K2Node", "GetSubsystem_InvalidSubsystemTypeMenuCategory", "Invalid Subsystem Type");
}

FText UK2Node_GetSubsystem::GetTooltipText() const
{
	if (CustomClass)
	{
		FText SubsystemTypeText;
		FText SubsystemNoteText;
		if (CustomClass->IsChildOf<UGameInstanceSubsystem>())
		{
			SubsystemTypeText =  NSLOCTEXT("K2Node", "GetSubsystem_GameInstanceSubsystemTooltip", "GameInstance Subsystem");
		}
		else if (CustomClass->IsChildOf<UWorldSubsystem>())
		{
			SubsystemTypeText = NSLOCTEXT("K2Node", "GetSubsystem_WorldSubsystemTooltip", "World Subsystem");
		}
		else if (CustomClass->IsChildOf<UAudioEngineSubsystem>())
		{
			SubsystemTypeText = NSLOCTEXT("K2Node", "GetSubsystem_AudioEngineSubsystemTooltip", "AudioEngine Subsystem");
		}
		else
		{
			SubsystemTypeText = NSLOCTEXT("K2Node", "GetSubsystem_LocalPlayerSubsystemTooltip_Label", "LocalPlayer Subsystem");
			SubsystemNoteText = NSLOCTEXT("K2Node", "GetSubsystem_LocalPlayerSubsystemTooltip_Details", "\n\nNote: This will only be valid if there is an available local player to this object.");
		}
		return FText::FormatNamed(NSLOCTEXT("K2Node", "GetSubsystem_TooltipFormat", "Get {ClassName} ({SubsystemType})\n\n{ClassTooltip}{SubsystemNote}"),
			TEXT("ClassName"), CustomClass->GetDisplayNameText(),
			TEXT("SubsystemType"), SubsystemTypeText,
			TEXT("ClassTooltip"), CustomClass->GetToolTipText(/*bShortTooltip=*/ true),
			TEXT("SubsystemNote"), SubsystemNoteText);
	}

	return NSLOCTEXT("K2Node", "GetSubsystem_InvalidSubsystemTypeTooltip", "Invalid Subsystem Type");
}

UEdGraphPin* UK2Node_GetSubsystem::GetWorldContextPin() const
{
	UEdGraphPin* Pin = FindPin(SubsystemWorldContextPinName);
	check(Pin == NULL || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_GetSubsystem::GetResultPin() const
{
	UEdGraphPin* Pin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}


UEdGraphPin* UK2Node_GetSubsystem::GetClassPin(const TArray<UEdGraphPin*>* InPinsToSearch /*= nullptr */) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* Pin = NULL;
	for (auto PinIt = PinsToSearch->CreateConstIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* TestPin = *PinIt;
		if (TestPin && TestPin->PinName == TEXT("Class"))
		{
			Pin = TestPin;
			break;
		}
	}

	return Pin;
}

// ************************************************************************************
//    UK2Node_GetSubsystemFromPC
// ************************************************************************************

void UK2Node_GetSubsystemFromPC::AllocateDefaultPins()
{
	// If required add the world context pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, APlayerController::StaticClass(), TEXT("PlayerController"));

	// Add blueprint pin
	if (!CustomClass)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, USubsystem::StaticClass(), TEXT("Class"));
	}

	// Result pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, (CustomClass ? (UClass*)CustomClass : ULocalPlayerSubsystem::StaticClass()), UEdGraphSchema_K2::PN_ReturnValue);

	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::AllocateDefaultPins();
}

void UK2Node_GetSubsystemFromPC::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::ExpandNode(CompilerContext, SourceGraph);

	static const FName PlayerController_ParamName(TEXT("PlayerController"));
	static const FName Class_ParamName(TEXT("Class"));

	UK2Node_GetSubsystemFromPC* GetSubsystemFromPCNode = this;
	UEdGraphPin* SpawnPlayerControllerPin = GetSubsystemFromPCNode->GetPlayerControllerPin();
	UEdGraphPin* SpawnClassPin = GetSubsystemFromPCNode->GetClassPin();
	UEdGraphPin* SpawnNodeResult = GetSubsystemFromPCNode->GetResultPin();

	UClass* SpawnClass = (SpawnClassPin != nullptr) ? Cast<UClass>(SpawnClassPin->DefaultObject) : nullptr;
	if (SpawnClassPin && (SpawnClassPin->LinkedTo.Num() == 0) && !SpawnClass)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetSubsystemFromPCNode);
		GetSubsystemFromPCNode->BreakAllNodeLinks();
		return;
	}

	// Choose appropriate underlying Getter
	FName Get_FunctionName;
	if (CustomClass->IsChildOf<ULocalPlayerSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetLocalPlayerSubSystemFromPlayerController);
	}
	else
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetSubsystemFromPCNode);
		GetSubsystemFromPCNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController' call node
	UK2Node_CallFunction* CallGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(GetSubsystemFromPCNode, SourceGraph);
	CallGetNode->FunctionReference.SetExternalMember(Get_FunctionName, USubsystemBlueprintLibrary::StaticClass());
	CallGetNode->AllocateDefaultPins();

	UEdGraphPin* CallCreatePlayerControllerPin = CallGetNode->FindPinChecked(PlayerController_ParamName);
	UEdGraphPin* CallCreateClassTypePin = CallGetNode->FindPinChecked(Class_ParamName);
	UEdGraphPin* CallCreateResult = CallGetNode->GetReturnValuePin();

	if (SpawnClassPin && SpawnClassPin->LinkedTo.Num() > 0)
	{
		// Copy the 'class' connection from the spawn node to 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateClassTypePin);
	}
	else
	{
		// Copy class literal onto 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController' call
		CallCreateClassTypePin->DefaultObject = *CustomClass;
	}

	// Copy the world context connection from the spawn node to 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController' if necessary
	if (SpawnPlayerControllerPin)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnPlayerControllerPin, *CallCreatePlayerControllerPin);
	}

	// Move result connection from spawn node to 'USubsystemBlueprintLibrary::Get[something]Subsystem'
	CallCreateResult->PinType = SpawnNodeResult->PinType;
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////

	// Break any links to the expanded node
	GetSubsystemFromPCNode->BreakAllNodeLinks();
}

void UK2Node_GetSubsystemFromPC::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	static TArray<UClass*> Subclasses;
	Subclasses.Reset();
	GetDerivedClasses(ULocalPlayerSubsystem::StaticClass(), Subclasses);

	auto CustomizeCallback = [](UEdGraphNode* Node, bool bIsTemplateNode, UClass* Subclass)
	{
		auto TypedNode = CastChecked<UK2Node_GetSubsystemFromPC>(Node);
		TypedNode->Initialize(Subclass);
	};

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (UClass* Iter : Subclasses)
		{
			if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Iter, true))
			{
				continue;
			}

			UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(ActionKey);
			check(Spawner);

			Spawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCallback, Iter);
			ActionRegistrar.AddBlueprintAction(ActionKey, Spawner);
		}
	}
}

FText UK2Node_GetSubsystemFromPC::GetMenuCategory() const
{
	return NSLOCTEXT("K2Node", "GetSubsystemFromPC_MenuCategory", "PlayerController|LocalPlayer Subsystems");
}

FText UK2Node_GetSubsystemFromPC::GetTooltipText() const
{
	if (CustomClass)
	{
		return
			FText::FormatNamed(NSLOCTEXT("K2Node", "GetSubsystemFromPC_TooltipFormat", "Get {ClassName} from Player Controller\n\n{ClassTooltip}"),
			TEXT("ClassName"), CustomClass->GetDisplayNameText(),
			TEXT("ClassTooltip"), CustomClass->GetToolTipText(/*bShortTooltip=*/ true));
	}

	return NSLOCTEXT("K2Node", "GetSubsystemFromPC_InvalidSubsystemTypeTooltip", "Invalid Subsystem Type");
}

UEdGraphPin* UK2Node_GetSubsystemFromPC::GetPlayerControllerPin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("PlayerController"));
	check(Pin == NULL || Pin->Direction == EGPD_Input);
	return Pin;
}


// ************************************************************************************
//    UK2Node_GetEngineSubsystem
// ************************************************************************************

void UK2Node_GetEngineSubsystem::AllocateDefaultPins()
{
	// Add blueprint pin
	if (!CustomClass)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, USubsystem::StaticClass(), TEXT("Class"));
	}

	// Result pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, (CustomClass ? (UClass*)CustomClass : UEngineSubsystem::StaticClass()), UEdGraphSchema_K2::PN_ReturnValue);

	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::AllocateDefaultPins();
}

void UK2Node_GetEngineSubsystem::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::ExpandNode(CompilerContext, SourceGraph);

	static const FName Class_ParamName(TEXT("Class"));

	UK2Node_GetEngineSubsystem* GetEngineSubsystemNode = this;
	UEdGraphPin* SpawnClassPin = GetEngineSubsystemNode->GetClassPin();
	UEdGraphPin* SpawnNodeResult = GetEngineSubsystemNode->GetResultPin();

	UClass* SpawnClass = (SpawnClassPin != nullptr) ? Cast<UClass>(SpawnClassPin->DefaultObject) : nullptr;
	if (SpawnClassPin && (SpawnClassPin->LinkedTo.Num() == 0) && !SpawnClass)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetEngineSubsystemNode);
		GetEngineSubsystemNode->BreakAllNodeLinks();
		return;
	}

	// Choose appropriate underlying Getter
	FName Get_FunctionName;
	if (CustomClass->IsChildOf<UEngineSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(USubsystemBlueprintLibrary, GetEngineSubsystem);
	}
	else
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetEngineSubsystemNode);
		GetEngineSubsystemNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'USubsystemBlueprintLibrary::GetEngineSubsystem' call node
	UK2Node_CallFunction* CallGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(GetEngineSubsystemNode, SourceGraph);
	CallGetNode->FunctionReference.SetExternalMember(Get_FunctionName, USubsystemBlueprintLibrary::StaticClass());
	CallGetNode->AllocateDefaultPins();

	UEdGraphPin* CallCreateClassTypePin = CallGetNode->FindPinChecked(Class_ParamName);
	UEdGraphPin* CallCreateResult = CallGetNode->GetReturnValuePin();

	if (SpawnClassPin && SpawnClassPin->LinkedTo.Num() > 0)
	{
		// Copy the 'class' connection from the spawn node to 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateClassTypePin);
	}
	else
	{
		// Copy class literal onto 'USubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController' call
		CallCreateClassTypePin->DefaultObject = *CustomClass;
	}

	// Move result connection from spawn node to 'USubsystemBlueprintLibrary::Get[something]Subsystem'
	CallCreateResult->PinType = SpawnNodeResult->PinType;
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////

	// Break any links to the expanded node
	GetEngineSubsystemNode->BreakAllNodeLinks();
}

void UK2Node_GetEngineSubsystem::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	static TArray<UClass*> Subclasses;
	Subclasses.Reset();
	GetDerivedClasses(UEngineSubsystem::StaticClass(), Subclasses);

	auto CustomizeCallback = [](UEdGraphNode* Node, bool bIsTemplateNode, UClass* Subclass)
	{
		auto TypedNode = CastChecked<UK2Node_GetEngineSubsystem>(Node);
		TypedNode->Initialize(Subclass);
	};

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (UClass* Iter : Subclasses)
		{
			if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Iter, true))
			{
				continue;
			}

			UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(ActionKey);
			check(Spawner);

			Spawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCallback, Iter);
			ActionRegistrar.AddBlueprintAction(ActionKey, Spawner);
		}
	}
}

FText UK2Node_GetEngineSubsystem::GetMenuCategory() const
{
	return NSLOCTEXT("K2Node", "GetEngineSubsystem_MenuCategory", "Engine Subsystems");
}

FText UK2Node_GetEngineSubsystem::GetTooltipText() const
{
	if (CustomClass)
	{
		return FText::FormatNamed(NSLOCTEXT("K2Node", "GetEngineSubsystem_TooltipFormat", "Get {ClassName} an Engine Subsystem"), TEXT("ClassName"), CustomClass->GetDisplayNameText());
	}

	return NSLOCTEXT("K2Node", "GetEngineSubsystem_InvalidSubsystemTypeTooltip", "Invalid Subsystem Type");
}

// ************************************************************************************
//    UK2Node_GetEditorSubsystem
// ************************************************************************************

void UK2Node_GetEditorSubsystem::AllocateDefaultPins()
{
	// Add blueprint pin
	if (!CustomClass)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Class, USubsystem::StaticClass(), TEXT("Class"));
	}

	// Result pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, (CustomClass ? (UClass*)CustomClass : UEditorSubsystem::StaticClass()), UEdGraphSchema_K2::PN_ReturnValue);

	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::AllocateDefaultPins();
}

void UK2Node_GetEditorSubsystem::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// Skip the UK2Node_GetSubsystem implementation
	UK2Node::ExpandNode(CompilerContext, SourceGraph);

	static const FName Class_ParamName(TEXT("Class"));

	UK2Node_GetEditorSubsystem* GetEditorSubsystemNode = this;
	UEdGraphPin* SpawnClassPin = GetEditorSubsystemNode->GetClassPin();
	UEdGraphPin* SpawnNodeResult = GetEditorSubsystemNode->GetResultPin();

	UClass* SpawnClass = (SpawnClassPin != nullptr) ? Cast<UClass>(SpawnClassPin->DefaultObject) : nullptr;
	if (SpawnClassPin && (SpawnClassPin->LinkedTo.Num() == 0) && !SpawnClass)
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetEditorSubsystemNode);
		GetEditorSubsystemNode->BreakAllNodeLinks();
		return;
	}

	// Choose appropriate underlying Getter
	FName Get_FunctionName;
	if (CustomClass->IsChildOf<UEditorSubsystem>())
	{
		Get_FunctionName = GET_FUNCTION_NAME_CHECKED(UEditorSubsystemBlueprintLibrary, GetEditorSubsystem);
	}
	else
	{
		CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "GetSubsystem_Error", "Node @@ must have a class specified.").ToString(), GetEditorSubsystemNode);
		GetEditorSubsystemNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'UEditorSubsystemBlueprintLibrary::GetEditorSubsystem' call node
	UK2Node_CallFunction* CallGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(GetEditorSubsystemNode, SourceGraph);
	CallGetNode->FunctionReference.SetExternalMember(Get_FunctionName, UEditorSubsystemBlueprintLibrary::StaticClass());
	CallGetNode->AllocateDefaultPins();

	UEdGraphPin* CallCreateClassTypePin = CallGetNode->FindPinChecked(Class_ParamName);
	UEdGraphPin* CallCreateResult = CallGetNode->GetReturnValuePin();

	if (SpawnClassPin && SpawnClassPin->LinkedTo.Num() > 0)
	{
		// Copy the 'class' connection from the spawn node to 'USubsystemUEditorSubsystemBlueprintLibraryBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallCreateClassTypePin);
	}
	else
	{
		// Copy class literal onto 'UEditorSubsystemBlueprintLibrary::GetLocalPlayerSubSystemFromPlayerController' call
		CallCreateClassTypePin->DefaultObject = *CustomClass;
	}

	// Move result connection from spawn node to 'UEditorSubsystemBlueprintLibrary::Get[something]Subsystem'
	CallCreateResult->PinType = SpawnNodeResult->PinType;
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallCreateResult);

	//////////////////////////////////////////////////////////////////////////

	// Break any links to the expanded node
	GetEditorSubsystemNode->BreakAllNodeLinks();
}

void UK2Node_GetEditorSubsystem::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	static TArray<UClass*> Subclasses;
	Subclasses.Reset();
	GetDerivedClasses(UEditorSubsystem::StaticClass(), Subclasses);

	auto CustomizeCallback = [](UEdGraphNode* Node, bool bIsTemplateNode, UClass* Subclass)
	{
		auto TypedNode = CastChecked<UK2Node_GetEditorSubsystem>(Node);
		TypedNode->Initialize(Subclass);
	};

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (UClass* Iter : Subclasses)
		{
			if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Iter, true))
			{
				continue;
			}

			UBlueprintNodeSpawner* Spawner = UBlueprintNodeSpawner::Create(ActionKey);
			check(Spawner);

			Spawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeCallback, Iter);
			ActionRegistrar.AddBlueprintAction(ActionKey, Spawner);
		}
	}
}

FText UK2Node_GetEditorSubsystem::GetMenuCategory() const
{
	return NSLOCTEXT("K2Node", "GetEditorSubsystem_MenuCategory", "Editor Subsystems");
}

FText UK2Node_GetEditorSubsystem::GetTooltipText() const
{
	if (CustomClass)
	{
		return FText::FormatNamed(NSLOCTEXT("K2Node", "GetEditorSubsystem_TooltipFormat", "Get {ClassName} an Editor Subsystem"), TEXT("ClassName"), CustomClass->GetDisplayNameText());
	}
	 
	return NSLOCTEXT("K2Node", "GetEditorSubsystem_InvalidSubsystemTypeTooltip", "Invalid Subsystem Type");
}

bool UK2Node_GetSubsystem::CanJumpToDefinition() const
{
	if (CustomClass && ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		return true;
	}
	return Super::CanJumpToDefinition();
}

void UK2Node_GetSubsystem::JumpToDefinition() const
{
	bool bSucceeded = false;
	
	if (CustomClass && ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
		// Attempt to navigate to the header file where the class is defined if the 
		// blueprint preferences allow for it
		if (GetDefault<UBlueprintEditorSettings>()->bNavigateToNativeFunctionsFromCallNodes)
		{
			if (FSourceCodeNavigation::CanNavigateToClass(CustomClass))
			{
				bSucceeded = FSourceCodeNavigation::NavigateToClass(CustomClass);
			}
		}
	}

	// Otherwise fall back to the base class which will just bring focus to this node
	if (!bSucceeded)
	{
		Super::JumpToDefinition();
	}
}

bool UK2Node_GetEditorSubsystem::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	for(UBlueprint* BP : Filter.Context.Blueprints)
	{
		if (!FBlueprintEditorUtils::IsEditorUtilityBlueprint(BP))
		{
			return true;
		}
	}
	return false;
}

void UK2Node_GetEditorSubsystem::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UBlueprint* BP = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);

	if (!FBlueprintEditorUtils::IsEditorUtilityBlueprint(BP))
	{
		const FText ErrorText = NSLOCTEXT("K2Node", "GetEditorSubsystem_Error", "Editor Subsystems can only be used in Editor Utilities / Blutilities");
		MessageLog.Error(*ErrorText.ToString(), this);
	}
}
