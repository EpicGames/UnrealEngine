// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_IfThenElse.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_IfThenElse)

struct FBPTerminal;

#define LOCTEXT_NAMESPACE "K2Node"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_Branch

class FKCHandler_Branch : public FNodeHandlingFunctor
{
public:
	FKCHandler_Branch(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		// For imperative nodes, make sure the exec function was actually triggered and not just included due to an output data dependency
		FEdGraphPinType ExpectedExecPinType;
		ExpectedExecPinType.PinCategory = UEdGraphSchema_K2::PC_Exec;

		FEdGraphPinType ExpectedBoolPinType;
		ExpectedBoolPinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;


		UEdGraphPin* ExecTriggeringPin = Context.FindRequiredPinByName(Node, UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		if ((ExecTriggeringPin == NULL) || !Context.ValidatePinType(ExecTriggeringPin, ExpectedExecPinType))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoValidExecutionPinForBranch_Error", "@@ must have a valid execution pin @@").ToString(), Node, ExecTriggeringPin);
			return;
		}
		else if (ExecTriggeringPin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Warning(*LOCTEXT("NodeNeverExecuted_Warning", "@@ will never be executed").ToString(), Node);
			return;
		}

		// Generate the output impulse from this node
		UEdGraphPin* CondPin = Context.FindRequiredPinByName(Node, UEdGraphSchema_K2::PN_Condition, EGPD_Input);
		UEdGraphPin* ThenPin = Context.FindRequiredPinByName(Node, UEdGraphSchema_K2::PN_Then, EGPD_Output);
		UEdGraphPin* ElsePin = Context.FindRequiredPinByName(Node, UEdGraphSchema_K2::PN_Else, EGPD_Output);
		if (Context.ValidatePinType(ThenPin, ExpectedExecPinType) &&
			Context.ValidatePinType(ElsePin, ExpectedExecPinType) &&
			Context.ValidatePinType(CondPin, ExpectedBoolPinType))
		{
			UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(CondPin);
			FBPTerminal** CondTerm = Context.NetMap.Find(PinToTry);

			if (CondTerm != NULL)
			{
				// First skip the if, if the term is false
				{
					FBlueprintCompiledStatement& SkipIfGoto = Context.AppendStatementForNode(Node);
					SkipIfGoto.Type = KCST_GotoIfNot;
					SkipIfGoto.LHS = *CondTerm;
					Context.GotoFixupRequestMap.Add(&SkipIfGoto, ElsePin);
				}

				// Now go to the If branch
				{
					FBlueprintCompiledStatement& GotoThen = Context.AppendStatementForNode(Node);
					GotoThen.Type = KCST_UnconditionalGoto;
					GotoThen.LHS = *CondTerm;
					Context.GotoFixupRequestMap.Add(&GotoThen, ThenPin);
				}
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermPassed_Error", "Failed to resolve term passed into @@").ToString(), CondPin);
			}
		}
	}
};

UK2Node_IfThenElse::UK2Node_IfThenElse(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_IfThenElse::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	UEdGraphPin* ConditionPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, UEdGraphSchema_K2::PN_Condition);
	K2Schema->SetPinAutogeneratedDefaultValue(ConditionPin, TEXT("true"));

	UEdGraphPin* TruePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	TruePin->PinFriendlyName = LOCTEXT("true", "true");

	UEdGraphPin* FalsePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Else);
	FalsePin->PinFriendlyName = LOCTEXT("false", "false");

	Super::AllocateDefaultPins();
}

FText UK2Node_IfThenElse::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Branch", "Branch");
}

FLinearColor UK2Node_IfThenElse::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->ExecBranchNodeTitleColor;
}

FSlateIcon UK2Node_IfThenElse::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Branch_16x");
	return Icon;
}

FText UK2Node_IfThenElse::GetTooltipText() const
{
	return LOCTEXT("BranchStatement_Tooltip", "Branch Statement\nIf Condition is true, execution goes to True, otherwise it goes to False");
}

UEdGraphPin* UK2Node_IfThenElse::GetElsePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Else);
	check(Pin);
	return Pin;
}

UEdGraphPin* UK2Node_IfThenElse::GetConditionPin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_Condition);
	check(Pin);
	return Pin;
}

FNodeHandlingFunctor* UK2Node_IfThenElse::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_Branch(CompilerContext);
}

void UK2Node_IfThenElse::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_IfThenElse::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::FlowControl);
}

#undef LOCTEXT_NAMESPACE
