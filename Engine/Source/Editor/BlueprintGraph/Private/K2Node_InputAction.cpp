// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputAction.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/MemberReference.h"
#include "GameFramework/InputSettings.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_InputActionEvent.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectVersion.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Blueprint/UserWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputAction)

#define LOCTEXT_NAMESPACE "K2Node_InputAction"

UK2Node_InputAction::UK2Node_InputAction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
}

void UK2Node_InputAction::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_BLUEPRINT_INPUT_BINDING_OVERRIDES)
	{
		// Don't change existing behaviors
		bOverrideParentBinding = false;
	}
}

void UK2Node_InputAction::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Pressed"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Released"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FKey::StaticStruct(), TEXT("Key"));

	Super::AllocateDefaultPins();
}

FLinearColor UK2Node_InputAction::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FText UK2Node_InputAction::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(InputActionName);
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputActionName"), FText::FromName(InputActionName));

		FText LocFormat = NSLOCTEXT("K2Node", "InputAction_Name", "InputAction {InputActionName}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
	}

	return CachedNodeTitle;
}

FText UK2Node_InputAction::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputAction_Tooltip", "Event for when the keys bound to input action {0} are pressed or released."), FText::FromName(InputActionName)), this);
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_InputAction::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Event_16x");
	return Icon;
}

bool UK2Node_InputAction::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	// This node expands into event nodes and must be placed in a Ubergraph
	EGraphType const GraphType = Graph->GetSchema()->GetGraphType(Graph);
	bool bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);

	if (bIsCompatible)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? UEdGraphSchema_K2::IsConstructionScript(Graph) : false;

		bIsCompatible = (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
	}
	return bIsCompatible;
}

bool UK2Node_InputAction::IsDeprecated() const
{
	const UInputSettings* Settings = GetDefault<UInputSettings>();
	return Super::IsDeprecated() || (Settings->bEnabledLegacyMappingDeprecationWarnings && Settings->IsActionMappingDeprecated(InputActionName));
}

bool UK2Node_InputAction::HasDeprecatedReference() const
{
	const bool bDeprecationWarningEnabled = GetDefault<UInputSettings>()->bEnabledLegacyMappingDeprecationWarnings;

	if (!bDeprecationWarningEnabled)
	{
		return false;
	}
	
	return GetDefault<UInputSettings>()->IsActionMappingDeprecated(InputActionName);	
}

FEdGraphNodeDeprecationResponse UK2Node_InputAction::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;

		FFormatNamedArguments Args;
		Args.Add(TEXT("ActionName"), FText::FromName(InputActionName));
		Response.MessageText = FText::Format(LOCTEXT("ActionMappingDeprecationWarning", "The Action Mapping '{ActionName}' has been deprecated."), Args);
	}

	return Response;
}

UEdGraphPin* UK2Node_InputAction::GetPressedPin() const
{
	return FindPin(TEXT("Pressed"));
}

UEdGraphPin* UK2Node_InputAction::GetReleasedPin() const
{
	return FindPin(TEXT("Released"));
}

void UK2Node_InputAction::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	TArray<FName> ActionNames;
	GetDefault<UInputSettings>()->GetActionNames(ActionNames);
	if (!ActionNames.Contains(InputActionName))
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingInputAction_WarningFmt", "InputAction Event references unknown Action '{0}' for @@"), FText::FromString(InputActionName.ToString())).ToString(), this);
	}
}

void UK2Node_InputAction::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* InputActionPressedPin = GetPressedPin();
	UEdGraphPin* InputActionReleasedPin = GetReleasedPin();
		
	struct EventPinData
	{
		EventPinData(UEdGraphPin* InPin,TEnumAsByte<EInputEvent> InEvent ){	Pin=InPin;EventType=InEvent;};
		UEdGraphPin* Pin;
		TEnumAsByte<EInputEvent> EventType;
	};

	TArray<EventPinData> ActivePins;
	if(( InputActionPressedPin != nullptr ) && (InputActionPressedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputActionPressedPin,IE_Pressed));
	}
	if((InputActionReleasedPin != nullptr) && (InputActionReleasedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputActionReleasedPin,IE_Released));
	}
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// If more than one is linked we have to do more complicated behaviors
	if( ActivePins.Num() > 1 )
	{
		// Create a temporary variable to copy Key in to
		static UScriptStruct* KeyStruct = FKey::StaticStruct();
		UK2Node_TemporaryVariable* ActionKeyVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		ActionKeyVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		ActionKeyVar->VariableType.PinSubCategoryObject = KeyStruct;
		ActionKeyVar->AllocateDefaultPins();

		for (auto PinIt = ActivePins.CreateIterator(); PinIt; ++PinIt)
		{			
			UEdGraphPin *EachPin = (*PinIt).Pin;
			// Create the input touch event
			UK2Node_InputActionEvent* InputActionEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputActionEvent>(this, SourceGraph);
			InputActionEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s"), *InputActionName.ToString(), *InputActionEvent->GetName()));
			InputActionEvent->InputActionName = InputActionName;
			InputActionEvent->bConsumeInput = bConsumeInput;
			InputActionEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputActionEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputActionEvent->InputKeyEvent = (*PinIt).EventType;
			InputActionEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputActionHandlerDynamicSignature__DelegateSignature")));
			InputActionEvent->bInternalEvent = true;
			InputActionEvent->AllocateDefaultPins();

			// Create assignment nodes to assign the key
			UK2Node_AssignmentStatement* ActionKeyInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			ActionKeyInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(ActionKeyVar->GetVariablePin(), ActionKeyInitialize->GetVariablePin());
			Schema->TryCreateConnection(ActionKeyInitialize->GetValuePin(), InputActionEvent->FindPinChecked(TEXT("Key")));
			// Connect the events to the assign key nodes
			Schema->TryCreateConnection(Schema->FindExecutionPin(*InputActionEvent, EGPD_Output), ActionKeyInitialize->GetExecPin());

			// Move the original event connections to the then pin of the key assign
			CompilerContext.MovePinLinksToIntermediate(*EachPin, *ActionKeyInitialize->GetThenPin());
			
			// Move the original event variable connections to the intermediate nodes
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *ActionKeyVar->GetVariablePin());
		}	
	}
	else if( ActivePins.Num() == 1 )
	{
		UEdGraphPin* InputActionPin = ActivePins[0].Pin;
		EInputEvent InputEvent = ActivePins[0].EventType;
	
		if (InputActionPin->LinkedTo.Num() > 0)
		{
			UK2Node_InputActionEvent* InputActionEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputActionEvent>(this, SourceGraph);
			InputActionEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s"), *InputActionName.ToString(), *InputActionEvent->GetName()));
			InputActionEvent->InputActionName = InputActionName;
			InputActionEvent->bConsumeInput = bConsumeInput;
			InputActionEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputActionEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputActionEvent->InputKeyEvent = InputEvent;
			InputActionEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputActionHandlerDynamicSignature__DelegateSignature")));
			InputActionEvent->bInternalEvent = true;
			InputActionEvent->AllocateDefaultPins();

			CompilerContext.MovePinLinksToIntermediate(*InputActionPin, *Schema->FindExecutionPin(*InputActionEvent, EGPD_Output));
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *InputActionEvent->FindPin(TEXT("Key")));
		}
	}

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	CompilerContext.SetAutomaticallyRegisterInputEvents(true);
}

void UK2Node_InputAction::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FName> ActionNames;
	GetDefault<UInputSettings>()->GetActionNames(ActionNames);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FName ActionName)
	{
		UK2Node_InputAction* InputNode = CastChecked<UK2Node_InputAction>(NewNode);
		InputNode->InputActionName = ActionName;
	};

	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		auto RefreshClassActions = []()
		{
			FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass());
		};

		static bool bRegisterOnce = true;
		if(bRegisterOnce)
		{
			bRegisterOnce = false;
			FEditorDelegates::OnActionAxisMappingsChanged.AddStatic(RefreshClassActions);
		}

		const UInputSettings* Settings = GetDefault<UInputSettings>();

		for (const FName& ActionName : ActionNames)
		{
			// Don't show any deprecated axis mappings in the BP action menu
			if (Settings->bEnabledLegacyMappingDeprecationWarnings && Settings->IsActionMappingDeprecated(ActionName))
			{
				continue;
			}
			
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, ActionName);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_InputAction::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, LOCTEXT("ActionMenuCategory", "Action Events")), this);
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_InputAction::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputActionName.ToString());

	return NodeSignature;
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_InputAction::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2InputAction> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2InputAction(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

#undef LOCTEXT_NAMESPACE
