// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputActionEvent.h"

#include "Containers/Array.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputActionDelegateBinding.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputActionEvent)

class UClass;

UK2Node_InputActionEvent::UK2Node_InputActionEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
	bInternalEvent = true;
}

UClass* UK2Node_InputActionEvent::GetDynamicBindingClass() const
{
	return UInputActionDelegateBinding::StaticClass();
}

void UK2Node_InputActionEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputActionDelegateBinding* InputActionBindingObject = CastChecked<UInputActionDelegateBinding>(BindingObject);

	FBlueprintInputActionDelegateBinding Binding;
	Binding.InputActionName = InputActionName;
	Binding.InputKeyEvent = InputKeyEvent;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputActionBindingObject->InputActionDelegateBindings.Add(Binding);
}
