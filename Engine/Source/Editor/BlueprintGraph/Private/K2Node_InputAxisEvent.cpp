// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputAxisEvent.h"

#include "BlueprintActionDatabase.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisDelegateBinding.h"
#include "Engine/MemberReference.h"
#include "GameFramework/InputSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "KismetCompiler.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/ObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputAxisEvent)

#define LOCTEXT_NAMESPACE "K2Node_InputAxisEvent"

UK2Node_InputAxisEvent::UK2Node_InputAxisEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
	bInternalEvent = true;

	EventReference.SetExternalDelegateMember(FName(TEXT("InputAxisHandlerDynamicSignature__DelegateSignature")));
}

void UK2Node_InputAxisEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE && EventSignatureName_DEPRECATED.IsNone() && EventSignatureClass_DEPRECATED == nullptr)
		{
			EventReference.SetExternalDelegateMember(TEXT("InputAxisHandlerDynamicSignature__DelegateSignature"));
		}
	}
}

void UK2Node_InputAxisEvent::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_BLUEPRINT_INPUT_BINDING_OVERRIDES)
	{
		// Don't change existing behaviors
		bOverrideParentBinding = false;
	}
}

void UK2Node_InputAxisEvent::Initialize(const FName AxisName)
{
	InputAxisName = AxisName;
	CustomFunctionName = FName( *FString::Printf(TEXT("InpAxisEvt_%s_%s"), *InputAxisName.ToString(), *GetName()));
}

FText UK2Node_InputAxisEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		return FText::FromName(InputAxisName);
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("InputAxisName"), FText::FromName(InputAxisName));

		FText LocFormat = NSLOCTEXT("K2Node", "InputAxis_Name", "InputAxis {InputAxisName}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LocFormat, Args), this);
	}

	return CachedNodeTitle;
}

FText UK2Node_InputAxisEvent::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputAxis_Tooltip", "Event that provides the current value of the {0} axis once per frame when input is enabled for the containing actor."), FText::FromName(InputAxisName)), this);
	}
	return CachedTooltip;
}

void UK2Node_InputAxisEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	TArray<FName> AxisNames;
	GetDefault<UInputSettings>()->GetAxisNames(AxisNames);
	if (!AxisNames.Contains(InputAxisName))
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "MissingInputAxisEvent_WarningFmt", "Input Axis Event references unknown Axis '{0}' for @@"), FText::FromString(InputAxisName.ToString())).ToString(), this);
	}
}

UClass* UK2Node_InputAxisEvent::GetDynamicBindingClass() const
{
	return UInputAxisDelegateBinding::StaticClass();
}

void UK2Node_InputAxisEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputAxisDelegateBinding* InputAxisBindingObject = CastChecked<UInputAxisDelegateBinding>(BindingObject);

	FBlueprintInputAxisDelegateBinding Binding;
	Binding.InputAxisName = InputAxisName;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputAxisBindingObject->InputAxisDelegateBindings.Add(Binding);
}

bool UK2Node_InputAxisEvent::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bIsCompatible = false;

	// Find the Blueprint that owns the target graph
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	if (Blueprint && Blueprint->SkeletonGeneratedClass)
	{
		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(TargetGraph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? UEdGraphSchema_K2::IsConstructionScript(TargetGraph) : false;
		
		bIsCompatible = Blueprint && Blueprint->SupportsInputEvents() && !bIsConstructionScript;
	}

	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

bool UK2Node_InputAxisEvent::IsDeprecated() const
{
	const UInputSettings* Settings = GetDefault<UInputSettings>();
	return Super::IsDeprecated() || (Settings->bEnabledLegacyMappingDeprecationWarnings && Settings->IsActionMappingDeprecated(InputAxisName));
}

bool UK2Node_InputAxisEvent::HasDeprecatedReference() const
{
	const bool bDeprecationWarningEnabled = GetDefault<UInputSettings>()->bEnabledLegacyMappingDeprecationWarnings;

	if (!bDeprecationWarningEnabled)
	{
		return false;
	}
	
	return GetDefault<UInputSettings>()->IsActionMappingDeprecated(InputAxisName);	
}

FEdGraphNodeDeprecationResponse UK2Node_InputAxisEvent::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		Response.MessageType = EEdGraphNodeDeprecationMessageType::Warning;

		FFormatNamedArguments Args;
		Args.Add(TEXT("ActionName"), FText::FromName(InputAxisName));
		Response.MessageText = FText::Format(LOCTEXT("AxisMappingDeprecationWarning", "The Axis Mapping '{ActionName}' has been deprecated."), Args);
	}

	return Response;
}

void UK2Node_InputAxisEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FName> AxisNames;
	GetDefault<UInputSettings>()->GetAxisNames(AxisNames);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FName AxisName)
	{
		UK2Node_InputAxisEvent* InputNode = CastChecked<UK2Node_InputAxisEvent>(NewNode);
		InputNode->Initialize(AxisName);
	};

	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		auto RefreshClassActions = []()
		{
			FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass());
		};

		static bool bRegisterOnce = true;
		if(bRegisterOnce)
		{
			bRegisterOnce = false;
			FEditorDelegates::OnActionAxisMappingsChanged.AddStatic(RefreshClassActions);
		}

		const UInputSettings* Settings = GetDefault<UInputSettings>();
		
		for (const FName& AxisName : AxisNames)
		{
			// Don't show any deprecated axis mappings in the BP action menu
			if (Settings->bEnabledLegacyMappingDeprecationWarnings && Settings->IsActionMappingDeprecated(AxisName))
			{
				continue;
			}
			
			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, AxisName);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_InputAxisEvent::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, LOCTEXT("ActionMenuCategory", "Axis Events")), this);
	}
	return CachedCategory;
}

FBlueprintNodeSignature UK2Node_InputAxisEvent::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputAxisName.ToString());

	return NodeSignature;
}

void UK2Node_InputAxisEvent::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	CompilerContext.SetAutomaticallyRegisterInputEvents(true);
}

#undef LOCTEXT_NAMESPACE
