// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputAxisKeyEvent.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Blueprint/UserWidget.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputAxisKeyEvent)

struct FLinearColor;

#define LOCTEXT_NAMESPACE "UK2Node_InputAxisKeyEvent"

UK2Node_InputAxisKeyEvent::UK2Node_InputAxisKeyEvent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
	bInternalEvent = true;

	EventReference.SetExternalDelegateMember(FName(TEXT("InputAxisHandlerDynamicSignature__DelegateSignature")));
}

void UK2Node_InputAxisKeyEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE && EventSignatureName_DEPRECATED.IsNone() && EventSignatureClass_DEPRECATED == nullptr)
		{
			EventReference.SetExternalDelegateMember(FName(TEXT("InputAxisHandlerDynamicSignature__DelegateSignature")));
		}
	}
}

void UK2Node_InputAxisKeyEvent::Initialize(const FKey InAxisKey)
{
	AxisKey = InAxisKey;
	CustomFunctionName = FName(*FString::Printf(TEXT("InpAxisKeyEvt_%s_%s"), *AxisKey.ToString(), *GetName()));
}

FText UK2Node_InputAxisKeyEvent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return AxisKey.GetDisplayName();
}

FText UK2Node_InputAxisKeyEvent::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputAxisKey_Tooltip", "Event that provides the current value of the {0} axis once per frame when input is enabled for the containing actor."), AxisKey.GetDisplayName()), this);
	}
	return CachedTooltip;
}

void UK2Node_InputAxisKeyEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!AxisKey.IsValid())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Invalid_InputAxisKey_Warning", "InputAxisKey Event specifies invalid FKey'{0}' for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (!AxisKey.IsAxis1D())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotAxis_InputAxisKey_Warning", "InputAxisKey Event specifies FKey'{0}' which is not a float axis for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (AxisKey.IsDeprecated())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Deprecated_InputAxisKey_Warning", "InputAxisKey Event specifies FKey'{0}' which has been deprecated for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (!AxisKey.IsBindableInBlueprints())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotBindable_InputAxisKey_Warning", "InputAxisKey Event specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
}

UClass* UK2Node_InputAxisKeyEvent::GetDynamicBindingClass() const
{
	return UInputAxisKeyDelegateBinding::StaticClass();
}

FSlateIcon UK2Node_InputAxisKeyEvent::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), EKeys::GetMenuCategoryPaletteIcon(AxisKey.GetMenuCategory()));
}

void UK2Node_InputAxisKeyEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputAxisKeyDelegateBinding* InputAxisKeyBindingObject = CastChecked<UInputAxisKeyDelegateBinding>(BindingObject);

	FBlueprintInputAxisKeyDelegateBinding Binding;
	Binding.AxisKey = AxisKey;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputAxisKeyBindingObject->InputAxisKeyDelegateBindings.Add(Binding);
}

bool UK2Node_InputAxisKeyEvent::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	// By default, to be safe, we don't allow events to be pasted, except under special circumstances (see below)
	bool bIsCompatible = false;

	// Find the Blueprint that owns the target graph
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	if (Blueprint != nullptr)
	{
		bIsCompatible = Blueprint && Blueprint->SupportsInputEvents();
	}

	UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(TargetGraph->GetSchema());
	bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(TargetGraph) : false;
	bIsCompatible &= !bIsConstructionScript;

	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

void UK2Node_InputAxisKeyEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_InputAxisKeyEvent* InputNode = CastChecked<UK2Node_InputAxisKeyEvent>(NewNode);
		InputNode->Initialize(Key);
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (const FKey& Key : AllKeys)
		{
			if (!Key.IsBindableInBlueprints() || !Key.IsAxis1D())
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_InputAxisKeyEvent::GetMenuCategory() const
{
	static TMap<FName, FNodeTextCache> CachedCategories;

	const FName KeyCategory = AxisKey.GetMenuCategory();
	const FText SubCategoryDisplayName = FText::Format(LOCTEXT("EventsCategory", "{0} Events"), EKeys::GetMenuCategoryDisplayName(KeyCategory));
	FNodeTextCache& NodeTextCache = CachedCategories.FindOrAdd(KeyCategory);

	if (NodeTextCache.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		NodeTextCache.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, SubCategoryDisplayName), this);
	}
	return NodeTextCache;
}

FBlueprintNodeSignature UK2Node_InputAxisKeyEvent::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(AxisKey.ToString());

	return NodeSignature;
}

void UK2Node_InputAxisKeyEvent::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);	
	
	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	if (GetBlueprint()->ShouldAutomaticallyRegisterInputOnConstruction())
	{
		CompilerContext.SetAutomaticallyRegisterInputEvents(true);
	}
}

#undef LOCTEXT_NAMESPACE
