// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputKey.h"
#include "GraphEditorSettings.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_InputKeyEvent.h"
#include "KismetCompiler.h"
#include "BlueprintNodeSpawner.h"
#include "EditorCategoryUtils.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Styling/AppStyle.h"
#include "GameFramework/InputSettings.h"
#include "Editor.h"								// for FEditorDelegates::OnEnableGestureRecognizerChanged
#include "Blueprint/UserWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputKey)

#define LOCTEXT_NAMESPACE "UK2Node_InputKey"

UK2Node_InputKey::UK2Node_InputKey(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;

#if PLATFORM_MAC && WITH_EDITOR
	if (IsTemplate())
	{
		FProperty* ControlProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UK2Node_InputKey, bControl));
		FProperty* CommandProp = GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UK2Node_InputKey, bCommand));

		ControlProp->SetMetaData(TEXT("DisplayName"), TEXT("Command"));
		CommandProp->SetMetaData(TEXT("DisplayName"), TEXT("Control"));
	}
#endif
}

void UK2Node_InputKey::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_BLUEPRINT_INPUT_BINDING_OVERRIDES)
	{
		// Don't change existing behaviors
		bOverrideParentBinding = false;
	}
}

void UK2Node_InputKey::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	CachedNodeTitle.Clear();
	CachedTooltip.Clear();
}

void UK2Node_InputKey::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Pressed"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Released"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, FKey::StaticStruct(), TEXT("Key"));

	Super::AllocateDefaultPins();
}

FLinearColor UK2Node_InputKey::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FName UK2Node_InputKey::GetModifierName() const
{
    FString ModName;
    bool bPlus = false;
    if (bControl)
    {
        ModName += TEXT("Ctrl");
        bPlus = true;
    }
    if (bCommand)
    {
        if (bPlus) ModName += TEXT("+");
        ModName += TEXT("Cmd");
        bPlus = true;
    }
    if (bAlt)
    {
        if (bPlus) ModName += TEXT("+");
        ModName += TEXT("Alt");
        bPlus = true;
    }
    if (bShift)
    {
        if (bPlus) ModName += TEXT("+");
        ModName += TEXT("Shift");
        bPlus = true;
    }

	return FName(*ModName);
}

FText UK2Node_InputKey::GetModifierText() const
{
#if PLATFORM_MAC
    const FText CommandText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Control", "Ctrl");
    const FText ControlText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Command", "Cmd");
#else
    const FText ControlText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Control", "Ctrl");
    const FText CommandText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Command", "Cmd");
#endif
    const FText AltText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Alt", "Alt");
    const FText ShiftText = NSLOCTEXT("UK2Node_InputKey", "KeyName_Shift", "Shift");

	const FText AppenderText = NSLOCTEXT("UK2Node_InputKey", "ModAppender", "+");

	FFormatNamedArguments Args;
	int32 ModCount = 0;

    if (bControl)
    {
		Args.Add(FString::Printf(TEXT("Mod%d"),++ModCount), ControlText);
    }
    if (bCommand)
    {
		Args.Add(FString::Printf(TEXT("Mod%d"),++ModCount), CommandText);
    }
    if (bAlt)
    {
		Args.Add(FString::Printf(TEXT("Mod%d"),++ModCount), AltText);
    }
    if (bShift)
    {
		Args.Add(FString::Printf(TEXT("Mod%d"),++ModCount), ShiftText);
    }

	for (int32 i = 1; i <= 4; ++i)
	{
		if (i > ModCount)
		{
			Args.Add(FString::Printf(TEXT("Mod%d"), i), FText::GetEmpty());
		}

		Args.Add(FString::Printf(TEXT("Appender%d"), i), (i < ModCount ? AppenderText : FText::GetEmpty()));
	}

	Args.Add(TEXT("Key"), GetKeyText());

	return FText::Format(NSLOCTEXT("UK2Node_InputKey", "NodeTitle", "{Mod1}{Appender1}{Mod2}{Appender2}{Mod3}{Appender3}{Mod4}"), Args);
}

FText UK2Node_InputKey::GetKeyText() const
{
	return InputKey.GetDisplayName();
}

FText UK2Node_InputKey::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (bControl || bAlt || bShift || bCommand)
	{
		if (CachedNodeTitle.IsOutOfDate(this))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("ModifierKey"), GetModifierText());
			Args.Add(TEXT("Key"), GetKeyText());

			// FText::Format() is slow, so we cache this to save on performance
			CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputKey_Name_WithModifiers", "{ModifierKey} {Key}"), Args), this);
		}
		return CachedNodeTitle;
	}
	else
	{
		return GetKeyText();
	}
}

FText UK2Node_InputKey::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FText ModifierText = GetModifierText();
		FText KeyText = GetKeyText();

		// FText::Format() is slow, so we cache this to save on performance
		if (!ModifierText.IsEmpty())
		{
			CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputKey_Tooltip_Modifiers", "Events for when the {0} key is pressed or released while {1} is also held."), KeyText, ModifierText), this);
		}
		else
		{
			CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "InputKey_Tooltip", "Events for when the {0} key is pressed or released."), KeyText), this);
		}
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_InputKey::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), EKeys::GetMenuCategoryPaletteIcon(InputKey.GetMenuCategory()));
}

bool UK2Node_InputKey::IsCompatibleWithGraph(UEdGraph const* Graph) const
{
	// This node expands into event nodes and must be placed in a Ubergraph
	EGraphType const GraphType = Graph->GetSchema()->GetGraphType(Graph);
	bool bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);

	if (bIsCompatible)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(Graph);

		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(Graph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? UEdGraphSchema_K2::IsConstructionScript(Graph) : false;

		bIsCompatible = (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(Graph);
	}
	return bIsCompatible;
}

UEdGraphPin* UK2Node_InputKey::GetPressedPin() const
{
	return FindPin(TEXT("Pressed"));
}

UEdGraphPin* UK2Node_InputKey::GetReleasedPin() const
{
	return FindPin(TEXT("Released"));
}

void UK2Node_InputKey::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (!InputKey.IsValid())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Invalid_InputKey_Warning", "InputKey Event specifies invalid FKey'{0}' for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
	else if (InputKey.IsAnalog())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Axis_InputKey_Warning", "InputKey Event specifies axis FKey'{0}' for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
	else if (InputKey.IsDeprecated())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Deprecated_InputKey_Warning", "InputKey Event specifies FKey'{0}' which has been deprecated for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
	else if (!InputKey.IsBindableInBlueprints())
	{
		MessageLog.Warning( *FText::Format( NSLOCTEXT("KismetCompiler", "NotBindanble_InputKey_Warning", "InputKey Event specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(InputKey.ToString())).ToString(), this);
	}
}

void UK2Node_InputKey::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* InputKeyPressedPin = GetPressedPin();
	UEdGraphPin* InputKeyReleasedPin = GetReleasedPin();

	struct EventPinData
	{
		EventPinData(UEdGraphPin* InPin,TEnumAsByte<EInputEvent> InEvent ){	Pin=InPin;EventType=InEvent;};
		UEdGraphPin* Pin;
		TEnumAsByte<EInputEvent> EventType;
	};

	TArray<EventPinData> ActivePins;
	if(( InputKeyPressedPin != nullptr ) && (InputKeyPressedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputKeyPressedPin,IE_Pressed));
	}
	if((InputKeyReleasedPin != nullptr) && (InputKeyReleasedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputKeyReleasedPin,IE_Released));
	}

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// If more than one is linked we have to do more complicated behaviors
	if( ActivePins.Num() > 1 )
	{
		// Create a temporary variable to copy Key in to
		static UScriptStruct* KeyStruct = FKey::StaticStruct();
		UK2Node_TemporaryVariable* KeyVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		KeyVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		KeyVar->VariableType.PinSubCategoryObject = KeyStruct;
		KeyVar->AllocateDefaultPins();

		for (auto PinIt = ActivePins.CreateIterator(); PinIt; ++PinIt)
		{
			UEdGraphPin *EachPin = (*PinIt).Pin;
			// Create the input touch event
			UK2Node_InputKeyEvent* InputKeyEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputKeyEvent>(this, SourceGraph);
			const FName ModifierName = GetModifierName();
			if ( ModifierName != NAME_None )
			{
				InputKeyEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s_%s"), *ModifierName.ToString(), *InputKey.ToString(), *InputKeyEvent->GetName()));
			}
			else
			{
				InputKeyEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s"), *InputKey.ToString(), *InputKeyEvent->GetName()));
			}
			InputKeyEvent->InputChord.Key = InputKey;
			InputKeyEvent->InputChord.bCtrl = bControl;
			InputKeyEvent->InputChord.bAlt = bAlt;
			InputKeyEvent->InputChord.bShift = bShift;
			InputKeyEvent->InputChord.bCmd = bCommand;
			InputKeyEvent->bConsumeInput = bConsumeInput;
			InputKeyEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputKeyEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputKeyEvent->InputKeyEvent = (*PinIt).EventType;
			InputKeyEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputActionHandlerDynamicSignature__DelegateSignature")));
			InputKeyEvent->bInternalEvent = true;
			InputKeyEvent->AllocateDefaultPins();

			// Create assignment nodes to assign the key
			UK2Node_AssignmentStatement* KeyInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			KeyInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(KeyVar->GetVariablePin(), KeyInitialize->GetVariablePin());
			Schema->TryCreateConnection(KeyInitialize->GetValuePin(), InputKeyEvent->FindPinChecked(TEXT("Key")));
			// Connect the events to the assign key nodes
			Schema->TryCreateConnection(Schema->FindExecutionPin(*InputKeyEvent, EGPD_Output), KeyInitialize->GetExecPin());

			// Move the original event connections to the then pin of the key assign
			CompilerContext.MovePinLinksToIntermediate(*EachPin, *KeyInitialize->GetThenPin());

			// Move the original event variable connections to the intermediate nodes
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *KeyVar->GetVariablePin());
		}
	}
	else if( ActivePins.Num() == 1 )
	{
		UEdGraphPin* InputKeyPin = ActivePins[0].Pin;
		EInputEvent InputEvent = ActivePins[0].EventType;

		if (InputKeyPin->LinkedTo.Num() > 0)
		{
			UK2Node_InputKeyEvent* InputKeyEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputKeyEvent>(this, SourceGraph);
			const FName ModifierName = GetModifierName();
			if ( ModifierName != NAME_None )
			{
				InputKeyEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s_%s"), *ModifierName.ToString(), *InputKey.ToString(), *InputKeyEvent->GetName()));
			}
			else
			{
				InputKeyEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpActEvt_%s_%s"), *InputKey.ToString(), *InputKeyEvent->GetName()));
			}
			InputKeyEvent->InputChord.Key = InputKey;
			InputKeyEvent->InputChord.bCtrl = bControl;
			InputKeyEvent->InputChord.bAlt = bAlt;
			InputKeyEvent->InputChord.bShift = bShift;
			InputKeyEvent->InputChord.bCmd = bCommand;
			InputKeyEvent->bConsumeInput = bConsumeInput;
			InputKeyEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputKeyEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputKeyEvent->InputKeyEvent = InputEvent;
			InputKeyEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputActionHandlerDynamicSignature__DelegateSignature")));
			InputKeyEvent->bInternalEvent = true;
			InputKeyEvent->AllocateDefaultPins();

			CompilerContext.MovePinLinksToIntermediate(*InputKeyPin, *Schema->FindExecutionPin(*InputKeyEvent, EGPD_Output));
			CompilerContext.MovePinLinksToIntermediate(*FindPin(TEXT("Key")), *InputKeyEvent->FindPin(TEXT("Key")));
		}
	}

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	CompilerContext.SetAutomaticallyRegisterInputEvents(true);
}

void UK2Node_InputKey::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_InputKey* InputNode = CastChecked<UK2Node_InputKey>(NewNode);
		InputNode->InputKey = Key;
	};

	auto RefreshClassActions = []()
	{
		FBlueprintActionDatabase::Get().RefreshClassActions(StaticClass());
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	
	const bool bAllowGestures = GetDefault<UInputSettings>()->bEnableGestureRecognizer;

	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		// Refresh the action database of this node when the input settings are changed
		static bool bRegisterOnce = true;
		if (bRegisterOnce)
		{
			bRegisterOnce = false;
			FEditorDelegates::OnEnableGestureRecognizerChanged.AddStatic(RefreshClassActions);
		}
		
		for (const FKey& Key : AllKeys)
		{
			// Do not show gesture keys if they are not enabled in the settings
			const bool bInvalidGestureKey = !bAllowGestures && Key.IsGesture();
			// these will be handled by UK2Node_GetInputAxisKeyValue and UK2Node_GetInputVectorAxisValue respectively
			if (!Key.IsBindableInBlueprints() || Key.IsAnalog() || bInvalidGestureKey)
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}

FText UK2Node_InputKey::GetMenuCategory() const
{
	static TMap<FName, FNodeTextCache> CachedCategories;

	const FName KeyCategory = InputKey.GetMenuCategory();
	const FText SubCategoryDisplayName = FText::Format(LOCTEXT("EventsCategory", "{0} Events"), EKeys::GetMenuCategoryDisplayName(KeyCategory));
	FNodeTextCache& NodeTextCache = CachedCategories.FindOrAdd(KeyCategory);

	if (NodeTextCache.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		NodeTextCache.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Input, SubCategoryDisplayName), this);
	}
	return NodeTextCache;
}

FBlueprintNodeSignature UK2Node_InputKey::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddKeyValue(InputKey.ToString());

	return NodeSignature;
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_InputKey::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2InputAction> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2InputAction(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

#undef LOCTEXT_NAMESPACE
