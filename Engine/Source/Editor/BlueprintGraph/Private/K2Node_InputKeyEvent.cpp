// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputKeyEvent.h"

#include "Containers/Array.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputKeyDelegateBinding.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputKeyEvent)

class UClass;

UK2Node_InputKeyEvent::UK2Node_InputKeyEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
	bInternalEvent = true;
}

UClass* UK2Node_InputKeyEvent::GetDynamicBindingClass() const
{
	return UInputKeyDelegateBinding::StaticClass();
}

void UK2Node_InputKeyEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputKeyDelegateBinding* InputKeyBindingObject = CastChecked<UInputKeyDelegateBinding>(BindingObject);

	FBlueprintInputKeyDelegateBinding Binding;
	Binding.InputChord = InputChord;
	Binding.InputKeyEvent = InputKeyEvent;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputKeyBindingObject->InputKeyDelegateBindings.Add(Binding);
}
