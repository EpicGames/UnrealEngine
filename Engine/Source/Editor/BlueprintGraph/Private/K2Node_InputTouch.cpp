// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputTouch.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/MemberReference.h"
#include "GraphEditorSettings.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_InputTouchEvent.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectVersion.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Blueprint/UserWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputTouch)

UK2Node_InputTouch::UK2Node_InputTouch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
}

void UK2Node_InputTouch::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_BLUEPRINT_INPUT_BINDING_OVERRIDES)
	{
		// Don't change existing behaviors
		bOverrideParentBinding = false;
	}
}

UEnum* UK2Node_InputTouch::GetTouchIndexEnum()
{
	static UEnum* TouchIndexEnum = nullptr;
	if (nullptr == TouchIndexEnum)
	{
		FTopLevelAssetPath TouchIndexEnumPath(TEXT("/Script/InputCore"), TEXT("ETouchIndex"));
		TouchIndexEnum = FindObject<UEnum>(TouchIndexEnumPath);
		check(TouchIndexEnum);
	}
	return TouchIndexEnum;
}

void UK2Node_InputTouch::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Pressed"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Released"));
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, TEXT("Moved"));
	
	UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, VectorStruct, TEXT("Location"));

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Byte, GetTouchIndexEnum(), TEXT("FingerIndex"));

	Super::AllocateDefaultPins();
}

FLinearColor UK2Node_InputTouch::GetNodeTitleColor() const
{
	return GetDefault<UGraphEditorSettings>()->EventNodeTitleColor;
}

FText UK2Node_InputTouch::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText Title = NSLOCTEXT("K2Node", "InputTouch_Name", "InputTouch");
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		Title = NSLOCTEXT("K2Node", "InputTouch_ListTitle", "Touch");
	}
	return Title;
}

FText UK2Node_InputTouch::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "InputTouch_Tooltip", "Event for when a finger presses, releases or is moved on a touch device.");
}

FSlateIcon UK2Node_InputTouch::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.TouchEvent_16x");
	return Icon;
}

void UK2Node_InputTouch::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_InputTouch::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Input);
}

TSharedPtr<FEdGraphSchemaAction> UK2Node_InputTouch::GetEventNodeAction(const FText& ActionCategory)
{
	TSharedPtr<FEdGraphSchemaAction_K2InputAction> EventNodeAction = MakeShareable(new FEdGraphSchemaAction_K2InputAction(ActionCategory, GetNodeTitle(ENodeTitleType::EditableTitle), GetTooltipText(), 0));
	EventNodeAction->NodeTemplate = this;
	return EventNodeAction;
}

bool UK2Node_InputTouch::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	// This node expands into event nodes and must be placed in a Ubergraph
	EGraphType const GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
	bool bIsCompatible = (GraphType == EGraphType::GT_Ubergraph);

	if (bIsCompatible)
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);

		UEdGraphSchema_K2 const* K2Schema = Cast<UEdGraphSchema_K2>(TargetGraph->GetSchema());
		bool const bIsConstructionScript = (K2Schema != nullptr) ? K2Schema->IsConstructionScript(TargetGraph) : false;

		bIsCompatible = (Blueprint != nullptr) && Blueprint->SupportsInputEvents() && !bIsConstructionScript && Super::IsCompatibleWithGraph(TargetGraph);
	}
	return bIsCompatible;
}

UEdGraphPin* UK2Node_InputTouch::GetPressedPin() const
{
	return FindPin(TEXT("Pressed"));
}

UEdGraphPin* UK2Node_InputTouch::GetReleasedPin() const
{
	return FindPin(TEXT("Released"));
}

UEdGraphPin* UK2Node_InputTouch::GetMovedPin() const
{
	return FindPin(TEXT("Moved"));
}

UEdGraphPin* UK2Node_InputTouch::GetLocationPin() const
{
	return FindPin(TEXT("Location"));
}

UEdGraphPin* UK2Node_InputTouch::GetFingerIndexPin() const
{
	return FindPin(TEXT("FingerIndex"));
}

void UK2Node_InputTouch::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	UEdGraphPin* InputTouchPressedPin = GetPressedPin();
	UEdGraphPin* InputTouchReleasedPin = GetReleasedPin();
	UEdGraphPin* InputTouchMovedPin = GetMovedPin();
		
	struct EventPinData
	{
		EventPinData(UEdGraphPin* InPin,TEnumAsByte<EInputEvent> InEvent ){	Pin=InPin;EventType=InEvent;};
		UEdGraphPin* Pin;
		TEnumAsByte<EInputEvent> EventType;
	};

	TArray<EventPinData> ActivePins;
	if(( InputTouchPressedPin != nullptr ) && (InputTouchPressedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputTouchPressedPin,IE_Pressed));
	}
	if((InputTouchReleasedPin != nullptr) && (InputTouchReleasedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputTouchReleasedPin,IE_Released));
	}
	if((InputTouchMovedPin != nullptr) && ( InputTouchMovedPin->LinkedTo.Num() > 0 ))
	{
		ActivePins.Add(EventPinData(InputTouchMovedPin,IE_Repeat));
	}
	
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// If more than one is linked we have to do more complicated behaviors
	if( ActivePins.Num() > 1 )
	{
		// Create a temporary variable to copy location in to
		static UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
		UK2Node_TemporaryVariable* TouchLocationVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		TouchLocationVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		TouchLocationVar->VariableType.PinSubCategoryObject = VectorStruct;
		TouchLocationVar->AllocateDefaultPins();

		// Create a temporary variable to copy finger index in to
		UK2Node_TemporaryVariable* TouchFingerVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
		TouchFingerVar->VariableType.PinCategory = UEdGraphSchema_K2::PC_Byte;
		TouchFingerVar->VariableType.PinSubCategoryObject = UK2Node_InputTouch::GetTouchIndexEnum();
		TouchFingerVar->AllocateDefaultPins();

		for (auto PinIt = ActivePins.CreateIterator(); PinIt; ++PinIt)
		{			
			UEdGraphPin *EachPin= (*PinIt).Pin;
			// Create the input touch event
			UK2Node_InputTouchEvent* InputTouchEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputTouchEvent>(this, SourceGraph);
			InputTouchEvent->CustomFunctionName = FName(*FString::Printf(TEXT("InpTchEvt_%s"), *EachPin->GetName()));
			InputTouchEvent->bConsumeInput = bConsumeInput;
			InputTouchEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputTouchEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputTouchEvent->InputKeyEvent = (*PinIt).EventType;
			InputTouchEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputTouchHandlerDynamicSignature__DelegateSignature")));
			InputTouchEvent->bInternalEvent = true;
			InputTouchEvent->AllocateDefaultPins();

			// Create assignment nodes to assign the location
			UK2Node_AssignmentStatement* TouchLocationInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			TouchLocationInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(TouchLocationVar->GetVariablePin(), TouchLocationInitialize->GetVariablePin());
			Schema->TryCreateConnection(TouchLocationInitialize->GetValuePin(), InputTouchEvent->FindPinChecked(TEXT("Location")));
			// Connect the events to the assign location nodes
			Schema->TryCreateConnection(Schema->FindExecutionPin(*InputTouchEvent, EGPD_Output), TouchLocationInitialize->GetExecPin());

			// Create assignment nodes to assign the finger index
			UK2Node_AssignmentStatement* TouchFingerInitialize = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
			TouchFingerInitialize->AllocateDefaultPins();
			Schema->TryCreateConnection(TouchFingerVar->GetVariablePin(), TouchFingerInitialize->GetVariablePin());
			Schema->TryCreateConnection(TouchFingerInitialize->GetValuePin(), InputTouchEvent->FindPinChecked(TEXT("FingerIndex")));
			// Connect the assign location to the assign finger index nodes
			Schema->TryCreateConnection(TouchLocationInitialize->GetThenPin(), TouchFingerInitialize->GetExecPin());			

			// Move the original event connections to the then pin of the finger index assign
			CompilerContext.MovePinLinksToIntermediate(*EachPin, *TouchFingerInitialize->GetThenPin());
			
			// Move the original event variable connections to the intermediate nodes
			CompilerContext.MovePinLinksToIntermediate(*GetLocationPin(), *TouchLocationVar->GetVariablePin());
			CompilerContext.MovePinLinksToIntermediate(*GetFingerIndexPin(), *TouchFingerVar->GetVariablePin());
		}	
	}
	else if( ActivePins.Num() == 1 )
	{
		UEdGraphPin* InputTouchPin = ActivePins[0].Pin;
		EInputEvent InputEvent = ActivePins[0].EventType;
		
		if (InputTouchPin->LinkedTo.Num() > 0)
		{
			UK2Node_InputTouchEvent* InputTouchEvent = CompilerContext.SpawnIntermediateNode<UK2Node_InputTouchEvent>(this, SourceGraph);
			InputTouchEvent->CustomFunctionName = FName( *FString::Printf(TEXT("InpTchEvt_%s"), *InputTouchEvent->GetName()));
			InputTouchEvent->InputKeyEvent = InputEvent;
			InputTouchEvent->bConsumeInput = bConsumeInput;
			InputTouchEvent->bExecuteWhenPaused = bExecuteWhenPaused;
			InputTouchEvent->bOverrideParentBinding = bOverrideParentBinding;
			InputTouchEvent->EventReference.SetExternalDelegateMember(FName(TEXT("InputTouchHandlerDynamicSignature__DelegateSignature")));
			InputTouchEvent->bInternalEvent = true;
			InputTouchEvent->AllocateDefaultPins();

			CompilerContext.MovePinLinksToIntermediate(*InputTouchPin, *Schema->FindExecutionPin(*InputTouchEvent, EGPD_Output));
			CompilerContext.MovePinLinksToIntermediate(*GetLocationPin(), *InputTouchEvent->FindPin(TEXT("Location")));
			CompilerContext.MovePinLinksToIntermediate(*GetFingerIndexPin(), *InputTouchEvent->FindPin(TEXT("FingerIndex")));
		}
	}

	// Widget blueprints require the bAutomaticallyRegisterInputOnConstruction to be set to true in order to receive callbacks
	CompilerContext.SetAutomaticallyRegisterInputEvents(true);
}
