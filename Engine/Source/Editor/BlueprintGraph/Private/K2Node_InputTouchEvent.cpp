// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputTouchEvent.h"

#include "Containers/Array.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputTouchDelegateBinding.h"
#include "HAL/PlatformCrt.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputTouchEvent)

class UClass;

UK2Node_InputTouchEvent::UK2Node_InputTouchEvent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bConsumeInput = true;
	bOverrideParentBinding = true;
	bInternalEvent = true;
}

UClass* UK2Node_InputTouchEvent::GetDynamicBindingClass() const
{
	return UInputTouchDelegateBinding::StaticClass();
}

void UK2Node_InputTouchEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputTouchDelegateBinding* InputTouchBindingObject = CastChecked<UInputTouchDelegateBinding>(BindingObject);

	FBlueprintInputTouchDelegateBinding Binding;
	Binding.InputKeyEvent = InputKeyEvent;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputTouchBindingObject->InputTouchDelegateBindings.Add(Binding);
}
