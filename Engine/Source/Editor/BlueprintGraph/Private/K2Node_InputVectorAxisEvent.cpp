// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InputVectorAxisEvent.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/DynamicBlueprintBinding.h"
#include "Engine/InputAxisKeyDelegateBinding.h"
#include "Engine/InputVectorAxisDelegateBinding.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_Event.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InputVectorAxisEvent)

UK2Node_InputVectorAxisEvent::UK2Node_InputVectorAxisEvent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	EventReference.SetExternalDelegateMember(FName(TEXT("InputVectorAxisHandlerDynamicSignature__DelegateSignature")));
}

void UK2Node_InputVectorAxisEvent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_K2NODE_EVENT_MEMBER_REFERENCE && EventSignatureName_DEPRECATED.IsNone() && EventSignatureClass_DEPRECATED == nullptr)
		{
			EventReference.SetExternalDelegateMember(FName(TEXT("InputVectorAxisHandlerDynamicSignature__DelegateSignature")));
		}
	}
}

void UK2Node_InputVectorAxisEvent::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	// Skip AxisKeyEvent validation
	UK2Node_Event::ValidateNodeDuringCompilation(MessageLog);

	if (!AxisKey.IsValid())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Invalid_InputVectorAxis_Warning", "InputVectorAxis Event specifies invalid FKey'{0}' for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (!AxisKey.IsAxis2D() && !AxisKey.IsAxis3D())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotAxis_InputVectorAxis_Warning", "InputVectorAxis Event specifies FKey'{0}' which is not a vector axis for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (AxisKey.IsDeprecated())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "Deprecated_InputVectorAxis_Warning", "InputVectorAxis Event specifies FKey'{0}' which has been deprecated for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
	else if (!AxisKey.IsBindableInBlueprints())
	{
		MessageLog.Warning(*FText::Format(NSLOCTEXT("KismetCompiler", "NotBindable_InputVectorAxis_Warning", "InputVectorAxis Event specifies FKey'{0}' that is not blueprint bindable for @@"), FText::FromString(AxisKey.ToString())).ToString(), this);
	}
}

UClass* UK2Node_InputVectorAxisEvent::GetDynamicBindingClass() const
{
	return UInputVectorAxisDelegateBinding::StaticClass();
}

void UK2Node_InputVectorAxisEvent::RegisterDynamicBinding(UDynamicBlueprintBinding* BindingObject) const
{
	UInputVectorAxisDelegateBinding* InputVectorAxisBindingObject = CastChecked<UInputVectorAxisDelegateBinding>(BindingObject);

	FBlueprintInputAxisKeyDelegateBinding Binding;
	Binding.AxisKey = AxisKey;
	Binding.bConsumeInput = bConsumeInput;
	Binding.bExecuteWhenPaused = bExecuteWhenPaused;
	Binding.bOverrideParentBinding = bOverrideParentBinding;
	Binding.FunctionNameToBind = CustomFunctionName;

	InputVectorAxisBindingObject->InputAxisKeyDelegateBindings.Add(Binding);
}

void UK2Node_InputVectorAxisEvent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	TArray<FKey> AllKeys;
	EKeys::GetAllKeys(AllKeys);

	auto CustomizeInputNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FKey Key)
	{
		UK2Node_InputVectorAxisEvent* InputNode = CastChecked<UK2Node_InputVectorAxisEvent>(NewNode);
		InputNode->Initialize(Key);
	};

	// actions get registered under specific object-keys; the idea is that
	// actions might have to be updated (or deleted) if their object-key is
	// mutated (or removed)... here we use the node's class (so if the node
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();

	// to keep from needlessly instantiating a UBlueprintNodeSpawner (and
	// iterating over keys), first check to make sure that the registrar is
	// looking for actions of this type (could be regenerating actions for a
	// specific asset, and therefore the registrar would only accept actions
	// corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		for (const FKey& Key : AllKeys)
		{
			if (!Key.IsBindableInBlueprints() || !(Key.IsAnalog() && !Key.IsAxis1D()))
			{
				continue;
			}

			UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
			check(NodeSpawner != nullptr);

			NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeInputNodeLambda, Key);
			ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
		}
	}
}
