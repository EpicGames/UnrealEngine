// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_InstancedStruct.h"
#include "EdGraphSchema_K2.h"
#include "BlueprintNodeSpawner.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "Kismet/BlueprintInstancedStructLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_InstancedStruct)

#define LOCTEXT_NAMESPACE "InstancedStruct"

void UK2Node_InstancedStruct::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::GetMenuActions(ActionRegistrar);
	UClass* Action = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(Action))
	{
		auto CustomizeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, const FName FunctionName)
		{
			UK2Node_InstancedStruct* Node = CastChecked<UK2Node_InstancedStruct>(NewNode);
			UFunction* Function = UBlueprintInstancedStructLibrary::StaticClass()->FindFunctionByName(FunctionName);
			check(Function);
			Node->SetFromFunction(Function);
		};
		
		// MakeInstancedStruct()
		UBlueprintNodeSpawner* MakeNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(MakeNodeSpawner != nullptr);
		MakeNodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeLambda, GET_FUNCTION_NAME_CHECKED(UBlueprintInstancedStructLibrary, MakeInstancedStruct));
		ActionRegistrar.AddBlueprintAction(Action, MakeNodeSpawner);

		// SetInstancedStructValue()
		UBlueprintNodeSpawner* SetNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(SetNodeSpawner != nullptr);
		SetNodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeLambda, GET_FUNCTION_NAME_CHECKED(UBlueprintInstancedStructLibrary, SetInstancedStructValue));
		ActionRegistrar.AddBlueprintAction(Action, SetNodeSpawner);

		// GetInstancedStructValue()
		UBlueprintNodeSpawner* GetNodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(GetNodeSpawner != nullptr);
		GetNodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeLambda, GET_FUNCTION_NAME_CHECKED(UBlueprintInstancedStructLibrary, GetInstancedStructValue));
		ActionRegistrar.AddBlueprintAction(Action, GetNodeSpawner);
	}
}

bool UK2Node_InstancedStruct::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	const UEdGraphPin* ValuePin = FindPinChecked(FName(TEXT("Value")));

	if (MyPin == ValuePin && MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		if (OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Struct)
		{
			OutReason = TEXT("Value must be a struct.");
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

