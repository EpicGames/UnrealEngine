// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Knot.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/WildcardNodeUtils.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Knot)

#define LOCTEXT_NAMESPACE "K2Node_Knot"

/////////////////////////////////////////////////////
// UK2Node_Knot

UK2Node_Knot::UK2Node_Knot(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
}

void UK2Node_Knot::AllocateDefaultPins()
{
	const FName InputPinName(TEXT("InputPin"));
	const FName OutputPinName(TEXT("OutputPin"));

	UEdGraphPin* MyInputPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, InputPinName);
	MyInputPin->bDefaultValueIsIgnored = true;

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, OutputPinName);
}

FText UK2Node_Knot::GetTooltipText() const
{
	//@TODO: Should pull the tooltip from the source pin
	return LOCTEXT("KnotTooltip", "Reroute Node (reroutes wires)");
}

FText UK2Node_Knot::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromString(NodeComment);
	}
	else if (TitleType == ENodeTitleType::MenuTitle)
	{
		return LOCTEXT("KnotListTitle", "Add Reroute Node...");
	}
	else
	{
		return LOCTEXT("KnotTitle", "Reroute Node");
	}
}

void UK2Node_Knot::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* MyInputPin = GetInputPin();
	UEdGraphPin* MyOutputPin = GetOutputPin();

	K2Schema->CombineTwoPinNetsAndRemoveOldPins(MyInputPin, MyOutputPin);
}

bool UK2Node_Knot::IsNodeSafeToIgnore() const
{
	return true;
}

bool UK2Node_Knot::CanSplitPin(const UEdGraphPin* Pin) const
{
	return false;
}

void UK2Node_Knot::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	PropagatePinType();
}

void UK2Node_Knot::PostReconstructNode()
{
	PropagatePinType();
	Super::PostReconstructNode();
}

void UK2Node_Knot::PropagatePinType()
{
	UEdGraphPin* MyInputPin  = GetInputPin();
	UEdGraphPin* MyOutputPin = GetOutputPin();

	for (UEdGraphPin* Inputs : MyInputPin->LinkedTo)
	{
		if (!FWildcardNodeUtils::HasAnyWildcards(Inputs))
		{
			PropagatePinTypeFromDirection(true);
			return;
		}
	}

	for (UEdGraphPin* Outputs : MyOutputPin->LinkedTo)
	{
		if (!FWildcardNodeUtils::HasAnyWildcards(Outputs))
		{
			PropagatePinTypeFromDirection(false);
			return;
		}
	}

	// if all inputs/outputs are wildcards, still favor the inputs first (propagate array/reference/etc. state)
	if (MyInputPin->LinkedTo.Num() > 0)
	{
		// If we can't mirror from output type, we should at least get the type information from the input connection chain
		PropagatePinTypeFromDirection(true);
	}
	else if (MyOutputPin->LinkedTo.Num() > 0)
	{
		// Try to mirror from output first to make sure we get appropriate member references
		PropagatePinTypeFromDirection(false);
	}
	else
	{
		// Revert to wildcard
		MyInputPin->BreakAllPinLinks();
		MyInputPin->PinType.ResetToDefaults();
		MyInputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;

		MyOutputPin->BreakAllPinLinks();
		MyOutputPin->PinType.ResetToDefaults();
		MyOutputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	}
}

void UK2Node_Knot::PropagatePinTypeFromDirection(bool bFromInput)
{
	if (bRecursionGuard)
	{
		return;
	}
	// Set the type of the pin based on the source connection, and then percolate
	// that type information up until we no longer reach another Reroute node
	UEdGraphPin* MySourcePin = bFromInput ? GetInputPin() : GetOutputPin();
	UEdGraphPin* MyDestinationPin = bFromInput ? GetOutputPin() : GetInputPin();

	TGuardValue<bool> RecursionGuard(bRecursionGuard, true);

	// Make sure any source knot pins compute their type, this will try to call back
	// into this function but the recursion guard will stop it
	for (UEdGraphPin* InPin : MySourcePin->LinkedTo)
	{
		if (InPin)
		{
			if (UK2Node_Knot * KnotNode = Cast<UK2Node_Knot>(InPin->GetOwningNode()))
			{
				KnotNode->PropagatePinTypeFromDirection(bFromInput);
			}
		}
	}

	UEdGraphPin* TypeSource = MySourcePin->LinkedTo.Num() ? MySourcePin->LinkedTo[0] : nullptr;
	if (TypeSource)
	{
		// if the type in the source and dest matches the type source then
		// lets early return to avoid expensive propagation in PinConnectionListChanged
		if (MySourcePin->PinType == TypeSource->PinType &&
			MyDestinationPin->PinType == TypeSource->PinType)
		{
			return;
		}

		MySourcePin->PinType = TypeSource->PinType;
		MyDestinationPin->PinType = TypeSource->PinType;

		for (UEdGraphPin* LinkPin : MyDestinationPin->LinkedTo)
		{
			// Order of reconstruction can be such that nulls haven't been cleared out of the destination node's list yet so
			// must protect against null here
			if (LinkPin)
			{
				if (UK2Node* OwningNode = Cast<UK2Node>(LinkPin->GetOwningNode()))
				{
					// Notify any pins in the destination direction
					if (UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(OwningNode))
					{
						KnotNode->PropagatePinTypeFromDirection(bFromInput);
					}
					else
					{
						OwningNode->PinConnectionListChanged(LinkPin);
					}
				}
			}
		}
	}
}

void UK2Node_Knot::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

UEdGraphNode* UK2Node_Knot::GetExecTerminal() const
{
	// if we're not an exec wire, just bail:
	const UK2Node_Knot* Knot = this;
	UEdGraphPin* OutputPin = Knot->GetOutputPin();
	if(!OutputPin)
	{
		return nullptr;
	}

	if(OutputPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec)
	{
		return nullptr;
	}

	UEdGraphNode* Result = nullptr;
	while(Knot)
	{
		UEdGraphPin* Next = Knot->GetOutputPin();
		if(Next && Next->LinkedTo.Num() > 0)
		{
			// knots for exec pins can have only one connection:
			Result = Next->LinkedTo[0]->GetOwningNode();
			Knot = Cast<const UK2Node_Knot>(Result);
		}
		else
		{
			// dead end knot:
			Knot = nullptr;
			Result = nullptr;
		}
	}
	return Result;
}

bool UK2Node_Knot::ShouldOverridePinNames() const
{
	return true;
}

FText UK2Node_Knot::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	// Keep the pin size tiny
	return FText::GetEmpty();
}

void UK2Node_Knot::OnRenameNode(const FString& NewName)
{
	NodeComment = NewName;
}

TSharedPtr<class INameValidatorInterface> UK2Node_Knot::MakeNameValidator() const
{
	// Comments can be duplicated, etc...
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

UEdGraphPin* UK2Node_Knot::GetPassThroughPin(const UEdGraphPin* FromPin) const
{
	if(FromPin && Pins.Contains(FromPin))
	{
		return FromPin == Pins[0] ? Pins[1] : Pins[0];
	}

	return nullptr;
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
