// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_Literal.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintBoundNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Literal)

struct FBlueprintActionContext;

#define LOCTEXT_NAMESPACE "K2Node_Literal"

class FKCHandler_LiteralStatement : public FNodeHandlingFunctor
{
public:
	FKCHandler_LiteralStatement(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		UK2Node_Literal* LiteralNode = Cast<UK2Node_Literal>(Net->GetOwningNode());
		check(LiteralNode);

		UObject* TargetObject = LiteralNode->GetObjectRef();

		if( !TargetObject )
		{
			CompilerContext.MessageLog.Warning(*LOCTEXT("InvalidLevelActor_Warning", "Node @@ is not referencing a valid level actor").ToString(), LiteralNode);
		}

		const FString TargetObjectName = TargetObject ? TargetObject->GetPathName() : TEXT("None");

		// First, try to see if we already have a term for this object
		FBPTerminal* Term = NULL;
		for( int32 i = 0; i < Context.LevelActorReferences.Num(); i++ )
		{
			FBPTerminal& CurrentTerm = Context.LevelActorReferences[i];
			if( CurrentTerm.PropertyDefault == TargetObjectName )
			{
				Term = &CurrentTerm;
				break;
			}
		}

		// If we didn't find one, then create a new term
		if( !Term )
		{
			FString RefPropName = (TargetObject ? TargetObject->GetName() : TEXT("None")) + TEXT("_") + (Context.SourceGraph ? *Context.SourceGraph->GetName() : TEXT("None")) + TEXT("_RefProperty");
			Term = new FBPTerminal();
			Context.LevelActorReferences.Add(Term);
			Term->CopyFromPin(Net, Context.NetNameMap->MakeValidName(Net));
			Term->Name = RefPropName;
			Term->PropertyDefault = TargetObjectName;
		}

		check(Term);

		Context.NetMap.Add(Net, Term);
	}
};

UK2Node_Literal::UK2Node_Literal(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

static const FName ValuePinName(TEXT("Value"));

void UK2Node_Literal::AllocateDefaultPins()
{
	// The literal node only has one pin:  an output of the desired value, on a wildcard pin type
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, ValuePinName);

	// After allocating the pins, try to coerce pin type
	SetObjectRef( ObjectRef );
}

void UK2Node_Literal::PostReconstructNode()
{
	UEdGraphPin* ValuePin = GetValuePin();

	// See if we need to fix up the value pin to have a valid type after reconstruction.  Could be invalid with a stale object ref
	if( ValuePin && !ObjectRef && (ValuePin->LinkedTo.Num() > 0) )
	{
		// Loop over the node, and figure out the most derived class connected to this pin, and use that, so everyone is happy
		UClass* PinSubtype = nullptr;
		for( TArray<UEdGraphPin*>::TIterator PinIt(ValuePin->LinkedTo); PinIt; ++PinIt )
		{
			UClass* TestType = Cast<UClass>((*PinIt)->PinType.PinSubCategoryObject.Get());
			if( !TestType )
			{
				// If this isn't a class, we're connected to something we shouldn't be.  Bail and make the scripter fix it up.
				return;
			}
			else
			{
				if( TestType && (!PinSubtype || (PinSubtype != TestType && TestType->IsChildOf(PinSubtype))) )
				{
					PinSubtype = TestType;
				}
			}
		}

		const UEdGraphPin* ConnectedPin = ValuePin->LinkedTo[0];
		ValuePin->PinType = ConnectedPin->PinType;
		ValuePin->PinType.PinSubCategoryObject = PinSubtype;
	}

	Super::PostReconstructNode();
}

FText UK2Node_Literal::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "Literal_Tooltip", "Stores a reference to an actor in the level");
}

FText UK2Node_Literal::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	AActor* Actor = Cast<AActor>(ObjectRef);
	if( ObjectRef != NULL )
	{
		if(Actor != NULL)
		{
			return FText::FromString(Actor->GetActorLabel());
		}
		else
		{
			return FText::FromString(ObjectRef->GetName());
		}
	}
	else
	{
		return NSLOCTEXT("K2Node", "Unknown", "Unknown");
	}
}

FLinearColor UK2Node_Literal::GetNodeTitleColor() const
{
	UEdGraphPin* ValuePin = GetValuePin();
	if( ValuePin )
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		return Schema->GetPinTypeColor(ValuePin->PinType);		
	}
	else
	{
		return Super::GetNodeTitleColor();
	}
}

void UK2Node_Literal::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* ActionKey = GetClass();
	if (!ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		return;
	}

	auto CanBindObjectLambda = [](UObject const* BindingObject)
	{
		if(AActor const* Actor = Cast<AActor>(BindingObject))
		{
			// Make sure the Actor has a world
			if(Actor->GetWorld())
			{
				return true;
			}
		}
		return false;
	};

	auto PostBindSetupLambda = [](UEdGraphNode* NewNode, UObject* BindObject)->bool
	{
 		UK2Node_Literal* LiteralNode = CastChecked<UK2Node_Literal>(NewNode);
 		LiteralNode->SetObjectRef(BindObject);
 		return true;
	};

	auto UiSpecOverride = [](const FBlueprintActionContext& /*Context*/, const IBlueprintNodeBinder::FBindingSet& Bindings, FBlueprintActionUiSpec* UiSpecOut)
	{
		if (Bindings.Num() == 1)
		{
			const AActor* ActorObj = Bindings.CreateConstIterator()->Get<AActor>();

			UiSpecOut->MenuName = FText::Format( NSLOCTEXT("K2Node", "LiteralTitle", "Create a Reference to {0}"), 
				FText::FromString(ActorObj->GetActorLabel()) );

			const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(ActorObj->GetClass());
			if (Icon.IsSet())
			{
				UiSpecOut->Icon = Icon;
			}
		}
		else if (Bindings.Num() > 1)
		{
			UiSpecOut->MenuName = FText::Format(NSLOCTEXT("K2Node", "LiteralTitleMultipleActors", "Create References to {0} selected Actors"), 
				FText::AsNumber(Bindings.Num()) );

			auto BindingIt = Bindings.CreateConstIterator();
			UObject* Binding = BindingIt->Get<UObject>();
			check(Binding); // Can the binding object be an FProperty?
			UClass* CommonClass = Binding->GetClass();
			for (++BindingIt; BindingIt; ++BindingIt)
			{
				UClass* Class = BindingIt->Get<UObject>()->GetClass(); // Can the binding object be an FProperty?
				while (!Class->IsChildOf(CommonClass))
				{
					CommonClass = CommonClass->GetSuperClass();
				}
			}

			const FSlateIcon Icon = FSlateIconFinder::FindIconForClass(CommonClass);
			if (Icon.IsSet())
			{
				UiSpecOut->Icon = Icon;
			}
		}
		else
		{
			UiSpecOut->MenuName = NSLOCTEXT("K2Node", "FallbackLiteralTitle", "Error: No Actors in Context");
		}
	};

	UBlueprintBoundNodeSpawner* NodeSpawner = UBlueprintBoundNodeSpawner::Create(GetClass());
	NodeSpawner->CanBindObjectDelegate    = UBlueprintBoundNodeSpawner::FCanBindObjectDelegate::CreateStatic(CanBindObjectLambda);
	NodeSpawner->OnBindObjectDelegate     = UBlueprintBoundNodeSpawner::FOnBindObjectDelegate::CreateStatic(PostBindSetupLambda);
	NodeSpawner->DynamicUiSignatureGetter = UBlueprintBoundNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(UiSpecOverride);

	ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
}

UK2Node::ERedirectType UK2Node_Literal::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// This allows the value pin (the only pin) to stay connected through reconstruction, even if the name changes due to an actor in the renamed, etc
	return ERedirectType_Name;
}

AActor* UK2Node_Literal::GetReferencedLevelActor() const
{
	return Cast<AActor>(ObjectRef);
}

UEdGraphPin* UK2Node_Literal::GetValuePin() const
{
	return (Pins.Num() > 0) ? Pins[0] : NULL;
}

void UK2Node_Literal::SetObjectRef(UObject* NewValue)
{
	UEdGraphPin* ValuePin = GetValuePin();

	// First, see if this is an object
	if( NewValue )
	{
		ObjectRef = NewValue;

		// Set the pin type to reflect the object we're referencing
		if( ValuePin )
		{
			ValuePin->Modify();
			ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Object;
			ValuePin->PinType.PinSubCategory = NAME_None;
			ValuePin->PinType.PinSubCategoryObject = ObjectRef->GetClass()->GetAuthoritativeClass();
		}
	}

	if( ValuePin )
	{
		ValuePin->PinFriendlyName = GetNodeTitle(ENodeTitleType::FullTitle);
		ValuePin->PinName = *ValuePin->PinFriendlyName.BuildSourceString();
	}
}

FNodeHandlingFunctor* UK2Node_Literal::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_LiteralStatement(CompilerContext);
}

FSlateIcon UK2Node_Literal::GetIconAndTint(FLinearColor& OutColor) const
{
	if(ObjectRef != NULL)
	{
		return FSlateIconFinder::FindIconForClass(ObjectRef->GetClass());	
	}
	return Super::GetIconAndTint(OutColor);
}

#undef LOCTEXT_NAMESPACE
