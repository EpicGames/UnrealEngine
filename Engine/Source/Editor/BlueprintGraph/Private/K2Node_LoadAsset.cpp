// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_LoadAsset.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_CustomEvent.h"
#include "K2Node_Event.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LoadAsset)

#define LOCTEXT_NAMESPACE "K2Node_LoadAsset"

void UK2Node_LoadAsset::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// The delayed completed pin, this used to be called Then
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Completed);

	CreatePin(EGPD_Input, GetInputCategory(), UObject::StaticClass(), GetInputPinName());
	CreatePin(EGPD_Output, GetOutputCategory(), UObject::StaticClass(), GetOutputPinName());
}

void UK2Node_LoadAsset::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	UEdGraphPin* OldThenPin = nullptr;
	UEdGraphPin* OldCompletedPin = nullptr;

	for (UEdGraphPin* CurrentPin : OldPins)
	{
		if (CurrentPin->PinName == UEdGraphSchema_K2::PN_Then)
		{
			OldThenPin = CurrentPin;
		}
		else if (CurrentPin->PinName == UEdGraphSchema_K2::PN_Completed)
		{
			OldCompletedPin = CurrentPin;
		}
	}

	if (OldThenPin && !OldCompletedPin)
	{
		// This is an old node from when Completed was called then, rename the node to Completed and allow normal rewire to take place
		OldThenPin->PinName = UEdGraphSchema_K2::PN_Completed;
	}
}

void UK2Node_LoadAsset::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	check(Schema);
	bool bIsErrorFree = true;

	// Sequence node, defaults to two output pins
	UK2Node_ExecutionSequence* SequenceNode = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
	SequenceNode->AllocateDefaultPins();

	// connect to input exe
	{
		UEdGraphPin* InputExePin = GetExecPin();
		UEdGraphPin* SequenceInputExePin = SequenceNode->GetExecPin();
		bIsErrorFree &= InputExePin && SequenceInputExePin && CompilerContext.MovePinLinksToIntermediate(*InputExePin, *SequenceInputExePin).CanSafeConnect();
	}

	// Create LoadAsset function call
	UK2Node_CallFunction* CallLoadAssetNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallLoadAssetNode->FunctionReference.SetExternalMember(NativeFunctionName(), UKismetSystemLibrary::StaticClass());
	CallLoadAssetNode->AllocateDefaultPins();

	// connect load to first sequence pin
	{
		UEdGraphPin* CallFunctionInputExePin = CallLoadAssetNode->GetExecPin();
		UEdGraphPin* SequenceFirstExePin = SequenceNode->GetThenPinGivenIndex(0);
		bIsErrorFree &= SequenceFirstExePin && CallFunctionInputExePin && Schema->TryCreateConnection(CallFunctionInputExePin, SequenceFirstExePin);
	}

	// connect then to second sequence pin
	{
		UEdGraphPin* OutputThenPin = FindPin(UEdGraphSchema_K2::PN_Then);
		UEdGraphPin* SequenceSecondExePin = SequenceNode->GetThenPinGivenIndex(1);
		bIsErrorFree &= OutputThenPin && SequenceSecondExePin && CompilerContext.MovePinLinksToIntermediate(*OutputThenPin, *SequenceSecondExePin).CanSafeConnect();
	}

	// Create Local Variable
	UEdGraphPin* AssetPin = FindPin(GetInputPinName());
	EPinContainerType PinContainerType = EPinContainerType::None;
	if (AssetPin)
	{
		PinContainerType = AssetPin->PinType.ContainerType;
	}
	UK2Node_TemporaryVariable* TempVarOutput = CompilerContext.SpawnInternalVariable(this, GetOutputCategory(), NAME_None, UObject::StaticClass(), PinContainerType);

	// Create assign node
	UK2Node_AssignmentStatement* AssignNode = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	AssignNode->AllocateDefaultPins();

	UEdGraphPin* LoadedObjectVariablePin = TempVarOutput->GetVariablePin();

	// connect local variable to assign node
	{
		UEdGraphPin* AssignLHSPPin = AssignNode->GetVariablePin();
		bIsErrorFree &= AssignLHSPPin && LoadedObjectVariablePin && Schema->TryCreateConnection(AssignLHSPPin, LoadedObjectVariablePin);
	}

	// connect local variable to output
	{
		UEdGraphPin* OutputObjectPinPin = FindPin(GetOutputPinName());
		bIsErrorFree &= LoadedObjectVariablePin && OutputObjectPinPin && CompilerContext.MovePinLinksToIntermediate(*OutputObjectPinPin, *LoadedObjectVariablePin).CanSafeConnect();
	}

	// connect to asset
	UEdGraphPin* CallFunctionAssetPin = CallLoadAssetNode->FindPin(GetInputPinName());
	{

		ensure(CallFunctionAssetPin);
		if (AssetPin && CallFunctionAssetPin)
		{
			if (AssetPin->LinkedTo.Num() > 0)
			{
				bIsErrorFree &= CompilerContext.MovePinLinksToIntermediate(*AssetPin, *CallFunctionAssetPin).CanSafeConnect();
			}
			else
			{
				// Copy literal value
				CallFunctionAssetPin->DefaultValue = AssetPin->DefaultValue;
			}
		}
		else
		{
			bIsErrorFree = false;
		}
	}

	// Create OnLoadEvent
	const FName DelegateOnLoadedParamName(TEXT("OnLoaded"));
	UK2Node_CustomEvent* OnLoadEventNode = CompilerContext.SpawnIntermediateNode<UK2Node_CustomEvent>(this, SourceGraph);
	OnLoadEventNode->CustomFunctionName = *FString::Printf(TEXT("OnLoaded_%s"), *CompilerContext.GetGuid(this));
	OnLoadEventNode->AllocateDefaultPins();
	{
		UFunction* LoadAssetFunction = CallLoadAssetNode->GetTargetFunction();
		FDelegateProperty* OnLoadDelegateProperty = LoadAssetFunction ? FindFProperty<FDelegateProperty>(LoadAssetFunction, DelegateOnLoadedParamName) : nullptr;
		UFunction* OnLoadedSignature = OnLoadDelegateProperty ? OnLoadDelegateProperty->SignatureFunction.Get() : nullptr;
		ensure(OnLoadedSignature);
		for (TFieldIterator<FProperty> PropIt(OnLoadedSignature); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;
			if (!Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				FEdGraphPinType PinType;
				bIsErrorFree &= Schema->ConvertPropertyToPinType(Param, /*out*/ PinType);
				bIsErrorFree &= (nullptr != OnLoadEventNode->CreateUserDefinedPin(Param->GetFName(), PinType, EGPD_Output));
			}
		}
	}

	// connect delegate
	{
		UEdGraphPin* CallFunctionDelegatePin = CallLoadAssetNode->FindPin(DelegateOnLoadedParamName);
		ensure(CallFunctionDelegatePin);
		UEdGraphPin* EventDelegatePin = OnLoadEventNode->FindPin(UK2Node_CustomEvent::DelegateOutputName);
		bIsErrorFree &= CallFunctionDelegatePin && EventDelegatePin && Schema->TryCreateConnection(CallFunctionDelegatePin, EventDelegatePin);
	}

	// connect loaded object from event to assign
	{
		UEdGraphPin* LoadedAssetEventPin = OnLoadEventNode->FindPin(TEXT("Loaded"));
		ensure(LoadedAssetEventPin);
		UEdGraphPin* AssignRHSPPin = AssignNode->GetValuePin();
		bIsErrorFree &= AssignRHSPPin && LoadedAssetEventPin && Schema->TryCreateConnection(LoadedAssetEventPin, AssignRHSPPin);
	}

	// connect assign exec input to event output
	{
		UEdGraphPin* OnLoadEventOutputPin = OnLoadEventNode->FindPin(UEdGraphSchema_K2::PN_Then);
		UEdGraphPin* AssignInputExePin = AssignNode->GetExecPin();
		bIsErrorFree &= AssignInputExePin && OnLoadEventOutputPin && Schema->TryCreateConnection(AssignInputExePin, OnLoadEventOutputPin);
	}

	// connect assign exec output to output
	{
		UEdGraphPin* OutputCompletedPin = FindPin(UEdGraphSchema_K2::PN_Completed);
		UEdGraphPin* AssignOutputExePin = AssignNode->GetThenPin();
		bIsErrorFree &= OutputCompletedPin && AssignOutputExePin && CompilerContext.MovePinLinksToIntermediate(*OutputCompletedPin, *AssignOutputExePin).CanSafeConnect();
	}

	if (!bIsErrorFree)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("InternalConnectionError", "K2Node_LoadAsset: Internal connection error. @@").ToString(), this);
	}

	BreakAllNodeLinks();
}

FText UK2Node_LoadAsset::GetTooltipText() const
{
	return FText(LOCTEXT("UK2Node_LoadAssetGetTooltipText", "Asynchronously loads a Soft Object Reference and returns object of the correct type if the load succeeds"));
}

FText UK2Node_LoadAsset::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UK2Node_LoadAssetGetNodeTitle", "Async Load Asset"));
}

bool UK2Node_LoadAsset::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool bIsCompatible = false;
	// Can only place events in ubergraphs and macros (other code will help prevent macros with latents from ending up in functions), and basicasync task creates an event node:
	EGraphType GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
	if (GraphType == EGraphType::GT_Ubergraph || GraphType == EGraphType::GT_Macro)
	{
		bIsCompatible = true;
	}
	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

FName UK2Node_LoadAsset::GetCornerIcon() const
{
	return TEXT("Graph.Latent.LatentIcon");
}

void UK2Node_LoadAsset::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_LoadAsset::GetMenuCategory() const
{
	return FText(LOCTEXT("UK2Node_LoadAssetGetMenuCategory", "Utilities"));
}

const FName& UK2Node_LoadAsset::GetInputCategory() const
{
	return UEdGraphSchema_K2::PC_SoftObject;
}

const FName& UK2Node_LoadAsset::GetOutputCategory() const
{
	return UEdGraphSchema_K2::PC_Object;
}

const FName& UK2Node_LoadAsset::GetInputPinName() const
{
	static const FName InputAssetPinName("Asset");
	return InputAssetPinName;
}

const FName& UK2Node_LoadAsset::GetOutputPinName() const
{
	static const FName OutputObjectPinName("Object");
	return OutputObjectPinName;
}

FName UK2Node_LoadAsset::NativeFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, LoadAsset);
}

// UK2Node_LoadAssetClass
FName UK2Node_LoadAssetClass::NativeFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, LoadAssetClass);
}

FText UK2Node_LoadAssetClass::GetTooltipText() const
{
	return FText(LOCTEXT("UK2Node_LoadAssetClassGetTooltipText", "Asynchronously loads a Soft Class Reference and returns class of the correct type if the load succeeds"));
}

FText UK2Node_LoadAssetClass::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UK2Node_LoadAssetClassGetNodeTitle", "Async Load Class Asset"));
}

const FName& UK2Node_LoadAssetClass::GetInputCategory() const
{
	return UEdGraphSchema_K2::PC_SoftClass;
}

const FName& UK2Node_LoadAssetClass::GetOutputCategory() const
{
	return UEdGraphSchema_K2::PC_Class;
}

const FName& UK2Node_LoadAssetClass::GetInputPinName() const
{
	static const FName InputAssetPinName("AssetClass");
	return InputAssetPinName;
}

const FName& UK2Node_LoadAssetClass::GetOutputPinName() const
{
	static const FName OutputObjectPinName("Class");
	return OutputObjectPinName;
}

// UK2Node_LoadAssets
void UK2Node_LoadAssets::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	// The immediate continue pin
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	// The delayed completed pin, this used to be called Then
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Completed);
	FCreatePinParams ArrayParams;
	ArrayParams.ContainerType = EPinContainerType::Array;
	CreatePin(EGPD_Input, GetInputCategory(), UObject::StaticClass(), GetInputPinName(), ArrayParams);
	CreatePin(EGPD_Output, GetOutputCategory(), UObject::StaticClass(), GetOutputPinName(), ArrayParams);
}

FText UK2Node_LoadAssets::GetTooltipText() const
{
	return FText(LOCTEXT("UK2Node_LoadAssetsGetTooltipText", "Asynchronously loads a list Soft Object References and returns the objects of the correct type if the load succeeds"));
}

FText UK2Node_LoadAssets::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return FText(LOCTEXT("UK2Node_LoadAssetsGetNodeTitle", "Async Load Assets"));
}

FName UK2Node_LoadAssets::NativeFunctionName() const
{
	return GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, LoadAssets);
}

const FName& UK2Node_LoadAssets::GetInputPinName() const
{
	static const FName InputAssetPinName("Assets");
	return InputAssetPinName;
}

const FName& UK2Node_LoadAssets::GetOutputPinName() const
{
	static const FName OutputObjectPinName("Objects");
	return OutputObjectPinName;
}

#undef LOCTEXT_NAMESPACE