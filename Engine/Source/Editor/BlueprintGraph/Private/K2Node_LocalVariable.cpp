// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_LocalVariable.h"

#include "Containers/Array.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_VariableGet.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_LocalVariable)


UDEPRECATED_K2Node_LocalVariable::UDEPRECATED_K2Node_LocalVariable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCanRenameNode = true;
	CustomVariableName = TEXT("NewLocalVar");
}

FText UDEPRECATED_K2Node_LocalVariable::GetTooltipText() const
{
	if(VariableTooltip.IsEmpty())
	{
		return Super::GetTooltipText();
	}

	return VariableTooltip;
}

FText UDEPRECATED_K2Node_LocalVariable::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (TitleType == ENodeTitleType::EditableTitle)
	{
		return FText::FromName(CustomVariableName);
	}
	else if(TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("TypeName"), UEdGraphSchema_K2::TypeToText(VariableType));
		return FText::Format(NSLOCTEXT("K2Node", "LocalVariable", "Local {TypeName}"), Args);
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("Title"), FText::FromName(CustomVariableName));
	return FText::Format(NSLOCTEXT("K2Node", "LocalVariable_Name", "{Title}\nLocal Variable"), Args);
}

void UDEPRECATED_K2Node_LocalVariable::OnRenameNode(const FString& NewName)
{
	if(CustomVariableName != *NewName)
	{
		const FScopedTransaction Transaction( NSLOCTEXT( "K2Node", "RenameLocalVariable", "Rename Local Variable" ) );
		Modify();

		CustomVariableName = *NewName;
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

TSharedPtr<class INameValidatorInterface> UDEPRECATED_K2Node_LocalVariable::MakeNameValidator() const
{
	return MakeShareable(new FKismetNameValidator(GetBlueprint(), *GetNodeTitle(ENodeTitleType::EditableTitle).ToString()));
}

void UDEPRECATED_K2Node_LocalVariable::ChangeVariableType(const FEdGraphPinType& InVariableType)
{
	// Local variables can never change type when the variable pin is hooked up
	check(GetVariablePin()->LinkedTo.Num() == 0);

	// Update the variable and the pin's type so that it properly reflects
	VariableType = InVariableType;
	GetVariablePin()->PinType = InVariableType;

	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UDEPRECATED_K2Node_LocalVariable::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	CustomVariableName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprint(), CustomVariableName.ToString());
}

void UDEPRECATED_K2Node_LocalVariable::PostPasteNode()
{
	Super::PostPasteNode();

	// Assign the local variable a unique name
	CustomVariableName = FBlueprintEditorUtils::FindUniqueKismetName(GetBlueprint(), CustomVariableName.GetPlainNameString());
}

bool UDEPRECATED_K2Node_LocalVariable::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	bool const bIsCompatible = (TargetGraph->GetSchema()->GetGraphType(TargetGraph) == GT_Function);
	return bIsCompatible && Super::IsCompatibleWithGraph(TargetGraph);
}

void UDEPRECATED_K2Node_LocalVariable::ReconstructNode()
{
	UEdGraph* Graph = GetGraph();

	UEdGraph* TopLevelGraph = FBlueprintEditorUtils::GetTopLevelGraph(Graph);
	if(TopLevelGraph->GetSchema()->GetGraphType(TopLevelGraph) == GT_Function)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// First, add the variable to the entry node
		FBPVariableDescription NewVar;
		NewVar.VarName = *GetNodeTitle(ENodeTitleType::EditableTitle).ToString();
		NewVar.VarGuid = FGuid::NewGuid();
		NewVar.VarType = GetVariablePin()->PinType;
		NewVar.PropertyFlags |= CPF_BlueprintVisible;
		NewVar.FriendlyName = FName::NameToDisplayString( NewVar.VarName.ToString(), (NewVar.VarType.PinCategory == UEdGraphSchema_K2::PC_Boolean) ? true : false );
		NewVar.Category = UEdGraphSchema_K2::VR_DefaultCategory;

		// Find the function entry node in the graph
		TArray<UK2Node_FunctionEntry*> FunctionEntryNodes;
		TopLevelGraph->GetNodesOfClass<UK2Node_FunctionEntry>(FunctionEntryNodes);
		check(FunctionEntryNodes.Num() == 1);

		UK2Node_FunctionEntry* FunctionEntry = FunctionEntryNodes[0];
		FunctionEntry->LocalVariables.Add(NewVar);

		// Duplicate the array, it is semi-destructive as we iterate through
		TArray< UEdGraphPin* > VariableLinkedPins = GetVariablePin()->LinkedTo;

		// Search through all linked pins for AssignmentStatement nodes, those need to be replaced with a VariableSet node and hooked up to the new variable
		for (UEdGraphPin* LinkedPin : VariableLinkedPins)
		{
			if (LinkedPin->GetOwningNode()->IsA<UK2Node_AssignmentStatement>())
			{
				UK2Node_AssignmentStatement* AssignementNode = Cast<UK2Node_AssignmentStatement>(LinkedPin->GetOwningNode());

				// Only replace the node if hooked up to the variable pin
				if(AssignementNode->GetVariablePin() == LinkedPin)
				{
					UK2Node_VariableSet* SetNode = NewObject<UK2Node_VariableSet>(Graph);
					SetNode->VariableReference.SetLocalMember(NewVar.VarName, TopLevelGraph->GetName(), NewVar.VarGuid);
					Graph->AddNode(SetNode, false, false);
					SetNode->CreateNewGuid();
					SetNode->PostPlacedNewNode();

					// Locally re-construct the pins, can't allow the node to do it because the property does not yet exist
					UEdGraphPin* ExecPin = SetNode->CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
					UEdGraphPin* ThenPin = SetNode->CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
					UEdGraphPin* ValuePin = SetNode->CreatePin(EGPD_Input, NAME_None, NewVar.VarName);
					ValuePin->PinType = NewVar.VarType;

					// Move the pin links over to the new node
					K2Schema->MovePinLinks(*AssignementNode->FindPin(UEdGraphSchema_K2::PN_Execute), *ExecPin);
					K2Schema->MovePinLinks(*AssignementNode->FindPin(UEdGraphSchema_K2::PN_Then), *ThenPin);
					K2Schema->MovePinLinks(*AssignementNode->GetValuePin(), *ValuePin);

					// Position the new node
					SetNode->NodePosX = AssignementNode->NodePosX;
					SetNode->NodePosY = AssignementNode->NodePosY;

					// Destroy the assignment node
					AssignementNode->DestroyNode();
				}
			}
		}

		// Only replace with a get node if the node still has connections
		if(GetVariablePin()->LinkedTo.Num())
		{
			// Only the local variable node needs to be re-constructed as a VariableGet node, there are no other nodes representing this local variable
			UK2Node_VariableGet* GetNode = NewObject<UK2Node_VariableGet>(Graph);
			GetNode->VariableReference.SetLocalMember(NewVar.VarName, TopLevelGraph->GetName(), NewVar.VarGuid);
			Graph->AddNode(GetNode, false, false);
			GetNode->CreateNewGuid();
			GetNode->PostPlacedNewNode();

			// Locally re-construct the pins, can't allow the node to do it because the property does not yet exist
			UEdGraphPin* VariablePin = GetNode->CreatePin(EGPD_Output, NAME_None, NewVar.VarName);
			VariablePin->PinType = NewVar.VarType;
			K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(VariablePin);

			// Position the new node
			GetNode->NodePosX = NodePosX;
			GetNode->NodePosY = NodePosY;

			// Move the pin links over to the new node
			K2Schema->MovePinLinks(*GetVariablePin(), *GetNode->GetValuePin());
		}
	}

	// This node should not persist anymore, it is deprecated and the above case catches any still valid uses and deletes the rest (invalid ones are from AnimationTransitionGraph)
	DestroyNode();
}
