// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "DelegateNodeHandlers.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_AddDelegate.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_CallDelegate.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ClearDelegate.h"
#include "K2Node_RemoveDelegate.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FK2Node_BaseMCDelegateHelper
{
	static FName DelegatePinName;
};
FName FK2Node_BaseMCDelegateHelper::DelegatePinName(TEXT("Delegate"));

/////// UK2Node_BaseMCDelegate ///////////

UK2Node_BaseMCDelegate::UK2Node_BaseMCDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_BaseMCDelegate::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
	if(FProperty* Property = GetProperty())
	{
		if(!Property->HasAllPropertyFlags(CPF_BlueprintAssignable))
		{
			MessageLog.Error(*NSLOCTEXT("K2Node", "BaseMCDelegateNotAssignable", "Event Dispatcher is not 'BlueprintAssignable' @@").ToString(), this);
		}
	}
}

bool UK2Node_BaseMCDelegate::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	UEdGraphSchema const* Schema = TargetGraph->GetSchema();
	EGraphType GraphType = Schema->GetGraphType(TargetGraph);

	bool const bIsCompatible = (GraphType == GT_Ubergraph) || (GraphType == GT_Function);
	return bIsCompatible&& Super::IsCompatibleWithGraph(TargetGraph);
}

UK2Node::ERedirectType UK2Node_BaseMCDelegate::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType OrginalResult = Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if ((ERedirectType::ERedirectType_None == OrginalResult) && NewPin && OldPin)
	{
		if ((OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate) &&
			(NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Delegate) &&
			(NewPin->PinName == OldPin->PinName))
		{
			return ERedirectType_Name;
		}
	}
	return OrginalResult;
}

void UK2Node_BaseMCDelegate::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	UClass* PropertyOwnerClass = DelegateReference.GetMemberParentClass(GetBlueprintClassFromNode());
	if (PropertyOwnerClass != nullptr)
	{
		PropertyOwnerClass = PropertyOwnerClass->GetAuthoritativeClass();
	}
	
	UEdGraphPin* SelfPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, PropertyOwnerClass, UEdGraphSchema_K2::PN_Self);

	if (SelfPin)
	{
		SelfPin->PinFriendlyName = NSLOCTEXT("K2Node", "BaseMCDelegateSelfPinName", "Target");
	}
}

UFunction* UK2Node_BaseMCDelegate::GetDelegateSignature(bool bForceNotFromSkelClass) const
{
	UClass* OwnerClass = DelegateReference.GetMemberParentClass(GetBlueprintClassFromNode());
	if (bForceNotFromSkelClass)
	{
		OwnerClass = (OwnerClass != nullptr) ? OwnerClass->GetAuthoritativeClass() : nullptr;
	}
	else if (UBlueprintGeneratedClass* BpClassOwner = Cast<UBlueprintGeneratedClass>(OwnerClass))
	{
		UBlueprint* DelegateBlueprint = Cast<UBlueprint>(BpClassOwner->ClassGeneratedBy);
		// favor the skeleton class, because the generated class may not 
		// have the delegate yet (hasn't been compiled with it), or it could 
		// be out of date
		UClass* SkeletonClass = DelegateBlueprint ? *DelegateBlueprint->SkeletonGeneratedClass : nullptr;
		OwnerClass = SkeletonClass ? SkeletonClass : OwnerClass;
	}

	FMemberReference ReferenceToUse;
	ReferenceToUse.SetDirect(DelegateReference.GetMemberName(), DelegateReference.GetMemberGuid(), OwnerClass, /*bIsConsideredSelfContext =*/false);

	FMulticastDelegateProperty* DelegateProperty = ReferenceToUse.ResolveMember<FMulticastDelegateProperty>();
	return (DelegateProperty != nullptr) ? DelegateProperty->SignatureFunction : nullptr;
}

UEdGraphPin* UK2Node_BaseMCDelegate::GetDelegatePin() const
{
	return FindPin(FK2Node_BaseMCDelegateHelper::DelegatePinName);
}

FString UK2Node_BaseMCDelegate::GetDocumentationLink() const
{
	UClass* ParentClass = nullptr;
	if (DelegateReference.IsSelfContext())
	{
		if (HasValidBlueprint())
		{
			UField* Delegate = FindUField<UField>(GetBlueprint()->GeneratedClass, DelegateReference.GetMemberName());
			if (Delegate)
			{
				ParentClass = Delegate->GetOwnerClass();
			}
		}		
	}
	else 
	{
		ParentClass = DelegateReference.GetMemberParentClass(GetBlueprintClassFromNode());
	}

	if (ParentClass)
	{
		return FString( TEXT("Shared/") ) + ParentClass->GetName();
	}

	return FString();
}

FString UK2Node_BaseMCDelegate::GetDocumentationExcerptName() const
{
	return DelegateReference.GetMemberName().ToString();
}

void UK2Node_BaseMCDelegate::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const bool bAllowMultipleSelfs = AllowMultipleSelfs(true);
	if(bAllowMultipleSelfs)
	{
		const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
		check(Schema);
		UEdGraphPin* MultiSelf = Schema->FindSelfPin(*this, EEdGraphPinDirection::EGPD_Input);
		check(MultiSelf);

		const bool bProperInputToExpandForEach = 
			(MultiSelf->LinkedTo.Num()) && 
			(NULL != MultiSelf->LinkedTo[0]) && 
			(MultiSelf->LinkedTo[0]->PinType.IsArray());
		if(bProperInputToExpandForEach)
		{
			if(MultiSelf->LinkedTo.Num() > 1)
			{
				CompilerContext.MessageLog.Error(*NSLOCTEXT("K2Node", "BaseMCDelegateMultiArray", "Event Dispatcher does not accept multi-array-self @@").ToString(), this);
			}
			else
			{
				UK2Node_CallFunction::CallForEachElementInArrayExpansion(this, MultiSelf, CompilerContext, SourceGraph);
			}
		}
	}
}

bool UK2Node_BaseMCDelegate::IsAuthorityOnly() const
{
	const FMulticastDelegateProperty* DelegateProperty = DelegateReference.ResolveMember<FMulticastDelegateProperty>(GetBlueprintClassFromNode());
	return DelegateProperty && DelegateProperty->HasAnyPropertyFlags(CPF_BlueprintAuthorityOnly);

}

bool UK2Node_BaseMCDelegate::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const UBlueprint* SourceBlueprint = GetBlueprint();

	FProperty* MCProperty = GetProperty();
	UClass* PropertySourceClass = MCProperty ? MCProperty->GetOwnerClass() : nullptr;
	const bool bPropertyResult = (PropertySourceClass != NULL) && (PropertySourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bPropertyResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(PropertySourceClass);
	}

	UFunction* Signature = GetDelegateSignature(true);

	checkf(!Signature || Signature->IsValidLowLevel(), TEXT("Invalid Signature: BP - %s Delegate Ref - %s %s" ),
		*SourceBlueprint->GetName(),
		*DelegateReference.GetMemberScopeName(),
		*DelegateReference.GetMemberName().ToString()
	);
	
	UClass* SignatureSourceClass = Signature ? Signature->GetOwnerClass() : nullptr;
	const bool bSignatureResult = (SignatureSourceClass != NULL) && (SignatureSourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bSignatureResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(Signature);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSignatureResult || bPropertyResult || bSuperResult;
}

void UK2Node_BaseMCDelegate::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "EventDelegate" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetPropertyName().ToString() ));
}

void UK2Node_BaseMCDelegate::AutowireNewNode(UEdGraphPin* FromPin)
{
	const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(GetSchema());

	// Since nodes no longer have a sense of scope when they're placed, look at the connection we're coming from, and use that to coerce the Target pin
	if (FromPin && K2Schema)
	{
		bool bConnected = false;

		// Only do the fixup if we're going coming from an output pin, which implies a contextual drag
		if (FromPin->Direction == EGPD_Output)
		{
			if (FromPin->PinType.PinSubCategoryObject.IsValid() && FromPin->PinType.PinSubCategoryObject->IsA(UClass::StaticClass()))
			{
				FProperty* DelegateProperty = DelegateReference.ResolveMember<FMulticastDelegateProperty>(GetBlueprintClassFromNode());
				if (DelegateProperty)
				{
					UClass* DelegateOwner = DelegateProperty->GetOwnerClass();
					const UClass* PinSubCategoryObject = dynamic_cast<UClass*>(FromPin->PinType.PinSubCategoryObject.Get());
					if (FromPin->PinType.PinSubCategoryObject == DelegateOwner || (PinSubCategoryObject && PinSubCategoryObject->IsChildOf(DelegateOwner)))
					{
						// If we get here, then the property delegate is also available on the FromPin's class.
						// Fix up the type by propagating it from the FromPin to our target pin
						UEdGraphPin* TargetPin = FindPin(UEdGraphSchema_K2::PN_Self);
						TargetPin->PinType.PinSubCategory = NAME_None;
						TargetPin->PinType.PinSubCategoryObject = DelegateOwner;

						// And finally, try to establish the connection
						if (K2Schema->TryCreateConnection(FromPin, TargetPin))
						{
							bConnected = true;

							DelegateReference.SetFromField<FProperty>(DelegateProperty, false);
							TargetPin->bHidden = false;
							FromPin->GetOwningNode()->NodeConnectionListChanged();
							this->NodeConnectionListChanged();
						}

					}
				}
			}
		}

		if (!bConnected)
		{
			Super::AutowireNewNode(FromPin);
		}
	}
}

bool UK2Node_BaseMCDelegate::HasDeprecatedReference() const
{
	// Check if the referenced delegate is deprecated.
	return DelegateReference.IsDeprecated();
}

FEdGraphNodeDeprecationResponse UK2Node_BaseMCDelegate::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		if (FProperty* DelegateProperty = DelegateReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
		{
			// Check the deprecation type to override the severity
			FString MessageType = DelegateProperty->GetMetaData(FBlueprintMetadata::MD_DeprecatedProperty);
			Response.MessageType = FBlueprintEditorUtils::GetDeprecatedMessageType(MessageType);

			FString DetailedMessage = DelegateProperty->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
			Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(GetPropertyDisplayName(), FText::FromString(DetailedMessage));
		}
	}
	
	return Response;
}

/////// UK2Node_AddDelegate ///////////

UK2Node_AddDelegate::UK2Node_AddDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_AddDelegate::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.bIsReference = true;
	PinParams.bIsConst = true;
	if(UEdGraphPin* DelegatePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Delegate, FK2Node_BaseMCDelegateHelper::DelegatePinName, PinParams))
	{
		FMemberReference::FillSimpleMemberReference<UFunction>(GetDelegateSignature(), DelegatePin->PinType.PinSubCategoryMemberReference);
		DelegatePin->PinFriendlyName = NSLOCTEXT("K2Node", "PinFriendlyDelegatetName", "Event");
	}
}

FText UK2Node_AddDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyName"), GetPropertyDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "AddDelegate", "Bind Event to {PropertyName}"), Args), this);
	}
	return CachedNodeTitle;
}

FNodeHandlingFunctor* UK2Node_AddDelegate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_AddRemoveDelegate(CompilerContext, KCST_AddMulticastDelegate);
}

void UK2Node_AddDelegate::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "AddDelegate" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetPropertyName().ToString() ));
}

/////// UK2Node_ClearDelegate ///////////

UK2Node_ClearDelegate::UK2Node_ClearDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UK2Node_ClearDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyName"), GetPropertyDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "ClearDelegate", "Unbind all Events from {PropertyName}"), Args), this);
	}
	return CachedNodeTitle;
}

FNodeHandlingFunctor* UK2Node_ClearDelegate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_ClearDelegate(CompilerContext);
}

/////// UK2Node_RemoveDelegate ///////////

UK2Node_RemoveDelegate::UK2Node_RemoveDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_RemoveDelegate::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.bIsReference = true;
	PinParams.bIsConst = true;
	if(UEdGraphPin* DelegatePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Delegate, FK2Node_BaseMCDelegateHelper::DelegatePinName, PinParams))
	{
		FMemberReference::FillSimpleMemberReference<UFunction>(GetDelegateSignature(), DelegatePin->PinType.PinSubCategoryMemberReference);
		DelegatePin->PinFriendlyName = NSLOCTEXT("K2Node", "PinFriendlyDelegatetName", "Event");
	}
}

FText UK2Node_RemoveDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyName"), GetPropertyDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "RemoveDelegate", "Unbind Event from {PropertyName}"), Args), this);
	}
	return CachedNodeTitle;
}

FNodeHandlingFunctor* UK2Node_RemoveDelegate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_AddRemoveDelegate(CompilerContext, KCST_RemoveMulticastDelegate);
}

/////// UK2Node_CallDelegate ///////////

UK2Node_CallDelegate::UK2Node_CallDelegate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UK2Node_CallDelegate::CreatePinsForFunctionInputs(const UFunction* Function)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Create the inputs and outputs
	bool bAllPinsGood = true;
	for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
	{
		FProperty* Param = *PropIt;
		const bool bIsFunctionInput = !Param->HasAnyPropertyFlags(CPF_OutParm) || Param->HasAnyPropertyFlags(CPF_ReferenceParm);
		if (bIsFunctionInput)
		{
			UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Param->GetFName());
			const bool bPinGood = K2Schema->ConvertPropertyToPinType(Param, /*out*/ Pin->PinType);

			bAllPinsGood = bAllPinsGood && bPinGood;
		}
	}

	return bAllPinsGood;
}

void UK2Node_CallDelegate::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePinsForFunctionInputs(GetDelegateSignature());
}

FText UK2Node_CallDelegate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PropertyName"), GetPropertyDisplayName());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "CallDelegate", "Call {PropertyName}"), Args), this);
	}
	return CachedNodeTitle;
}

void UK2Node_CallDelegate::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	UK2Node::ValidateNodeDuringCompilation(MessageLog);
	if(FProperty* Property = GetProperty())
	{
		if(!Property->HasAllPropertyFlags(CPF_BlueprintCallable))
		{
			MessageLog.Error(*NSLOCTEXT("K2Node", "BaseMCDelegateNotCallable", "Event Dispatcher is not 'BlueprintCallable' @@").ToString(), this);
		}
	}
}

FNodeHandlingFunctor* UK2Node_CallDelegate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_CallDelegate(CompilerContext);
}

FName UK2Node_CallDelegate::GetCornerIcon() const
{
	return TEXT("Graph.Message.MessageIcon");
}
