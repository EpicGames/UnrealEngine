// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MacroInstance.h"

#include "BlueprintActionFilter.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_EditablePinBase.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/WildcardNodeUtils.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Settings/EditorStyleSettings.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MacroInstance)

#define LOCTEXT_NAMESPACE "K2Node_MacroInstance"

UK2Node_MacroInstance::UK2Node_MacroInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReconstructNode = false;
}

void UK2Node_MacroInstance::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_K2NODE_REFERENCEGUIDS)
	{
		MacroGraphReference.SetGraph(MacroGraph_DEPRECATED);
	}
}

bool UK2Node_MacroInstance::IsActionFilteredOut(FBlueprintActionFilter const& Filter)
{
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UEdGraph* Graph : FilterContext.Graphs)
	{
		if (!CanPasteHere(Graph))
		{
			bIsFilteredOut = true;
			break;
		}
	}
	return bIsFilteredOut;
}

void UK2Node_MacroInstance::PostPasteNode()
{
	const UBlueprint* InstanceOwner = GetBlueprint();

	// Find the owner of the macro graph
	if (const UEdGraph* MacroGraph = MacroGraphReference.GetGraph())
	{
		UObject* MacroOwner = MacroGraph->GetOuter();
		UBlueprint* MacroOwnerBP = nullptr;
		while (MacroOwner)
		{
			MacroOwnerBP = Cast<UBlueprint>(MacroOwner);
			if (MacroOwnerBP)
			{
				break;
			}

			MacroOwner = MacroOwner->GetOuter();
		}

		if ((MacroOwnerBP != nullptr)
			&& (MacroOwnerBP->BlueprintType != BPTYPE_MacroLibrary)
			&& (MacroOwnerBP != InstanceOwner))
		{
			// If this is a graph from another blueprint that is NOT a library, disallow the connection!
			MacroGraphReference.SetGraph(nullptr);
		}
	}
	else
	{
		// Can't find the referenced macro, fully clear this reference
		MacroGraphReference.SetGraph(nullptr);
	}

	Super::PostPasteNode();
}

void UK2Node_MacroInstance::AllocateDefaultPins()
{
	UK2Node::AllocateDefaultPins();

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	PreloadObject(MacroGraphReference.GetBlueprint());
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	
	if (MacroGraph != nullptr)
	{
		PreloadObject(MacroGraph);

		// Preload the macro graph, if needed, so that we can get the proper pins
		if (MacroGraph->HasAnyFlags(RF_NeedLoad))
		{
			PreloadObject(MacroGraph);
			FBlueprintEditorUtils::PreloadMembers(MacroGraph);
		}

		for (decltype(MacroGraph->Nodes)::TIterator NodeIt(MacroGraph->Nodes); NodeIt; ++NodeIt)
		{
			if (UK2Node_Tunnel* TunnelNode = Cast<UK2Node_Tunnel>(*NodeIt))
			{
				// Only want exact tunnel nodes, more specific nodes like composites or macro instances shouldn't be grabbed.
				if (TunnelNode->GetClass() == UK2Node_Tunnel::StaticClass())
				{
					for (TArray<UEdGraphPin*>::TIterator PinIt(TunnelNode->Pins); PinIt; ++PinIt)
					{
						UEdGraphPin* PortPin = *PinIt;

						// We're not interested in any pins that have been expanded internally on the macro
						if (PortPin->ParentPin == nullptr)
						{
							UEdGraphPin* NewLocalPin = CreatePin(UEdGraphPin::GetComplementaryDirection(PortPin->Direction), PortPin->PinType, PortPin->PinName);
							Schema->SetPinAutogeneratedDefaultValue(NewLocalPin, PortPin->GetDefaultAsString());
						}
					}
				}
			}
		}
	}

	CacheWildcardPins();
}

void UK2Node_MacroInstance::PreloadRequiredAssets()
{
	PreloadObject(MacroGraphReference.GetBlueprint());
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	PreloadObject(MacroGraph);
	Super::PreloadRequiredAssets();
}

FText UK2Node_MacroInstance::GetTooltipText() const
{
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetAssociatedGraphMetadata(MacroGraph))
	{
		if (!Metadata->ToolTip.IsEmpty())
		{
			return Metadata->ToolTip;
		}
	}

	if (MacroGraph == nullptr)
	{
		return NSLOCTEXT("K2Node", "Macro_Tooltip", "Macro");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "MacroGraphInstance_Tooltip", "{0} instance"), FText::FromName(MacroGraph->GetFName())), this);
	}
	return CachedTooltip;
}

FText UK2Node_MacroInstance::GetKeywords() const
{
	FText Keywords = GetAssociatedGraphMetadata(GetMacroGraph())->Keywords;

	// If the Macro has Compact Node Title data, append the compact node title as a Keyword so it can be searched.
	if (ShouldDrawCompact())
	{
		Keywords = FText::Format(FText::FromString(TEXT("{0} {1}")), Keywords, GetCompactNodeTitle());
	}
	return Keywords;
}

FText UK2Node_MacroInstance::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	FText Result = NSLOCTEXT("K2Node", "MacroInstance", "Macro instance");
	if (MacroGraph)
	{
		Result = FText::FromString(MacroGraph->GetName());
		if ((GEditor != NULL) && (GetDefault<UEditorStyleSettings>()->bShowFriendlyNames))
		{
			Result = FText::FromString(FName::NameToDisplayString(Result.ToString(), false));
		}
	}

	return Result;
}

FLinearColor UK2Node_MacroInstance::GetNodeTitleColor() const
{
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	if (FKismetUserDeclaredFunctionMetadata* Metadata = GetAssociatedGraphMetadata(MacroGraph))
	{
		return Metadata->InstanceTitleColor.ToFColor(false);
	}

	return FLinearColor::White;
}

void UK2Node_MacroInstance::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	if ( Context->Pin == nullptr )
	{
		{
			FToolMenuSection& Section = Menu->AddSection("K2NodeMacroInstance", NSLOCTEXT("K2Node", "MacroInstanceHeader", "Macro Instance"));
			Section.AddMenuEntry(
				"MacroInstanceFindInContentBrowser",
				NSLOCTEXT("K2Node", "MacroInstanceFindInContentBrowser", "Find in Content Browser"),
				NSLOCTEXT("K2Node", "MacroInstanceFindInContentBrowserTooltip", "Finds the Blueprint Macro Library that contains this Macro in the Content Browser"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
				FUIAction( FExecuteAction::CreateStatic( &UK2Node_MacroInstance::FindInContentBrowser, MakeWeakObjectPtr(const_cast<UK2Node_MacroInstance*>(this)) ) )
				);
		}
	}
}

FKismetUserDeclaredFunctionMetadata* UK2Node_MacroInstance::GetAssociatedGraphMetadata(const UEdGraph* AssociatedMacroGraph)
{
	if (AssociatedMacroGraph)
	{
		// Look at the graph for the entry node, to get the default title color
		TArray<UK2Node_Tunnel*> TunnelNodes;
		AssociatedMacroGraph->GetNodesOfClass(TunnelNodes);

		for (int32 i = 0; i < TunnelNodes.Num(); i++)
		{
			UK2Node_Tunnel* Node = TunnelNodes[i];
			if (Node->IsEditable())
			{
				if (Node->bCanHaveOutputs)
				{
					return &(Node->MetaData);
				}
			}
		}
	}

	return nullptr;
}

void UK2Node_MacroInstance::FindInContentBrowser(TWeakObjectPtr<UK2Node_MacroInstance> MacroInstance)
{
	if ( MacroInstance.IsValid() )
	{
		UEdGraph* InstanceMacroGraph = MacroInstance.Get()->MacroGraphReference.GetGraph();
		if ( InstanceMacroGraph )
		{
			UBlueprint* BlueprintToSync = FBlueprintEditorUtils::FindBlueprintForGraph(InstanceMacroGraph);
			if ( BlueprintToSync )
			{
				TArray<UObject*> ObjectsToSync;
				ObjectsToSync.Add( BlueprintToSync );
				GEditor->SyncBrowserToObjects(ObjectsToSync);
			}
		}
	}
}




void UK2Node_MacroInstance::NotifyPinConnectionListChanged(UEdGraphPin* ChangedPin)
{
	Super::NotifyPinConnectionListChanged(ChangedPin);
	const bool bShouldDoSmartInference = ShouldDoSmartWildcardInference();
	if(bShouldDoSmartInference)
	{
		const bool bIsWildcardPin = FWildcardNodeUtils::HasAnyWildcards(ChangedPin);
		if (bIsWildcardPin && ChangedPin->LinkedTo.Num() > 0)
		{
			// Search the changed pin's links for an inferrable pin:
			if(const UEdGraphPin* InferrablePin = FWildcardNodeUtils::FindInferrableLinkedPin(ChangedPin))
			{
				// we found one, infer from it and then propagate the inference:
				FWildcardNodeUtils::InferType(ChangedPin, InferrablePin->PinType);

				const UEdGraph* Graph = GetGraph();
				const bool bIsMacroGraph = (Graph->GetSchema()->GetGraphType(Graph) == GT_Macro);
				if (!bIsMacroGraph)
				{
					InferWildcards();
				}
			}
		}
	}

	// added a link?
	if (ChangedPin->LinkedTo.Num() > 0)
	{
		// ... to a wildcard pin?
		const bool bIsWildcardPin = ChangedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard;
		if (bIsWildcardPin)
		{
			// get type of pin we just got linked to
			FEdGraphPinType const LinkedPinType = ChangedPin->LinkedTo[0]->PinType;

			// change all other wildcard pins to the new type
			// note we're assuming only one wildcard type per Macro node, for now
			if(!bShouldDoSmartInference)
			{
				for(int32 PinIdx=0; PinIdx<Pins.Num(); PinIdx++)
				{
					UEdGraphPin* const TmpPin = Pins[PinIdx];
					if (FWildcardNodeUtils::IsWildcardPin(TmpPin))
					{
						// only copy the category stuff to preserve array and ref status
						TmpPin->PinType.PinCategory = LinkedPinType.PinCategory;
						TmpPin->PinType.PinSubCategory = LinkedPinType.PinSubCategory;
						TmpPin->PinType.PinSubCategoryObject = LinkedPinType.PinSubCategoryObject;
					}
				}
			}

			ResolvedWildcardType = LinkedPinType;
			bReconstructNode = true;
		}
	}
 	else
	{
		// reconstruct on disconnects so we can revert back to wildcards if necessary
		bReconstructNode = true;
	}
}

void UK2Node_MacroInstance::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	if (bReconstructNode)
	{
		ReconstructNode();

		UBlueprint* const Blueprint = GetBlueprint();
		if (Blueprint && !Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

FString UK2Node_MacroInstance::GetDocumentationLink() const
{
	return TEXT("Shared/GraphNodes/Blueprint/UK2Node_MacroInstance");
}

FString UK2Node_MacroInstance::GetDocumentationExcerptName() const
{
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	if (MacroGraph != nullptr)
	{
		return MacroGraph->GetName();
	}
	return Super::GetDocumentationExcerptName();
}

void UK2Node_MacroInstance::PostReconstructNode()
{
	bReconstructNode = false;

	if(ShouldDoSmartWildcardInference())
	{
		// conform any type mismatches - or just conform
		for(UEdGraphPin* Pin : WildcardPins)
		{
			if (FWildcardNodeUtils::HasAnyWildcards(Pin))
			{
				const FEdGraphPinType* ConnectedType = nullptr;
				for(UEdGraphPin* Link : Pin->LinkedTo)
				{
					if(!FWildcardNodeUtils::HasAnyWildcards(Link))
					{
						ConnectedType = &Link->PinType;
					}
				}

				if(ConnectedType)
				{
					FWildcardNodeUtils::InferType(Pin->PinType, *ConnectedType);
				}
			}
		}

		
		const UEdGraph* Graph = GetGraph();
		const bool bIsMacroGraph = (Graph->GetSchema()->GetGraphType(Graph) == GT_Macro);
		UBlueprint* Blueprint  = GetBlueprint();
		const bool bIsCompiling = Blueprint ? Blueprint->bBeingCompiled : false;
		if(!bIsMacroGraph || !bIsCompiling)
		{
			// rerun inference
			InferWildcards();
		}
	}
	else
	{
		// fix up ResolvedWildcardType, which could have been cleared for certain CL ranges
		if (ResolvedWildcardType.PinCategory.IsNone() && WildcardPins.Num() > 0)
		{
			UEdGraphPin* const* NonWildcardPin = Algo::FindByPredicate(WildcardPins,
				[](const UEdGraphPin* Pin )
				{
					return !FWildcardNodeUtils::IsWildcardPin(Pin);
				});

			if(NonWildcardPin)
			{
				ResolvedWildcardType = (*NonWildcardPin)->PinType;
			}
		}
	}

	Super::PostReconstructNode();
}

FName UK2Node_MacroInstance::GetCornerIcon() const
{
	if (UEdGraph* MacroGraph = MacroGraphReference.GetGraph())
	{
		FBlueprintMacroCosmeticInfo CosmeticInfo = FBlueprintEditorUtils::GetCosmeticInfoForMacro(MacroGraph);
		if (CosmeticInfo.bContainsLatentNodes)
		{
			return TEXT("Graph.Latent.LatentIcon");
		}
	}

	return Super::GetCornerIcon();
}

FSlateIcon UK2Node_MacroInstance::GetIconAndTint(FLinearColor& OutColor) const
{
	const char* IconName = "GraphEditor.Macro_16x";

	// Special case handling for standard macros
	// @TODO Change this to a SlateBurushAsset pointer on the graph or something similar, to allow any macro to have an icon
	UEdGraph* MacroGraph = MacroGraphReference.GetGraph();
	if(MacroGraph != nullptr && MacroGraph->GetOuter()->GetName() == TEXT("StandardMacros"))
	{
		FName MacroName = FName(*MacroGraph->GetName());
		if(	MacroName == TEXT("ForLoop" ) ||
			MacroName == TEXT("ForLoopWithBreak") ||
			MacroName == TEXT("WhileLoop") )
		{
			IconName = "GraphEditor.Macro.Loop_16x";
		}
		else if( MacroName == TEXT("Gate") )
		{
			IconName = "GraphEditor.Macro.Gate_16x";
		}
		else if( MacroName == TEXT("Do N") )
		{
			IconName = "GraphEditor.Macro.DoN_16x";
		}
		else if (MacroName == TEXT("DoOnce"))
		{
			IconName = "GraphEditor.Macro.DoOnce_16x";
		}
		else if (MacroName == TEXT("IsValid"))
		{
			IconName = "GraphEditor.Macro.IsValid_16x";
		}
		else if (MacroName == TEXT("FlipFlop"))
		{
			IconName = "GraphEditor.Macro.FlipFlop_16x";
		}
		else if ( MacroName == TEXT("ForEachLoop") ||
				  MacroName == TEXT("ForEachLoopWithBreak") )
		{
			IconName = "GraphEditor.Macro.ForEach_16x";
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName);
}

FText UK2Node_MacroInstance::GetCompactNodeTitle() const
{
	FText ResultText;
	if (FKismetUserDeclaredFunctionMetadata* MacroGraphMetadata = GetAssociatedGraphMetadata(GetMacroGraph()))
	{
		ResultText = MacroGraphMetadata->CompactNodeTitle;
	}
	return ResultText;	
}

bool UK2Node_MacroInstance::ShouldDrawCompact() const
{
	return !GetCompactNodeTitle().IsEmpty();
}

bool UK2Node_MacroInstance::CanPasteHere(const UEdGraph* TargetGraph) const
{
	bool bCanPaste = false;

	UBlueprint* MacroBlueprint  = GetSourceBlueprint();
	UBlueprint* TargetBlueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);

	if ((MacroBlueprint != nullptr) && (TargetBlueprint != nullptr))
	{
		// Only allow "local" macro instances or instances from a macro library blueprint with the same parent class
		check(MacroBlueprint->ParentClass != nullptr && TargetBlueprint->ParentClass != nullptr);
		bCanPaste = (MacroBlueprint == TargetBlueprint) || (MacroBlueprint->BlueprintType == BPTYPE_MacroLibrary
			&& TargetBlueprint->ParentClass && TargetBlueprint->ParentClass->IsChildOf(MacroBlueprint->ParentClass));
	}

	// Macro Instances are not allowed in it's own graph
	UEdGraph* MacroGraph = GetMacroGraph();
	bCanPaste &= (MacroGraph != TargetGraph);
	// nor in Function graphs if the macro has latent functions in it
	bool const bIsTargetFuncGraph = (TargetGraph->GetSchema()->GetGraphType(TargetGraph) == GT_Function);
	bCanPaste &= (!bIsTargetFuncGraph || !FBlueprintEditorUtils::CheckIfGraphHasLatentFunctions(MacroGraph));

	return bCanPaste && Super::CanPasteHere(TargetGraph);
}

void UK2Node_MacroInstance::PostFixupAllWildcardPins(bool bInAllWildcardPinsUnlinked)
{
	if (bInAllWildcardPinsUnlinked)
	{
		// Reset the type to a wildcard because there are no longer any wildcard pins linked to determine a type with
		ResolvedWildcardType.ResetToDefaults();

		// Collapse any wildcard pins that are split and set their type back to wildcard
		// doing this would be unsafe when using smart wildcard inference
		// because recombining pin in the middle of reconstruction could result in
		// pin allocation during reconstruction. Therefore we don't rely upon it 
		// when doing smart wildcard inference
		if (!ShouldDoSmartWildcardInference())
		{
			for (UEdGraphPin* Pin : WildcardPins)
			{
				GetSchema()->RecombinePin(Pin);

				Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
				Pin->PinType.PinSubCategory = NAME_None;
				Pin->PinType.PinSubCategoryObject = nullptr;
			}
		}
	}
}

void UK2Node_MacroInstance::InferWildcards(const TArray<UEdGraphNode*>& InNodes) const
{
	if(ShouldDoSmartWildcardInference())
	{
		SmartInferWildcardsImpl(InNodes);
		return;
	}

	if (!ResolvedWildcardType.PinCategory.IsNone())
	{
		for (UEdGraphNode* const ClonedNode : InNodes)
		{
			if (ClonedNode)
			{
				for (UEdGraphPin* const ClonedPin : ClonedNode->Pins)
				{
					if (ClonedPin && (ClonedPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard))
					{
						// copy only type info, so array or ref status is preserved
						ClonedPin->PinType.PinCategory = ResolvedWildcardType.PinCategory;
						ClonedPin->PinType.PinSubCategory = ResolvedWildcardType.PinSubCategory;
						ClonedPin->PinType.PinSubCategoryObject = ResolvedWildcardType.PinSubCategoryObject;
					}
				}
			}
		}
	}
}

bool UK2Node_MacroInstance::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UBlueprint* OtherBlueprint = MacroGraphReference.GetBlueprint();
	const bool bResult = OtherBlueprint && (OtherBlueprint != GetBlueprint());
	if (bResult && OptionalOutput)
	{
		if (UClass* OtherClass = *OtherBlueprint->GeneratedClass)
		{
			OptionalOutput->AddUnique(OtherClass);
		}

		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinType.PinSubCategoryObject.IsValid())
			{
				if (UStruct* Struct = Cast<UStruct>(Pin->PinType.PinSubCategoryObject.Get()))
				{
					OptionalOutput->AddUnique(Struct);
				}
				else
				{
					OptionalOutput->AddUnique(Pin->PinType.PinSubCategoryObject.Get()->GetClass());
				}
			}
		}
	}
	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_MacroInstance::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	FString MacroName( TEXT( "InvalidMacro" ));

	if( UEdGraph* MacroGraph = GetMacroGraph() )
	{
		MacroName = MacroGraph->GetName();
	}

	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "Macro" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), MacroName ));
}

FText UK2Node_MacroInstance::GetMenuCategory() const
{
	FText MenuCategory = FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Utilities);
	if (UEdGraph* MacroGraph = GetMacroGraph())
	{
		FKismetUserDeclaredFunctionMetadata* MacroGraphMetadata = UK2Node_MacroInstance::GetAssociatedGraphMetadata(MacroGraph);
		if ((MacroGraphMetadata != nullptr) && !MacroGraphMetadata->Category.IsEmpty())
		{
			MenuCategory = MacroGraphMetadata->Category;
		}
	}

	return MenuCategory;
}

FBlueprintNodeSignature UK2Node_MacroInstance::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();
	NodeSignature.AddSubObject(GetMacroGraph());

	return NodeSignature;
}

void UK2Node_MacroInstance::InferWildcards()
{
	// we've got a new user provided pin, expand the macro 
	UEdGraph* MacroGraph = GetMacroGraph();
	if (MacroGraph)
	{
		// perform macro expansion in a dummy graph, inferring whatever types we can from the provided wildcards:
		FCompilerResultsLog MessageLog;
		UBlueprint* BP = GetBlueprint();
		UEdGraph* ClonedGraph = FEdGraphUtilities::CloneGraph(MacroGraph, BP, &MessageLog, true);
		if (ClonedGraph)
		{
			InferWildcards(ClonedGraph->Nodes);

			// Uncomment to record this graph as an intermediate product - useful for debugging
			//ClonedGraph->Schema = UEdGraphSchema_K2::StaticClass();
			//GetBlueprint()->IntermediateGeneratedGraphs.Add(ClonedGraph);
			//ClonedGraph->SetFlags(RF_Transient);
		}
	}
}

TArray<UEdGraphPin*> UK2Node_MacroInstance::GetAllWildcardPins() const
{
	UEdGraph* MacroGraph = GetMacroGraph();
	if (MacroGraph == nullptr)
	{
		return TArray<UEdGraphPin*>();
	}

	TArray<UEdGraphPin*> Result;
	for(UEdGraphNode* Node : MacroGraph->Nodes)
	{
		if (UK2Node_Tunnel* Tunnel = ExactCast<UK2Node_Tunnel>(Node))
		{
			for(UEdGraphPin* TunnelPin : Tunnel->Pins)
			{
				if(FWildcardNodeUtils::IsWildcardPin(TunnelPin))
				{
					Result.Add(FindPin(TunnelPin->GetName(), UEdGraphPin::GetComplementaryDirection(TunnelPin->Direction)));
				}
			}
		}
	}

	return Result;
}

namespace UE::Private
{

// Recursively infers a type for a network of wildcard pins - i.e. infers type for your linkedto's linkedto's linketo's....
static void InferLinkedPinsImpl(UEdGraphPin* Pin, const FEdGraphPinType& Type, TArray<TPair<UEdGraphNode*, UEdGraphPin*>>& OutDirtyNodePins, TSet<UEdGraphPin*>& ProcessedPins)
{
	FWildcardNodeUtils::InferType(Pin, Type);
	OutDirtyNodePins.AddUnique({Pin->GetOwningNode(), Pin});
	ProcessedPins.Add(Pin);

	for(UEdGraphPin* LinkedPin : Pin->LinkedTo)
	{
		if(!ProcessedPins.Contains(LinkedPin) && FWildcardNodeUtils::IsWildcardPin(LinkedPin))
		{
			InferLinkedPinsImpl(LinkedPin, Type, OutDirtyNodePins, ProcessedPins);
		}
	}
}

}

void UK2Node_MacroInstance::SmartInferWildcardsImpl(const TArray<UEdGraphNode*>& InNodes) const
{
	// Gather wild card pins on the tunnel pins:
	TArray<UEdGraphPin*> TunnelWildcards;
	for (UEdGraphNode* Node : InNodes)
	{
		if (UK2Node_Tunnel* Tunnel = ExactCast<UK2Node_Tunnel>(Node))
		{
			for (UEdGraphPin* TunnelPin : Tunnel->Pins)
			{
				if (FWildcardNodeUtils::HasAnyWildcards(TunnelPin))
				{
					// the tunnel node with input pins is the output on the macro:
					TunnelWildcards.Add(TunnelPin);
				}
			}
		}
	}

	// no wildcards to infer, bail:
	if(TunnelWildcards.Num() == 0)
	{
		return;
	}
			
	// Seed any tunnel wildcard pins that have known values on the macro instance:
	TArray<TPair<UEdGraphNode*, UEdGraphPin*>> DirtyNodePins; // when a pin is inferred we want to give the node a chance to propagate
	for (UEdGraphPin* TunnelPin : TunnelWildcards)
	{
		UEdGraphPin* MacroPin = FindPin(TunnelPin->GetName(), UEdGraphPin::GetComplementaryDirection(TunnelPin->Direction));
		if (ensure(MacroPin))
		{
			if (!FWildcardNodeUtils::IsWildcardPin(MacroPin))
			{
				// tunnel is wildcard, but we are not .. we want to set type on the tunnel and allow inference to run
				FEdGraphPinType const& LinkedPinType = MacroPin->PinType;
				FWildcardNodeUtils::InferType(TunnelPin, LinkedPinType);

				for(UEdGraphPin* LinkedPin : TunnelPin->LinkedTo)
				{
					if (FWildcardNodeUtils::HasAnyWildcards(LinkedPin))
					{
						DirtyNodePins.AddUnique({LinkedPin->GetOwningNode(), LinkedPin});
					}
				}
			}
		}
	}

	// Helper to count the number of wildcard pins on a node
	// we monitor these counts to detect when notifications need
	// to be sent to owning nodes:
	const auto CountWildcardPins = [](const UEdGraphNode* Node)
	{
		int32 WildcardCount = 0;
		for(UEdGraphPin* Pin : Node->Pins)
		{
			if(FWildcardNodeUtils::HasAnyWildcards(Pin))
			{
				++WildcardCount;
			}
		}
		return WildcardCount;
	};

	// helper for counting the number of connections a node has,
	// used to validate that we aren't modifying graph topology
	const auto CountConnections = [](const UEdGraphNode* Node)
	{
		int32 ConnectionCount = 0;
		for(UEdGraphPin* Pin : Node->Pins)
		{
			ConnectionCount += Pin->LinkedTo.Num();
		}
		return ConnectionCount;
	};
	
	TMap<UEdGraphNode*, int32> WildcardCounts;
	TMap<UEdGraphNode*, int32> ConnectionCounts;
	for(UEdGraphNode* Node : InNodes)
	{
		const int32 WildcardCount = CountWildcardPins(Node);
		if(WildcardCount > 0)
		{
			WildcardCounts.Add(Node, WildcardCount);
		}
		ConnectionCounts.Add(Node, CountConnections(Node));
	}

	// We've seeded, now iteratively refresh nodes until pins stabilize:
	while(DirtyNodePins.Num())
	{
		TArray<TPair<UEdGraphNode*, UEdGraphPin*>> CurrentDirtyNodePins = MoveTemp(DirtyNodePins);
		TArray<UEdGraphNode*> NodesToForceInference;
		for(const TPair<UEdGraphNode*, UEdGraphPin*>& DirtyNodePin : CurrentDirtyNodePins)
		{
			// Any wildcard pins that are connected to this pin need to be
			// inferred and marked dirty:
			UK2Node* TypedNode = CastChecked<UK2Node>(DirtyNodePin.Key);
			TypedNode->NotifyPinConnectionListChanged(DirtyNodePin.Value);
			const int32 WildcardCount = CountWildcardPins(TypedNode);
			if (WildcardCount != 0 && WildcardCount == WildcardCounts[TypedNode])
			{
				NodesToForceInference.AddUnique(TypedNode); 
			}
		}

		const auto InferLinkedPins = [](UEdGraphPin * Pin, const UEdGraphPin * SourcePin, TArray<TPair<UEdGraphNode*, UEdGraphPin*>>&OutDirtyNodePins)
		{
			TSet<UEdGraphPin*> ProcessedPins;
			UE::Private::InferLinkedPinsImpl(Pin, SourcePin->PinType, OutDirtyNodePins, ProcessedPins);
		};
		for(UEdGraphNode* Node : NodesToForceInference)
		{
			// force inference, the node is refusing to propagate based on connections.
			// We can directly infer wildcard connections from their connected non wildcard
			// pins:
			for(UEdGraphPin* Pin : Node->Pins)
			{
				if(FWildcardNodeUtils::HasAnyWildcards(Pin))
				{
					for(UEdGraphPin* LinkedPin : Pin->LinkedTo)
					{
						if (!FWildcardNodeUtils::HasAnyWildcards(LinkedPin) )
						{
							// infer and mark dirty:
							InferLinkedPins(Pin, LinkedPin, DirtyNodePins);
							break;
						}
					}
				}
			}
		}

		// look for pins that are now inferable:
		for(const TPair<UEdGraphNode*, int32>& NodeWithCount : WildcardCounts)
		{
			// if count has changed the node is dirty:
			UEdGraphNode* WildcardNode = NodeWithCount.Key;
			const int32 WildcardCount = CountWildcardPins(WildcardNode);
			if(NodeWithCount.Value != WildcardCount)
			{
				for(UEdGraphPin* Pin : WildcardNode->Pins)
				{
					for(UEdGraphPin* LinkedPin : Pin->LinkedTo)
					{
						if(FWildcardNodeUtils::HasAnyWildcards(LinkedPin))
						{
							// mark dirty:
							DirtyNodePins.AddUnique({LinkedPin->GetOwningNode(), LinkedPin});
						}
					}
				}
			}

			// we must also update the count:
			WildcardCounts[NodeWithCount.Key] = WildcardCount;
		}
	}

	for(const TPair<UEdGraphNode*, int32>& NodeWithCount : ConnectionCounts)
	{
		// This ensure indicates that we have a node that is destroying the graph in its 
		// NotifyPinConnectionListChanged override - note that this is an imperfect test
		// but it seems to be cheap and will catch the most egregious errors
		UEdGraph* MacroGraph = GetMacroGraph();
		ensureMsgf(NodeWithCount.Value == CountConnections(NodeWithCount.Key), 
			TEXT("Node connection count changed while inferring %s - consider setting [Blueprints] bUseSimpleWildcardInference as a workaround"),
			MacroGraph ? *MacroGraph->GetPathName() : TEXT("Unknown Graph")
		);
	}

	// copy back inferred values to any pins on our macro instance:
	for (const UEdGraphPin* TunnelWildcard : TunnelWildcards)
	{
		UEdGraphPin* SourcePin = FindPin(
			*TunnelWildcard->PinName.ToString(), 
			UEdGraphPin::GetComplementaryDirection(TunnelWildcard->Direction));
		if(FWildcardNodeUtils::HasAnyWildcards(SourcePin))
		{
			FWildcardNodeUtils::InferType(SourcePin, TunnelWildcard->PinType);
		}
	}
}

#undef LOCTEXT_NAMESPACE
