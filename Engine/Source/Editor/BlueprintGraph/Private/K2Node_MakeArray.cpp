// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_MakeArray.h"

#include "BlueprintCompiledStatement.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeArray)

class FKismetCompilerContext;
struct FLinearColor;

namespace MakeArrayLiterals
{
	static const FName OutputPinName(TEXT("Array"));
};

#define LOCTEXT_NAMESPACE "MakeArrayNode"

/////////////////////////////////////////////////////
// FKCHandler_MakeArray
class FKCHandler_MakeArray : public FKCHandler_MakeContainer
{
public:
	FKCHandler_MakeArray(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_MakeContainer(InCompilerContext)
	{
		CompiledStatementType = KCST_CreateArray;
	}
};

/////////////////////////////////////////////////////
// UK2Node_MakeArray

UK2Node_MakeArray::UK2Node_MakeArray(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ContainerType = EPinContainerType::Array;
}

FNodeHandlingFunctor* UK2Node_MakeArray::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MakeArray(CompilerContext);
}

FText UK2Node_MakeArray::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Make Array");
}

FName UK2Node_MakeArray::GetOutputPinName() const
{
	return MakeArrayLiterals::OutputPinName;
}

FText UK2Node_MakeArray::GetTooltipText() const
{
	return LOCTEXT("MakeArrayTooltip", "Create an array from a series of items.");
}

FSlateIcon UK2Node_MakeArray::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeArray_16x");
	return Icon;
}

void UK2Node_MakeArray::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("K2NodeMakeArray", NSLOCTEXT("K2Nodes", "MakeArrayHeader", "MakeArray"));

		if (Context->Pin != NULL)
		{
			if (Context->Pin->Direction == EGPD_Input && Context->Pin->ParentPin == nullptr)
			{
				Section.AddMenuEntry(
					"RemovePin",
					LOCTEXT("RemovePin", "Remove array element pin"),
					LOCTEXT("RemovePinTooltip", "Remove this array element pin"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(const_cast<UK2Node_MakeArray*>(this), &UK2Node_MakeArray::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
					)
				);
			}
		}
		else
		{
			Section.AddMenuEntry(
				"AddPin",
				LOCTEXT("AddPin", "Add array element pin"),
				LOCTEXT("AddPinTooltip", "Add another array element pin"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_MakeArray*>(this), &UK2Node_MakeArray::InteractiveAddInputPin)
				)
			);
		}

		Section.AddMenuEntry(
			"ResetToWildcard",
			LOCTEXT("ResetToWildcard", "Reset to wildcard"),
			LOCTEXT("ResetToWildcardTooltip", "Reset the node to have wildcard input/outputs. Requires no pins are connected."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_MakeArray*>(this), &UK2Node_MakeArray::ClearPinTypeToWildcard),
				FCanExecuteAction::CreateUObject(this, &UK2Node_MakeArray::CanResetToWildcard)
			)
		);
	}
}

void UK2Node_MakeArray::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	UEdGraphPin* OutputPin = GetOutputPin();
	if (!ensure(Schema) || !ensure(OutputPin) || Schema->IsExecPin(*OutputPin))
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "MakeArray_OutputIsExec", "Unacceptable array type in @@").ToString(), this);
	}
}

FText UK2Node_MakeArray::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("ActionMenuCategory", "Array")), this);
	}
	return CachedCategory;
}

#undef LOCTEXT_NAMESPACE
