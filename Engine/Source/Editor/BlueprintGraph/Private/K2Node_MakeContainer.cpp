// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_MakeContainer.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeContainer)

#define LOCTEXT_NAMESPACE "MakeArrayNode"

void FKCHandler_MakeContainer::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
 	UK2Node_MakeContainer* ContainerNode = CastChecked<UK2Node_MakeContainer>(Node);
 	UEdGraphPin* OutputPin = ContainerNode->GetOutputPin();

	FNodeHandlingFunctor::RegisterNets(Context, Node);

	// Create a local term to drop the container into
	FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(OutputPin, Context.NetNameMap->MakeValidName(OutputPin));
	Term->bPassedByReference = false;
	Term->Source = Node;
 	Context.NetMap.Add(OutputPin, Term);
}

void FKCHandler_MakeContainer::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	TArray<FBPTerminal*> RHSTerms;

	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (Pin && Pin->Direction == EGPD_Input)
		{
			FBPTerminal** InputTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(Pin));
			if (InputTerm)
			{
				FBPTerminal* RHSTerm = *InputTerm;

				{
					using namespace UE::KismetCompiler;

					FBPTerminal* ImplicitCastEntry =
						CastingUtils::InsertImplicitCastStatement(Context, Pin, RHSTerm);

					if (ImplicitCastEntry)
					{
						RHSTerm = ImplicitCastEntry;
					}
				}
				RHSTerms.Add(RHSTerm);
			}
		}
	}

	UK2Node_MakeContainer* ContainerNode = CastChecked<UK2Node_MakeContainer>(Node);
	UEdGraphPin* OutputPin = ContainerNode->GetOutputPin();

	FBPTerminal** ContainerTerm = Context.NetMap.Find(OutputPin);
	check(ContainerTerm);

	FBlueprintCompiledStatement& CreateContainerStatement = Context.AppendStatementForNode(Node);
	CreateContainerStatement.Type = CompiledStatementType;
	CreateContainerStatement.LHS = *ContainerTerm;
	CreateContainerStatement.RHS = MoveTemp(RHSTerms);
}

/////////////////////////////////////////////////////
// UK2Node_MakeContainer

UK2Node_MakeContainer::UK2Node_MakeContainer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NumInputs = 1;
}

UEdGraphPin* UK2Node_MakeContainer::GetOutputPin() const
{
	return FindPin(GetOutputPinName());
}

void UK2Node_MakeContainer::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// This is necessary to retain type information after pasting or loading from disc
	if (UEdGraphPin* OutputPin = GetOutputPin())
	{
		// Only update the output pin if it is currently a wildcard
		if (OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			// Find the matching Old Pin if it exists
			for (UEdGraphPin* OldPin : OldPins)
			{
				if (OldPin->Direction == EGPD_Output)
				{
					// Update our output pin with the old type information and then propagate it to our input pins
					OutputPin->PinType = OldPin->PinType;
					PropagatePinType();
					break;
				}
			}
		}
	}
}

void UK2Node_MakeContainer::AllocateDefaultPins()
{
	// Create the output pin
	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.ContainerType = ContainerType;
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, GetOutputPinName(), PinParams);

	// Create the input pins to create the container from
	for (int32 i = 0; i < NumInputs; ++i)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, GetPinName(i));
	}
}

void UK2Node_MakeContainer::GetKeyAndValuePins(TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const
{
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (CurrentPin->Direction == EGPD_Input && CurrentPin->ParentPin == nullptr)
		{
			KeyPins.Add(CurrentPin);
		}
	}
}

bool UK2Node_MakeContainer::CanResetToWildcard() const
{
	bool bClearPinsToWildcard = true;

	// Check to see if we want to clear the wildcards.
	for (const UEdGraphPin* Pin : Pins)
	{
		if( Pin->LinkedTo.Num() > 0 )
		{
			// One of the pins is still linked, we will not be clearing the types.
			bClearPinsToWildcard = false;
			break;
		}
	}

	return bClearPinsToWildcard;
}

void UK2Node_MakeContainer::ClearPinTypeToWildcard()
{
	if (CanResetToWildcard())
	{
		UEdGraphPin* OutputPin = GetOutputPin();
		OutputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
		OutputPin->PinType.PinSubCategory = NAME_None;
		OutputPin->PinType.PinSubCategoryObject = nullptr;

		if (ContainerType == EPinContainerType::Map)
		{
			OutputPin->PinType.PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;
			OutputPin->PinType.PinValueType.TerminalSubCategory = NAME_None;
			OutputPin->PinType.PinValueType.TerminalSubCategoryObject = nullptr;
		}

		PropagatePinType();
	}
}

void UK2Node_MakeContainer::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	// Array to cache the input pins we might want to find these if we are removing the last link
	TArray<UEdGraphPin*> KeyPins;
	TArray<UEdGraphPin*> ValuePins;
	GetKeyAndValuePins(KeyPins, ValuePins);

	auto CountLinkedPins = [](const TArray<UEdGraphPin*> PinsToCount)
	{
		int32 LinkedPins = 0;
		for (UEdGraphPin* CurrentPin : PinsToCount)
		{
			if (CurrentPin->LinkedTo.Num() > 0)
			{
				++LinkedPins;
			}
		}
		return LinkedPins;
	};

	// Was this the first or last connection?
	const int32 NumKeyPinsWithLinks = CountLinkedPins(KeyPins);
	const int32 NumValuePinsWithLinks = CountLinkedPins(ValuePins);

	UEdGraphPin* OutputPin = GetOutputPin();

	bool bNotifyGraphChanged = false;
	if (Pin->LinkedTo.Num() > 0)
	{
		if (Pin->ParentPin == nullptr)
		{
			if (Pin == OutputPin)
			{
				if (NumKeyPinsWithLinks == 0 && (OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard && Pin->LinkedTo[0]->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard))
				{
					FEdGraphTerminalType TerminalType = MoveTemp(OutputPin->PinType.PinValueType);
					OutputPin->PinType = Pin->LinkedTo[0]->PinType;
					OutputPin->PinType.PinValueType = MoveTemp(TerminalType);
					OutputPin->PinType.ContainerType = ContainerType;
					bNotifyGraphChanged = true;
				}
				if (ContainerType == EPinContainerType::Map && NumValuePinsWithLinks == 0 && (OutputPin->PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Wildcard && Pin->LinkedTo[0]->PinType.PinValueType.TerminalCategory != UEdGraphSchema_K2::PC_Wildcard))
				{
					OutputPin->PinType.PinValueType = Pin->LinkedTo[0]->PinType.PinValueType;
					bNotifyGraphChanged = true;
				}
			}
			else if (ValuePins.Contains(Pin))
			{
				// Just made a connection to a value pin, was it the first?
				if (NumValuePinsWithLinks == 1 && (OutputPin->PinType.PinValueType.TerminalCategory == UEdGraphSchema_K2::PC_Wildcard && Pin->LinkedTo[0]->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard))
				{
					// Update the types on all the pins
					OutputPin->PinType.PinValueType = FEdGraphTerminalType::FromPinType(Pin->LinkedTo[0]->PinType);
					bNotifyGraphChanged = true;
				}
			}
			else 
			{
				// Just made a connection to a key pin, was it the first?
				if (NumKeyPinsWithLinks == 1 && (OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard && Pin->LinkedTo[0]->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard))
				{
					FEdGraphTerminalType TerminalType = MoveTemp(OutputPin->PinType.PinValueType);
					OutputPin->PinType = Pin->LinkedTo[0]->PinType;
					OutputPin->PinType.PinValueType = MoveTemp(TerminalType);
					OutputPin->PinType.ContainerType = ContainerType;
					bNotifyGraphChanged = true;
				}
			}
		}
	}
	else if (OutputPin->LinkedTo.Num() == 0)
	{
		// Return to wildcard if theres nothing in any of the input pins
		TFunction<bool(TArray<UEdGraphPin*>&)> PinsInUse = [this, &PinsInUse](TArray<UEdGraphPin*>& PinsToConsider)
		{
			bool bPinInUse = false;
			for (UEdGraphPin* CurrentPin : PinsToConsider)
			{
				// Is there something in this pin?
				if (CurrentPin->SubPins.Num() > 0 || !CurrentPin->DoesDefaultValueMatchAutogenerated())
				{
					bPinInUse = true;
					break;
				}
			}
			return bPinInUse;
		};

		const bool bResetOutputPinPrimary = ((NumKeyPinsWithLinks == 0) && !PinsInUse(KeyPins));
		const bool bResetOutputPinSecondary = ((NumValuePinsWithLinks == 0) && !PinsInUse(ValuePins));

		if (bResetOutputPinPrimary)
		{
			OutputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			OutputPin->PinType.PinSubCategory = NAME_None;
			OutputPin->PinType.PinSubCategoryObject = nullptr;

			bNotifyGraphChanged = true;
		}
		if (bResetOutputPinSecondary && ContainerType == EPinContainerType::Map)
		{
			OutputPin->PinType.PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;
			OutputPin->PinType.PinValueType.TerminalSubCategory = NAME_None;
			OutputPin->PinType.PinValueType.TerminalSubCategoryObject = nullptr;

			bNotifyGraphChanged = true;
		}
	}

	if (bNotifyGraphChanged)
	{
		PropagatePinType();
		GetGraph()->NotifyNodeChanged(this);
	}
}

void UK2Node_MakeContainer::PropagatePinType()
{
	const UEdGraphPin* OutputPin = GetOutputPin();

	if (OutputPin)
	{
		UClass const* CallingContext = nullptr;
		if (UBlueprint const* Blueprint = GetBlueprint())
		{
			CallingContext = Blueprint->GeneratedClass;
			if (CallingContext == nullptr)
			{
				CallingContext = Blueprint->ParentClass;
			}
		}

		TArray<UEdGraphPin*> KeyPins;
		TArray<UEdGraphPin*> ValuePins;
		GetKeyAndValuePins(KeyPins, ValuePins);

		// Propagate pin type info (except for array info!) to pins with dependent types
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		auto PropagateToPin = [Schema](UEdGraphPin* CurrentPin, const FEdGraphPinType& PinType)
		{
			// if we've reset to wild card or the parent pin no longer matches we need to collapse the split pin(s)
			// otherwise everything should be OK:
			if (CurrentPin->SubPins.Num() != 0 &&
				(CurrentPin->PinType.PinCategory != PinType.PinCategory ||
				 CurrentPin->PinType.PinSubCategory != PinType.PinSubCategory ||
				 CurrentPin->PinType.PinSubCategoryObject != PinType.PinSubCategoryObject)
				)
			{
				Schema->RecombinePin(CurrentPin->SubPins[0]);
			}

			CurrentPin->PinType.PinCategory = PinType.PinCategory;
			CurrentPin->PinType.PinSubCategory = PinType.PinSubCategory;
			CurrentPin->PinType.PinSubCategoryObject = PinType.PinSubCategoryObject;
		};

		for (UEdGraphPin* CurrentKeyPin : KeyPins)
		{
			PropagateToPin(CurrentKeyPin, OutputPin->PinType);
		}

		if (ValuePins.Num() > 0)
		{
			const FEdGraphPinType ValuePinType = FEdGraphPinType::GetPinTypeForTerminalType(OutputPin->PinType.PinValueType);
			for (UEdGraphPin* CurrentValuePin : ValuePins)
			{
				PropagateToPin(CurrentValuePin, ValuePinType);
			}
		}

		for (UEdGraphPin* CurrentPin : Pins)
		{
			if (CurrentPin && CurrentPin != OutputPin)
			{
				if (CurrentPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard || CurrentPin->GetDefaultAsString().IsEmpty() == true)
				{
					// Only reset default value if there isn't one set or it is a wildcard. Otherwise this deletes data!
					Schema->SetPinAutogeneratedDefaultValueBasedOnType(CurrentPin);
				}

				// Verify that all previous connections to this pin are still valid with the new type
				TArray<UEdGraphPin*> LinkedToCopy = CurrentPin->LinkedTo;
				for (UEdGraphPin* ConnectedPin : LinkedToCopy)
				{
					if (!Schema->ArePinsCompatible(CurrentPin, ConnectedPin, CallingContext))
					{
						CurrentPin->BreakLinkTo(ConnectedPin);
					}
				}
			}
		}
		// If we have a valid graph we should refresh it now to refelect any changes we made
		if (UEdGraphNode* OwningNode = OutputPin->GetOwningNode())
		{
			if (UEdGraph* Graph = OwningNode->GetGraph())
			{
				Graph->NotifyNodeChanged(this);
			}
		}
	}
}

void UK2Node_MakeContainer::PostReconstructNode()
{
	// Find a pin that has connections to use to jumpstart the wildcard process
	FEdGraphPinType OutputPinType;
	FEdGraphTerminalType OutputPinValueType;

	const bool bMapContainer = ContainerType == EPinContainerType::Map;
	bool bFoundKey = false;
	bool bFoundValue = !bMapContainer;

	UEdGraphPin* OutputPin = GetOutputPin();
	if (OutputPin->LinkedTo.Num() > 0)
	{
		OutputPinType = OutputPin->LinkedTo[0]->PinType;
		bFoundKey = true;

		if (bMapContainer)
		{
			OutputPinValueType = OutputPin->LinkedTo[0]->PinType.PinValueType;
			bFoundValue = true;
		}
	}
	else
	{
		bool bKeyPin = !bMapContainer;
		UEdGraphPin* CurrentTopParent = nullptr;

		check(Pins[0] == OutputPin);
		for (int32 PinIndex = 1; PinIndex < Pins.Num() && (!bFoundKey || !bFoundValue); ++PinIndex)
		{
			if (UEdGraphPin* CurrentPin = Pins[PinIndex])
			{
				if (CurrentPin->ParentPin == nullptr)
				{
					CurrentTopParent = CurrentPin;
					if (bMapContainer)
					{
						bKeyPin = !bKeyPin;
					}
				}

				if ((bKeyPin && !bFoundKey) || (!bKeyPin && !bFoundValue))
				{
					checkSlow((CurrentPin->ParentPin == nullptr) || (CurrentTopParent != nullptr));
					if (CurrentPin->LinkedTo.Num() > 0)
					{
						// The pin is linked, use its type as the type for the key or value type as appropriate

						// If this is a split pin, so we want to base the pin type on the parent rather than the linked to pin
						const FEdGraphPinType& PinType = (CurrentPin->ParentPin ? CurrentTopParent->PinType : CurrentPin->LinkedTo[0]->PinType);
						if (bKeyPin)
						{
							OutputPinType = PinType;
							bFoundKey = true;
						}
						else
						{
							OutputPinValueType = FEdGraphTerminalType::FromPinType(PinType);
							bFoundValue = true;
						}
					}
					else if (!Pins[PinIndex]->DoesDefaultValueMatchAutogenerated())
					{
						// The pin has user data in it, continue to use its type as the type for all pins.

						// If this is a split pin, so we want to base the pin type on the parent rather than this pin
						const FEdGraphPinType& PinType = (CurrentPin->ParentPin ? CurrentTopParent->PinType : CurrentPin->PinType);

						if (bKeyPin)
						{
							OutputPinType = PinType;
							bFoundKey = true;
						}
						else
						{
							OutputPinValueType = FEdGraphTerminalType::FromPinType(PinType);
							bFoundValue = true;
						}
					}
				}
			}
		}
	}

	if (bFoundKey)
	{
		OutputPin->PinType = MoveTemp(OutputPinType);
	}
	else
	{
		OutputPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
		OutputPin->PinType.PinSubCategory = NAME_None;
		OutputPin->PinType.PinSubCategoryObject = nullptr;
	}

	if (bMapContainer)
	{
		if (bFoundValue)
		{
			OutputPin->PinType.PinValueType = MoveTemp(OutputPinValueType);
		}
		else
		{
			OutputPin->PinType.PinValueType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;
			OutputPin->PinType.PinValueType.TerminalSubCategory = NAME_None;
			OutputPin->PinType.PinValueType.TerminalSubCategoryObject = nullptr;
		}
	}

	OutputPin->PinType.ContainerType = ContainerType;
	PropagatePinType();

	Super::PostReconstructNode();
}

void UK2Node_MakeContainer::InteractiveAddInputPin()
{
	FScopedTransaction Transaction(LOCTEXT("AddPinTx", "Add Pin"));
	AddInputPin();
}

void UK2Node_MakeContainer::AddInputPin()
{
	Modify();

	++NumInputs;
	const FEdGraphPinType& OutputPinType = GetOutputPin()->PinType;
	UEdGraphPin* Pin = CreatePin(EGPD_Input, OutputPinType.PinCategory, OutputPinType.PinSubCategory, OutputPinType.PinSubCategoryObject.Get(), GetPinName(NumInputs-1));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(Pin);

	const bool bIsCompiling = GetBlueprint()->bBeingCompiled;
	if( !bIsCompiling )
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

FName UK2Node_MakeContainer::GetPinName(const int32 PinIndex) const
{
	return *FString::Printf(TEXT("[%d]"), PinIndex);
}

void UK2Node_MakeContainer::SyncPinNames()
{
	int32 CurrentNumParentPins = 0;
	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin*& CurrentPin = Pins[PinIndex];
		if (CurrentPin->Direction == EGPD_Input &&
			CurrentPin->ParentPin == nullptr)
		{
			const FName OldName = CurrentPin->PinName;
			const FName ElementName = GetPinName(CurrentNumParentPins++);

			CurrentPin->Modify();
			CurrentPin->PinName = ElementName;

			if (CurrentPin->SubPins.Num() > 0)
			{
				const FString OldNameStr = OldName.ToString();
				const FString ElementNameStr = ElementName.ToString();
				FString OldFriendlyName = OldNameStr;
				FString ElementFriendlyName = ElementNameStr;

				// SubPin Friendly Name has an extra space in it so we need to account for that
				OldFriendlyName.InsertAt(1, " ");
				ElementFriendlyName.InsertAt(1, " ");

				for (UEdGraphPin* SubPin : CurrentPin->SubPins)
				{
					FString SubPinFriendlyName = SubPin->PinFriendlyName.ToString();
					SubPinFriendlyName.ReplaceInline(*OldFriendlyName, *ElementFriendlyName);

					SubPin->Modify();
					SubPin->PinName = *SubPin->PinName.ToString().Replace(*OldNameStr, *ElementNameStr);
					SubPin->PinFriendlyName = FText::FromString(SubPinFriendlyName);
				}
			}
		}
	}
}

void UK2Node_MakeContainer::RemoveInputPin(UEdGraphPin* Pin)
{
	check(Pin->Direction == EGPD_Input);
	check(Pin->ParentPin == nullptr);
	checkSlow(Pins.Contains(Pin));

	FScopedTransaction Transaction(LOCTEXT("RemovePinTx", "RemovePin"));
	Modify();
	
	TFunction<void(UEdGraphPin*)> RemovePinLambda = [this, &RemovePinLambda](UEdGraphPin* PinToRemove)
	{
		for (int32 SubPinIndex = PinToRemove->SubPins.Num()-1; SubPinIndex >= 0; --SubPinIndex)
		{
			RemovePinLambda(PinToRemove->SubPins[SubPinIndex]);
		}

		int32 PinRemovalIndex = INDEX_NONE;
		if (Pins.Find(PinToRemove, PinRemovalIndex))
		{
			Pins.RemoveAt(PinRemovalIndex);
			PinToRemove->MarkAsGarbage();
		}
	};

	if (ContainerType == EPinContainerType::Map)
	{
		TArray<UEdGraphPin*> KeyPins;
		TArray<UEdGraphPin*> ValuePins;
		GetKeyAndValuePins(KeyPins, ValuePins);

		int32 PinIndex = INDEX_NONE;
		if (ValuePins.Find(Pin, PinIndex))
		{
			RemovePinLambda(KeyPins[PinIndex]);
		}
		else
		{
			verify(KeyPins.Find(Pin, PinIndex));
			RemovePinLambda(ValuePins[PinIndex]);
		}
	}

	RemovePinLambda(Pin);
	PinConnectionListChanged(Pin);

	--NumInputs;
	SyncPinNames();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
}

void UK2Node_MakeContainer::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

bool UK2Node_MakeContainer::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (!ensure(OtherPin))
	{
		return true;
	}

	// if MyPin has a ParentPin then we are dealing with a split pin and we should evaluate it with default behavior
	if (MyPin->ParentPin == nullptr && OtherPin->PinType.IsContainer() == true && MyPin->Direction == EGPD_Input)
	{
		OutReason = NSLOCTEXT("K2Node", "MakeContainer_InputIsContainer", "Cannot make a container with an input of a container!").ToString();
		return true;
	}

	if (UEdGraphSchema_K2::IsExecPin(*OtherPin))
	{
		OutReason = NSLOCTEXT("K2Node", "MakeContainer_InputIsExec", "Cannot make a container with an execution input!").ToString();
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
