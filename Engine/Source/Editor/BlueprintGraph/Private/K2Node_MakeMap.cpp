// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_MakeMap.h"

#include "BlueprintCompiledStatement.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeMap)

class FKismetCompilerContext;
struct FLinearColor;

namespace MakeMapLiterals
{
	static const FName OutputPinName(TEXT("Map"));
};

#define LOCTEXT_NAMESPACE "MakeMapNode"

/////////////////////////////////////////////////////
// FKCHandler_MakeMap
class FKCHandler_MakeMap : public FKCHandler_MakeContainer
{
public:
	FKCHandler_MakeMap(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_MakeContainer(InCompilerContext)
	{
		CompiledStatementType = KCST_CreateMap;
	}
};

/////////////////////////////////////////////////////
// UK2Node_MakeMap

UK2Node_MakeMap::UK2Node_MakeMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ContainerType = EPinContainerType::Map;
}

FNodeHandlingFunctor* UK2Node_MakeMap::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MakeMap(CompilerContext);
}

void UK2Node_MakeMap::AllocateDefaultPins()
{
	// Create the output pin
	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.ContainerType = ContainerType;
	PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, GetOutputPinName(), PinParams);

	// Create the input pins to create the container from
	for (int32 i = 0; i < NumInputs; ++i)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, GetPinName(i * 2));
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, GetPinName((i * 2) + 1));
	}
}

void UK2Node_MakeMap::GetKeyAndValuePins(TArray<UEdGraphPin*>& KeyPins, TArray<UEdGraphPin*>& ValuePins) const
{
	for (UEdGraphPin* CurrentPin : Pins)
	{
		if (CurrentPin->Direction == EGPD_Input && CurrentPin->ParentPin == nullptr)
		{
			// Key/Value pins alternate so if this is a map and the counts are even then this is a key
			if (KeyPins.Num() == ValuePins.Num())
			{
				KeyPins.Add(CurrentPin);
			}
			else
			{
				ValuePins.Add(CurrentPin);
			}
		}
	}
	check(KeyPins.Num() == ValuePins.Num());
}

void UK2Node_MakeMap::AddInputPin()
{
	Modify();

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	++NumInputs;
	const FEdGraphPinType& OutputPinType = GetOutputPin()->PinType;
	UEdGraphPin* Pin = CreatePin(EGPD_Input, OutputPinType.PinCategory, OutputPinType.PinSubCategory, OutputPinType.PinSubCategoryObject.Get(), GetPinName((NumInputs - 1) * 2));
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);

	const FEdGraphPinType ValuePinType = FEdGraphPinType::GetPinTypeForTerminalType(OutputPinType.PinValueType);
	Pin = CreatePin(EGPD_Input, ValuePinType.PinCategory, ValuePinType.PinSubCategory, ValuePinType.PinSubCategoryObject.Get(), GetPinName(((NumInputs - 1) * 2) + 1));
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);

	const bool bIsCompiling = GetBlueprint()->bBeingCompiled;
	if (!bIsCompiling)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

FText UK2Node_MakeMap::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Make Map");
}

FName UK2Node_MakeMap::GetPinName(const int32 PinIndex) const
{
	const int32 PairIndex = PinIndex / 2;
	if (PinIndex % 2 == 0)
	{
		return *FString::Printf(TEXT("Key %d"), PairIndex);
	}
	else
	{
		return *FString::Printf(TEXT("Value %d"), PairIndex);
	}
}

FName UK2Node_MakeMap::GetOutputPinName() const
{
	return MakeMapLiterals::OutputPinName;
}

FText UK2Node_MakeMap::GetTooltipText() const
{
	return LOCTEXT("MakeMapTooltip", "Create a map from a series of key/value items.");
}

FSlateIcon UK2Node_MakeMap::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeMap_16x");
	return Icon;
}

void UK2Node_MakeMap::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("K2NodeMakeMap", NSLOCTEXT("K2Nodes", "MakeMapHeader", "MakeMap"));

		if (Context->Pin)
		{
			if (Context->Pin->Direction == EGPD_Input && Context->Pin->ParentPin == nullptr)
			{
				Section.AddMenuEntry(
					"RemovePin",
					LOCTEXT("RemovePin", "Remove key/value pair"),
					LOCTEXT("RemovePinTooltip", "Remove this pin and its corresponding key/value pin"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(const_cast<UK2Node_MakeMap*>(this), &UK2Node_MakeMap::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
					)
				);
			}
		}
		else
		{
			Section.AddMenuEntry(
				"AddPin",
				LOCTEXT("AddPin", "Add key/value pair"),
				LOCTEXT("AddPinTooltip", "Add another pair of key/value pins"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_MakeMap*>(this), &UK2Node_MakeMap::InteractiveAddInputPin)
				)
			);
		}

		Section.AddMenuEntry(
			"ResetToWildcard",
			LOCTEXT("ResetToWildcard", "Reset to wildcard"),
			LOCTEXT("ResetToWildcardTooltip", "Reset the node to have wildcard input/outputs. Requires no pins are connected."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_MakeMap*>(this), &UK2Node_MakeMap::ClearPinTypeToWildcard),
				FCanExecuteAction::CreateUObject(this, &UK2Node_MakeMap::CanResetToWildcard)
			)
		);
	}
}

void UK2Node_MakeMap::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	UEdGraphPin* OutputPin = GetOutputPin();
	if (!ensure(Schema) || !ensure(OutputPin) || Schema->IsExecPin(*OutputPin))
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "MakeMap_OutputIsExec", "Unacceptable map type in @@").ToString(), this);
	}
}

FText UK2Node_MakeMap::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("ActionMenuCategory", "Map")), this);
	}
	return CachedCategory;
}

#undef LOCTEXT_NAMESPACE
