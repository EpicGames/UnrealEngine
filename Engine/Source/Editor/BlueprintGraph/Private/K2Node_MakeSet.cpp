// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_MakeSet.h"

#include "BlueprintCompiledStatement.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeSet)

class FKismetCompilerContext;
struct FLinearColor;

namespace MakeSetLiterals
{
	static const FName OutputPinName(TEXT("Set"));
};

#define LOCTEXT_NAMESPACE "MakeSetNode"

/////////////////////////////////////////////////////
// FKCHandler_MakeSet
class FKCHandler_MakeSet : public FKCHandler_MakeContainer
{
public:
	FKCHandler_MakeSet(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_MakeContainer(InCompilerContext)
	{
		CompiledStatementType = KCST_CreateSet;
	}
};

/////////////////////////////////////////////////////
// UK2Node_MakeSet

UK2Node_MakeSet::UK2Node_MakeSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ContainerType = EPinContainerType::Set;
}

FNodeHandlingFunctor* UK2Node_MakeSet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MakeSet(CompilerContext);
}

FText UK2Node_MakeSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Make Set");
}

FName UK2Node_MakeSet::GetOutputPinName() const
{
	return MakeSetLiterals::OutputPinName;
}

FText UK2Node_MakeSet::GetTooltipText() const
{
	return LOCTEXT("MakeSetTooltip", "Create a set from a series of items.");
}

FSlateIcon UK2Node_MakeSet::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeSet_16x");
	return Icon;
}

void UK2Node_MakeSet::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (!Context->bIsDebugging)
	{
		FToolMenuSection& Section = Menu->AddSection("K2NodeMakeSet", NSLOCTEXT("K2Nodes", "MakeSetHeader", "MakeSet"));

		if (Context->Pin)
		{
			if (Context->Pin->Direction == EGPD_Input && Context->Pin->ParentPin == nullptr)
			{
				Section.AddMenuEntry(
					"RemovePin",
					LOCTEXT("RemovePin", "Remove set element pin"),
					LOCTEXT("RemovePinTooltip", "Remove this set element pin"),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateUObject(const_cast<UK2Node_MakeSet*>(this), &UK2Node_MakeSet::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
					)
				);
			}
		}
		else
		{
			Section.AddMenuEntry(
				"AddPin",
				LOCTEXT("AddPin", "Add set element pin"),
				LOCTEXT("AddPinTooltip", "Add another set element pin"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_MakeSet*>(this), &UK2Node_MakeSet::InteractiveAddInputPin)
				)
			);
		}

		Section.AddMenuEntry(
			"ResetToWildcard",
			LOCTEXT("ResetToWildcard", "Reset to wildcard"),
			LOCTEXT("ResetToWildcardTooltip", "Reset the node to have wildcard input/outputs. Requires no pins are connected."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_MakeSet*>(this), &UK2Node_MakeSet::ClearPinTypeToWildcard),
				FCanExecuteAction::CreateUObject(this, &UK2Node_MakeSet::CanResetToWildcard)
			)
		);
	}
}

void UK2Node_MakeSet::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UEdGraphSchema_K2* Schema = Cast<const UEdGraphSchema_K2>(GetSchema());
	UEdGraphPin* OutputPin = GetOutputPin();
	if (!ensure(Schema) || !ensure(OutputPin) || Schema->IsExecPin(*OutputPin))
	{
		MessageLog.Error(*NSLOCTEXT("K2Node", "MakeSet_OutputIsExec", "Unacceptable set type in @@").ToString(), this);
	}
}

FText UK2Node_MakeSet::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::Utilities, LOCTEXT("ActionMenuCategory", "Set")), this);
	}
	return CachedCategory;
}

#undef LOCTEXT_NAMESPACE
