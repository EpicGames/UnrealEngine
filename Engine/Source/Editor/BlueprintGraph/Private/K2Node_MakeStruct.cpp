// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MakeStruct.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompilerMisc.h"
#include "MakeStructHandler.h"
#include "Math/Rotator.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "PropertyCustomizationHelpers.h"
#include "Serialization/Archive.h"
#include "Styling/AppStyle.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeStruct)

class FKismetCompilerContext;

#define LOCTEXT_NAMESPACE "K2Node_MakeStruct"

//////////////////////////////////////////////////////////////////////////
// UK2Node_MakeStruct


UK2Node_MakeStruct::FMakeStructPinManager::FMakeStructPinManager(const uint8* InSampleStructMemory, UBlueprint* InOwningBP)
	: FStructOperationOptionalPinManager()
	, SampleStructMemory(InSampleStructMemory)
	, OwningBP(InOwningBP)
	, bHasAdvancedPins(false)
{
}

void UK2Node_MakeStruct::FMakeStructPinManager::GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
{
	UK2Node_StructOperation::FStructOperationOptionalPinManager::GetRecordDefaults(TestProperty, Record);
	Record.bIsMarkedForAdvancedDisplay = TestProperty ? TestProperty->HasAnyPropertyFlags(CPF_AdvancedDisplay) : false;
	bHasAdvancedPins |= Record.bIsMarkedForAdvancedDisplay;
}

void UK2Node_MakeStruct::FMakeStructPinManager::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const
{
	UK2Node_StructOperation::FStructOperationOptionalPinManager::CustomizePinData(Pin, SourcePropertyName, ArrayIndex, Property);
	if (Pin && Property)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		check(Schema);

		// Should pin default value be filled as FText?
		const bool bIsText = Property->IsA<FTextProperty>();
		checkSlow(bIsText == ((UEdGraphSchema_K2::PC_Text == Pin->PinType.PinCategory) && !Pin->PinType.IsContainer()));

		const bool bIsObject = Property->IsA<FObjectPropertyBase>();
		checkSlow(bIsObject == ((UEdGraphSchema_K2::PC_Object == Pin->PinType.PinCategory || UEdGraphSchema_K2::PC_Class == Pin->PinType.PinCategory || 
			UEdGraphSchema_K2::PC_SoftObject == Pin->PinType.PinCategory || UEdGraphSchema_K2::PC_SoftClass == Pin->PinType.PinCategory) && !Pin->PinType.IsContainer()));

		if (Property->HasAnyPropertyFlags(CPF_AdvancedDisplay))
		{
			Pin->bAdvancedView = true;
			bHasAdvancedPins = true;
		}

		const FString& MetadataDefaultValue = Property->GetMetaData(TEXT("MakeStructureDefaultValue"));
		if (!MetadataDefaultValue.IsEmpty())
		{
			Schema->SetPinAutogeneratedDefaultValue(Pin, MetadataDefaultValue);
			return;
		}

		if (nullptr != SampleStructMemory)
		{
			FString NewDefaultValue;
			if (FBlueprintEditorUtils::PropertyValueToString(Property, SampleStructMemory, NewDefaultValue))
			{
				if (Schema->IsPinDefaultValid(Pin, NewDefaultValue, nullptr, FText::GetEmpty()).IsEmpty())
				{
					Schema->SetPinAutogeneratedDefaultValue(Pin, NewDefaultValue);
					return;
				}
			}
		}

		Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
	}
}

static bool CanBeExposed(const FProperty* Property, UBlueprint* BP)
{
	if (Property)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		check(Schema);

		// Treat all inline edit condition properties as override flags; that is, don't allow
		// these to be exposed as part of the optional input pin set. Their value will be set
		// implicitly at runtime based on whether or not any bound members are exposed, rather
		// than explicitly via an exposed input pin. This emulates how the Property Editor
		// handles setting these values at edit time (they appear as an inline checkbox that
		// the user ticks on to set the flag and enable/override a bound property's value).
		static const FName MD_InlineEditConditionToggle(TEXT("InlineEditConditionToggle"));
		if (Property->HasMetaData(MD_InlineEditConditionToggle))
		{
			return false;
		}

		const bool bIsEditorBP = IsEditorOnlyObject(BP);
		const bool bIsEditAnywhereProperty = Property->HasAllPropertyFlags(CPF_Edit) &&
			!Property->HasAnyPropertyFlags(CPF_EditConst);

		if (!Property->HasAllPropertyFlags(CPF_BlueprintReadOnly) || 
			(bIsEditorBP && bIsEditAnywhereProperty) )
		{
			if (Property->HasAllPropertyFlags(CPF_BlueprintVisible) && !(Property->ArrayDim > 1))
			{
				FEdGraphPinType DumbGraphPinType;
				if (Schema->ConvertPropertyToPinType(Property, /*out*/ DumbGraphPinType))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool UK2Node_MakeStruct::FMakeStructPinManager::CanTreatPropertyAsOptional(FProperty* TestProperty) const
{
	return CanBeExposed(TestProperty, OwningBP);
}

UK2Node_MakeStruct::UK2Node_MakeStruct(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bMadeAfterOverridePinRemoval(false)
{
}

void UK2Node_MakeStruct::AllocateDefaultPins()
{
	if (StructType)
	{
		PreloadObject(StructType);
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, StructType, StructType->GetFName());
		
		bool bHasAdvancedPins = false;
		{
			FStructOnScope StructOnScope(StructType);
			FMakeStructPinManager OptionalPinManager(StructOnScope.GetStructMemory(), GetBlueprint());
			OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructType);
			OptionalPinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Input, this);

			bHasAdvancedPins = OptionalPinManager.HasAdvancedPins();
		}

		// Set container pin types to have their default values ignored, which will in turn
		// enable auto generation for any that are not set by the user. 
		for(UEdGraphPin* Pin : Pins)
		{
			Pin->bDefaultValueIsIgnored = Pin->bDefaultValueIsIgnored || Pin->PinType.IsContainer();
		}

		// When struct has a lot of fields, mark their pins as advanced
		if(!bHasAdvancedPins && Pins.Num() > 5)
		{
			for(int32 PinIndex = 3; PinIndex < Pins.Num(); ++PinIndex)
			{
				if(UEdGraphPin * EdGraphPin = Pins[PinIndex])
				{
					EdGraphPin->bAdvancedView = true;
					bHasAdvancedPins = true;
				}
			}
		}

		if (bHasAdvancedPins && (ENodeAdvancedPins::NoPins == AdvancedPinDisplay))
		{
			AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
		}
	}
}

void UK2Node_MakeStruct::PreloadRequiredAssets()
{
	PreloadObject(StructType);
	Super::PreloadRequiredAssets();
}

void UK2Node_MakeStruct::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if(!StructType)
	{
		MessageLog.Error(*LOCTEXT("NoStruct_Error", "No Struct in @@").ToString(), this);
	}
	else
	{
		UBlueprint* BP = GetBlueprint();
		for (TFieldIterator<FProperty> It(StructType); It; ++It)
		{
			const FProperty* Property = *It;
			if (CanBeExposed(Property, BP))
			{
				if (Property->ArrayDim > 1)
				{
					const UEdGraphPin* Pin = FindPin(Property->GetFName());
					MessageLog.Warning(*LOCTEXT("StaticArray_Warning", "@@ - the native property is a static array, which is not supported by blueprints").ToString(), Pin);
				}
			}
		}

		if (!bMadeAfterOverridePinRemoval)
		{
			MessageLog.Note(*NSLOCTEXT("K2Node", "OverridePinRemoval_SetFieldsInStruct", "Override pins have been removed from @@ in @@, it functions the same as it did but some functionality may be deprecated! This note will go away after you resave the asset!").ToString(), this, GetBlueprint());
		}
	}
}


FText UK2Node_MakeStruct::GetNodeTitle(ENodeTitleType::Type TitleType) const 
{
	if (StructType == nullptr)
	{
		return LOCTEXT("MakeNullStructTitle", "Make <unknown struct>");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("StructName"), StructType->GetDisplayNameText());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("MakeNodeTitle", "Make {StructName}"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_MakeStruct::GetTooltipText() const
{
	if (StructType == nullptr)
	{
		return LOCTEXT("MakeNullStruct_Tooltip", "Adds a node that create an '<unknown struct>' from its members");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(
			LOCTEXT("MakeStruct_Tooltip", "Adds a node that create a '{0}' from its members"),
			StructType->GetDisplayNameText()
		), this);
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_MakeStruct::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeStruct_16x");
	return Icon;
}

FLinearColor UK2Node_MakeStruct::GetNodeTitleColor() const
{
	if(const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>())
	{
		FEdGraphPinType PinType;
		PinType.PinCategory = UEdGraphSchema_K2::PC_Struct;
		PinType.PinSubCategoryObject = StructType;
		return K2Schema->GetPinTypeColor(PinType);
	}
	return UK2Node::GetNodeTitleColor();
}

bool UK2Node_MakeStruct::CanBeMade(const UScriptStruct* Struct, const bool bForInternalUse)
{
	return (Struct && !Struct->HasMetaData(FBlueprintMetadata::MD_NativeMakeFunction) && UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Struct, bForInternalUse));
}

bool UK2Node_MakeStruct::CanBeSplit(const UScriptStruct* Struct, UBlueprint* InBP)
{
	if (CanBeMade(Struct))
	{
		for (TFieldIterator<FProperty> It(Struct); It; ++It)
		{
			if (CanBeExposed(*It, InBP))
			{
				return true;
			}
		}
	}
	return false;
}

FNodeHandlingFunctor* UK2Node_MakeStruct::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MakeStruct(CompilerContext);
}

UK2Node::ERedirectType UK2Node_MakeStruct::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	ERedirectType Result = UK2Node::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if ((ERedirectType_None == Result) && DoRenamedPinsMatch(NewPin, OldPin, false))
	{
		Result = ERedirectType_Name;
	}
	return Result;
}

void UK2Node_MakeStruct::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	Super::SetupMenuActions(ActionRegistrar, FMakeStructSpawnerAllowedDelegate::CreateStatic(&UK2Node_MakeStruct::CanBeMade), EGPD_Input);
}

FText UK2Node_MakeStruct::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Struct);
}

void UK2Node_MakeStruct::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
	if (Blueprint && !Blueprint->bBeingCompiled)
	{
		bMadeAfterOverridePinRemoval = true;
	}
}

void UK2Node_MakeStruct::PostPlacedNewNode()
{
	Super::PostPlacedNewNode();

	// New nodes automatically have this set.
	bMadeAfterOverridePinRemoval = true;
}

void UK2Node_MakeStruct::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading() && !Ar.IsTransacting() && !HasAllFlags(RF_Transient))
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this);
		if (Blueprint && !bMadeAfterOverridePinRemoval)
		{
			// Check if this node actually requires warning the user that functionality has changed.
			bMadeAfterOverridePinRemoval = true;
			if (StructType != nullptr)
			{
				FOptionalPinManager PinManager;

				// Have to check if this node is even in danger.
				for (FOptionalPinFromProperty& PropertyEntry : ShowPinForProperties)
				{
					FProperty* Property = StructType->FindPropertyByName(PropertyEntry.PropertyName);
					bool bNegate = false;
					if (FProperty* OverrideProperty = PropertyCustomizationHelpers::GetEditConditionProperty(Property, bNegate))
					{
						bool bHadOverridePropertySeparation = false;
						for (FOptionalPinFromProperty& OverridePropertyEntry : ShowPinForProperties)
						{
							if (OverridePropertyEntry.PropertyName == OverrideProperty->GetFName())
							{
								bHadOverridePropertySeparation = true;
								break;
							}
						}

						bMadeAfterOverridePinRemoval = false;
						UEdGraphPin* Pin = FindPin(Property->GetFName());

						if (bHadOverridePropertySeparation)
						{
							UEdGraphPin* OverridePin = FindPin(OverrideProperty->GetFName());
							if (OverridePin)
							{
								// Override pins are always booleans
								check(OverridePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
								// If the old override pin's default value was true, then the override should be marked as enabled
								PropertyEntry.bIsOverrideEnabled = OverridePin->DefaultValue.ToBool();
								// It had an override pin, so conceptually the override pin is visible
								PropertyEntry.bIsOverridePinVisible = true;
								// Because there was an override pin visible for this property, this property will be forced to have a pin
								PropertyEntry.bShowPin = true;
							}
							else
							{
								// No override pin, ensure all override bools are false
								PropertyEntry.bIsOverrideEnabled = false;
								PropertyEntry.bIsOverridePinVisible = false;
							}
						}
						else
						{
							if (Pin)
							{
								PropertyEntry.bIsOverrideEnabled = true;
								PropertyEntry.bIsOverridePinVisible = true;
							}
						}

						// If the pin for this property, which sets the property's value, does not exist then the user was not trying to set the value
						PropertyEntry.bIsSetValuePinVisible = Pin != nullptr;
					}
				}
			}
		}
	}
}

void UK2Node_MakeStruct::ConvertDeprecatedNode(UEdGraph* Graph, bool bOnlySafeChanges)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// User may have since deleted the struct type
	if (StructType == nullptr)
	{
		return;
	}

	// Check to see if the struct has a native make/break that we should try to convert to.
	if (StructType->HasMetaData(FBlueprintMetadata::MD_NativeMakeFunction))
	{
		UFunction* MakeNodeFunction = nullptr;

		// If any pins need to change their names during the conversion, add them to the map.
		TMap<FName, FName> OldPinToNewPinMap;

		if (StructType == TBaseStructure<FRotator>::Get())
		{
			MakeNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, MakeRotator));
			OldPinToNewPinMap.Add(TEXT("Rotator"), UEdGraphSchema_K2::PN_ReturnValue);
		}
		else if (StructType == TBaseStructure<FVector>::Get())
		{
			MakeNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED_ThreeParams(UKismetMathLibrary, MakeVector, double, double, double));
			OldPinToNewPinMap.Add(TEXT("Vector"), UEdGraphSchema_K2::PN_ReturnValue);
		}
		else if (StructType == TBaseStructure<FVector2D>::Get())
		{
			MakeNodeFunction = UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED_TwoParams(UKismetMathLibrary, MakeVector2D, double, double));
			OldPinToNewPinMap.Add(TEXT("Vector2D"), UEdGraphSchema_K2::PN_ReturnValue);
		}
		else
		{
			const FString& MetaData = StructType->GetMetaData(FBlueprintMetadata::MD_NativeMakeFunction);
			MakeNodeFunction = FindObject<UFunction>(nullptr, *MetaData, EFindObjectFlags::ExactClass);

			if (MakeNodeFunction)
			{
				OldPinToNewPinMap.Add(*StructType->GetName(), UEdGraphSchema_K2::PN_ReturnValue);
			}
		}

		if (MakeNodeFunction)
		{
			Schema->ConvertDeprecatedNodeToFunctionCall(this, MakeNodeFunction, OldPinToNewPinMap, Graph);
		}
	}
}


#undef LOCTEXT_NAMESPACE
