// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MakeVariable.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeVariable)

static const TCHAR* MakeVariableOutputPinName = TEXT("MakeVariableOutput");

/////////////////////////////////////////////////////
// FKCHandler_MakeVariable
class FKCHandler_MakeVariable : public FNodeHandlingFunctor
{
public:
	FKCHandler_MakeVariable(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
 		UK2Node_MakeVariable* MakeVariableNode = CastChecked<UK2Node_MakeVariable>(Node);
 		UEdGraphPin* OutputPin = MakeVariableNode->GetOutputPin();

		FNodeHandlingFunctor::RegisterNets(Context, Node);

		// Create a local term to drop the variable into
		FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(OutputPin, Context.NetNameMap->MakeValidName(OutputPin));
		Term->bPassedByReference = false;
		Term->Source = Node;
 		Context.NetMap.Add(OutputPin, Term);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_MakeVariable* MakeVariableNode = CastChecked<UK2Node_MakeVariable>(Node);
		UEdGraphPin* OutputPin = MakeVariableNode->GetOutputPin();

		FBPTerminal** VariableTerm = Context.NetMap.Find(OutputPin);
		check(VariableTerm);

		FBlueprintCompiledStatement& CreateVariableStatement = Context.AppendStatementForNode(Node);
		CreateVariableStatement.LHS = *VariableTerm;

		// This node only supports containers at the moment, it could be extended to support any type:
		const FBPVariableDescription& VariableType = MakeVariableNode->GetVariableType();
		if(VariableType.VarType.IsArray())
		{
			CreateVariableStatement.Type = KCST_CreateArray;
		}
		else if(VariableType.VarType.IsSet())
		{
			CreateVariableStatement.Type = KCST_CreateSet;
		}
		else if( VariableType.VarType.IsMap())
		{
			CreateVariableStatement.Type = KCST_CreateMap;
		}

		for(auto PinIt = Node->Pins.CreateIterator(); PinIt; ++PinIt)
		{
			UEdGraphPin* Pin = *PinIt;
			if(Pin && Pin->Direction == EGPD_Input)
			{
				FBPTerminal** InputTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(Pin));
				if( InputTerm )
				{
					CreateVariableStatement.RHS.Add(*InputTerm);
				}
			}
		}
	}
};

/////////////////////////////////////////////////////
// UK2Node_MakeVariable

UK2Node_MakeVariable::UK2Node_MakeVariable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_MakeVariable::SetupVariable(const FBPVariableDescription& InVariableType, UEdGraphPin* TargetInputPin, FKismetCompilerContext& CompilerContext, UFunction* Scope, const FProperty* Property )
{
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	VariableType = InVariableType;
	const FEdGraphPinType ContainedType = FEdGraphPinType::GetTerminalTypeForContainer(InVariableType.VarType);

	// make an output pin that will reference the provided variable type:
	UEdGraphPin* VariableOutputPin = CreatePin(EGPD_Output, InVariableType.VarType, MakeVariableOutputPinName);

	// make input pins for every value in the default value, for map pins we'll make a pair of inputs:
	TSharedPtr<FStructOnScope> StructData = MakeShareable(new FStructOnScope(Scope));
	FBlueprintEditorUtils::PropertyValueFromString(Property, VariableType.DefaultValue, StructData->GetStructMemory(), this);

	if(VariableType.VarType.IsArray())
	{
		const FArrayProperty* ArrayProperty = CastFieldChecked<const FArrayProperty>(Property);
		FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(StructData->GetStructMemory()));
		
		// Go through each element in the array to set the default value
		for( int32 ArrayIndex = 0 ; ArrayIndex < ArrayHelper.Num() ; ArrayIndex++ )
		{
			const uint8* PropData = ArrayHelper.GetRawPtr(ArrayIndex);

			// Retrieve the element's default value
			FString DefaultValue;
			FBlueprintEditorUtils::PropertyValueToString_Direct(ArrayProperty->Inner, PropData, DefaultValue);
			
			UEdGraphPin* VariableInputPin = CreatePin(EGPD_Input, ContainedType, *FString::FromInt(ArrayIndex));
			// Add one to the index for the pin to set the default on to skip the output pin
			Schema->SetPinAutogeneratedDefaultValue(VariableInputPin, DefaultValue);
		}
	}
	else if( VariableType.VarType.IsSet())
	{
		const FSetProperty* SetProperty = CastFieldChecked<const FSetProperty>(Property);
		FScriptSetHelper SetHelper(SetProperty, SetProperty->ContainerPtrToValuePtr<void>(StructData->GetStructMemory()));

		// Go through each element in the set to set the default value
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			const uint8* PropData = SetHelper.GetElementPtr(It);

			// Retrieve the element's default value
			FString DefaultValue;
			FBlueprintEditorUtils::PropertyValueToString_Direct(SetProperty->ElementProp, PropData, DefaultValue);
			UEdGraphPin* VariableInputPin = CreatePin(EGPD_Input, ContainedType, *FString::FromInt(It.GetLogicalIndex()));
			// Add one to the index for the pin to set the default on to skip the output pin
			Schema->SetPinAutogeneratedDefaultValue(VariableInputPin, DefaultValue);
		}
	}
	else if( VariableType.VarType.IsMap())
	{
		const FMapProperty* MapProperty = CastFieldChecked<const FMapProperty>(Property);
		FScriptMapHelper MapHelper(MapProperty, MapProperty->ContainerPtrToValuePtr<void>(StructData->GetStructMemory()));
		
		const FEdGraphPinType ValueType = FEdGraphPinType::GetPinTypeForTerminalType(InVariableType.VarType.PinValueType);

		// Go through each element in the map to set the default value
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			const uint8* KeyData = MapHelper.GetKeyPtr(It);

			// Retrieve the element's default value
			FString KeyDefaultValue;
			FBlueprintEditorUtils::PropertyValueToString_Direct(MapProperty->KeyProp, KeyData, KeyDefaultValue);

			UEdGraphPin* VariableInputPin = CreatePin(EGPD_Input, ContainedType, *FString::FromInt(It.GetLogicalIndex()).Append(TEXT("_Key")));
			// Add one to the index for the pin to set the default on to skip the output pin
			Schema->SetPinAutogeneratedDefaultValue(VariableInputPin, KeyDefaultValue);

			const uint8* ValueData = MapHelper.GetValuePtr(It);

			FString ValueDefaultValue;
			FBlueprintEditorUtils::PropertyValueToString_Direct(MapProperty->ValueProp, ValueData, ValueDefaultValue);

			VariableInputPin = CreatePin(EGPD_Input, ValueType, *FString::FromInt(It.GetLogicalIndex()).Append(TEXT("_Value")));
			// Add one to the index for the pin to set the default on to skip the output pin
			Schema->SetPinAutogeneratedDefaultValue(VariableInputPin, ValueDefaultValue);
		}
	}

	// connect it to the target inputpin:
	if(!Schema->TryCreateConnection(VariableOutputPin, TargetInputPin))
	{
		CompilerContext.MessageLog.Error(TEXT("Blueprint Internal Compiler Error: Could not connect default value to input pin @@"), TargetInputPin);
	}
}

FNodeHandlingFunctor* UK2Node_MakeVariable::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MakeVariable(CompilerContext);
}

UEdGraphPin* UK2Node_MakeVariable::GetOutputPin() const
{
	return FindPinChecked(MakeVariableOutputPinName);
}
