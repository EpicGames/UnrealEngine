// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MapForEach.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_TemporaryVariable.h"
#include "KismetCompiler.h"
#include "Kismet/BlueprintMapLibrary.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/WildcardNodeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MapForEach)

#define LOCTEXT_NAMESPACE "K2Node_MapForEach"

const FName UK2Node_MapForEach::MapPinName(TEXT("MapPin"));
const FName UK2Node_MapForEach::BreakPinName(TEXT("BreakPin"));
const FName UK2Node_MapForEach::KeyPinName(TEXT("KeyPin"));
const FName UK2Node_MapForEach::ValuePinName(TEXT("ValuePin"));
const FName UK2Node_MapForEach::CompletedPinName(TEXT("CompletedPin"));

UK2Node_MapForEach::UK2Node_MapForEach()
{
	KeyName = LOCTEXT("KeyPin_FriendlyName", "Map Key").ToString();
	ValueName = LOCTEXT("ValuePin_FriendlyName", "Map Value").ToString();
}

UEdGraphPin* UK2Node_MapForEach::GetMapPin() const
{
	return FindPinChecked(MapPinName);
}

UEdGraphPin* UK2Node_MapForEach::GetBreakPin() const
{
	return FindPinChecked(BreakPinName);
}

UEdGraphPin* UK2Node_MapForEach::GetForEachPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UK2Node_MapForEach::GetKeyPin() const
{
	return FindPinChecked(KeyPinName);
}

UEdGraphPin* UK2Node_MapForEach::GetValuePin() const
{
	return FindPinChecked(ValuePinName);
}

UEdGraphPin* UK2Node_MapForEach::GetCompletedPin() const
{
	return FindPinChecked(CompletedPinName);
}

void UK2Node_MapForEach::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Actions are registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed).
	// Here, we use the node's class to keep from needlessly instantiating a UBlueprintNodeSpawner.
	// Additionally, if the node type disappears, then the action should go with it.

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(ActionKey);
		check(NodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_MapForEach::GetMenuCategory() const
{
	return LOCTEXT("NodeMenu", "Utilities|Map");
}

void UK2Node_MapForEach::PostReconstructNode()
{
	Super::PostReconstructNode();

	RefreshWildcardPins();
}

void UK2Node_MapForEach::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.ContainerType = EPinContainerType::Map;
	PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;

	UEdGraphPin* MapPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, MapPinName, PinParams);
	MapPin->PinType.bIsConst = true;
	MapPin->PinType.bIsReference = true;
	MapPin->PinFriendlyName = LOCTEXT("MapPin_FriendlyName", "Map");

	UEdGraphPin* BreakPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, BreakPinName);
	BreakPin->PinFriendlyName = LOCTEXT("BreakPin_FriendlyName", "Break");
	BreakPin->bAdvancedView = true;

	UEdGraphPin* ForEachPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	ForEachPin->PinFriendlyName = LOCTEXT("ForEachPin_FriendlyName", "Loop Body");

	UEdGraphPin* KeyPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, KeyPinName);
	KeyPin->PinFriendlyName = FText::FromString(KeyName);

	UEdGraphPin* ValuePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, ValuePinName);
	ValuePin->PinFriendlyName = FText::FromString(ValueName);

	UEdGraphPin* CompletedPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, CompletedPinName);
	CompletedPin->PinFriendlyName = LOCTEXT("CompletedPin_FriendlyName", "Completed");
	CompletedPin->PinToolTip = LOCTEXT("CompletedPin_Tooltip", "Execution once all map elements have been visited").ToString();

	if (AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
}

void UK2Node_MapForEach::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (CheckForErrors(CompilerContext))
	{
		// Remove all the links to this node as they are no longer needed
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	///////////////////////////////////////////////////////////////////////////////////
	// Cache off versions of all our important pins

	UEdGraphPin* ForEach_Exec = GetExecPin();
	UEdGraphPin* ForEach_Map = GetMapPin();
	UEdGraphPin* ForEach_Break = GetBreakPin();
	UEdGraphPin* ForEach_ForEach = GetForEachPin();
	UEdGraphPin* ForEach_Key = GetKeyPin();
	UEdGraphPin* ForEach_Value = GetValuePin();
	UEdGraphPin* ForEach_Completed = GetCompletedPin();

	///////////////////////////////////////////////////////////////////////////////////
	// Create a loop counter variable

	UK2Node_TemporaryVariable* CreateTemporaryVariable = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	CreateTemporaryVariable->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
	CreateTemporaryVariable->AllocateDefaultPins();

	UEdGraphPin* Temp_Variable = CreateTemporaryVariable->GetVariablePin();

	///////////////////////////////////////////////////////////////////////////////////
	// Initialize the temporary to 0

	UK2Node_AssignmentStatement* InitTemporaryVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	InitTemporaryVariable->AllocateDefaultPins();

	UEdGraphPin* Init_Exec = InitTemporaryVariable->GetExecPin();
	UEdGraphPin* Init_Variable = InitTemporaryVariable->GetVariablePin();
	UEdGraphPin* Init_Value = InitTemporaryVariable->GetValuePin();
	UEdGraphPin* Init_Then = InitTemporaryVariable->GetThenPin();

	CompilerContext.MovePinLinksToIntermediate(*ForEach_Exec, *Init_Exec);
	K2Schema->TryCreateConnection(Init_Variable, Temp_Variable);
	Init_Value->DefaultValue = TEXT("0");

	///////////////////////////////////////////////////////////////////////////////////
	// Branch on comparing the loop index with the size of the map

	UK2Node_IfThenElse* BranchOnIndex = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	BranchOnIndex->AllocateDefaultPins();

	UEdGraphPin* Branch_Exec = BranchOnIndex->GetExecPin();
	UEdGraphPin* Branch_Input = BranchOnIndex->GetConditionPin();
	UEdGraphPin* Branch_Then = BranchOnIndex->GetThenPin();
	UEdGraphPin* Branch_Else = BranchOnIndex->GetElsePin();

	Init_Then->MakeLinkTo(Branch_Exec);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_Completed, *Branch_Else);

	UK2Node_CallFunction* CompareLessThan = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CompareLessThan->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Less_IntInt), UKismetMathLibrary::StaticClass());
	CompareLessThan->AllocateDefaultPins();

	UEdGraphPin* Compare_A = CompareLessThan->FindPinChecked(TEXT("A"));
	UEdGraphPin* Compare_B = CompareLessThan->FindPinChecked(TEXT("B"));
	UEdGraphPin* Compare_Return = CompareLessThan->GetReturnValuePin();

	Branch_Input->MakeLinkTo(Compare_Return);
	Temp_Variable->MakeLinkTo(Compare_A);

	UK2Node_CallFunction* GetMapLength = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetMapLength->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintMapLibrary, Map_Length), UBlueprintMapLibrary::StaticClass());
	GetMapLength->AllocateDefaultPins();

	UEdGraphPin* MapLength_Map = GetMapLength->FindPinChecked(TEXT("TargetMap"));
	UEdGraphPin* MapLength_Return = GetMapLength->GetReturnValuePin();

	// Coerce the wildcard pin types
	MapLength_Map->PinType = ForEach_Map->PinType;

	Compare_B->MakeLinkTo(MapLength_Return);
	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Map, *MapLength_Map);

	///////////////////////////////////////////////////////////////////////////////////
	// Sequence the loop body and incrementing the loop counter

	UK2Node_ExecutionSequence* LoopSequence = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
	LoopSequence->AllocateDefaultPins();

	UEdGraphPin* Sequence_Exec = LoopSequence->GetExecPin();
	UEdGraphPin* Sequence_One = LoopSequence->GetThenPinGivenIndex(0);
	UEdGraphPin* Sequence_Two = LoopSequence->GetThenPinGivenIndex(1);

	Branch_Then->MakeLinkTo(Sequence_Exec);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_ForEach, *Sequence_One);

	UK2Node_CallFunction* GetMapPair = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetMapPair->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintMapLibrary, Map_GetKeyValueByIndex), UBlueprintMapLibrary::StaticClass());
	GetMapPair->AllocateDefaultPins();

	UEdGraphPin* GetPair_Map = GetMapPair->FindPinChecked(TEXT("TargetMap"));
	UEdGraphPin* GetPair_Index = GetMapPair->FindPinChecked(TEXT("Index"));
	UEdGraphPin* GetPair_Key = GetMapPair->FindPinChecked(TEXT("Key"));
	UEdGraphPin* GetPair_Value = GetMapPair->FindPinChecked(TEXT("Value"));

	// Coerce the wildcard pin types
	GetPair_Map->PinType = ForEach_Map->PinType;
	GetPair_Key->PinType = ForEach_Key->PinType;
	GetPair_Value->PinType = ForEach_Value->PinType;

	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Map, *GetPair_Map);
	GetPair_Index->MakeLinkTo(Temp_Variable);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_Key, *GetPair_Key);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_Value, *GetPair_Value);

	///////////////////////////////////////////////////////////////////////////////////
	// Increment the loop counter by one

	UK2Node_AssignmentStatement* IncrementVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	IncrementVariable->AllocateDefaultPins();

	UEdGraphPin* Inc_Exec = IncrementVariable->GetExecPin();
	UEdGraphPin* Inc_Variable = IncrementVariable->GetVariablePin();
	UEdGraphPin* Inc_Value = IncrementVariable->GetValuePin();
	UEdGraphPin* Inc_Then = IncrementVariable->GetThenPin();

	Sequence_Two->MakeLinkTo(Inc_Exec);
	Branch_Exec->MakeLinkTo(Inc_Then);

	K2Schema->TryCreateConnection(Temp_Variable, Inc_Variable);

	UK2Node_CallFunction* AddOne = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	AddOne->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Add_IntInt), UKismetMathLibrary::StaticClass());
	AddOne->AllocateDefaultPins();

	UEdGraphPin* Add_A = AddOne->FindPinChecked(TEXT("A"));
	UEdGraphPin* Add_B = AddOne->FindPinChecked(TEXT("B"));
	UEdGraphPin* Add_Return = AddOne->GetReturnValuePin();

	Temp_Variable->MakeLinkTo(Add_A);
	Add_B->DefaultValue = TEXT("1");
	Add_Return->MakeLinkTo(Inc_Value);

	///////////////////////////////////////////////////////////////////////////////////
	// Create a sequence from the break exec that will set the loop counter to the last array index.
	// The loop will then increment the counter and terminate on the next run of SequenceTwo.

	UK2Node_AssignmentStatement* SetVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	SetVariable->AllocateDefaultPins();

	UEdGraphPin* Set_Exec = SetVariable->GetExecPin();
	UEdGraphPin* Set_Variable = SetVariable->GetVariablePin();
	UEdGraphPin* Set_Value = SetVariable->GetValuePin();

	CompilerContext.MovePinLinksToIntermediate(*ForEach_Break, *Set_Exec);
	K2Schema->TryCreateConnection(Temp_Variable, Set_Variable);

	UK2Node_CallFunction* GetMapLastIndex = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetMapLastIndex->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintMapLibrary, Map_GetLastIndex), UBlueprintMapLibrary::StaticClass());
	GetMapLastIndex->AllocateDefaultPins();

	UEdGraphPin* GetIndex_Map = GetMapLastIndex->FindPinChecked(TEXT("TargetMap"));
	UEdGraphPin* GetIndex_Return = GetMapLastIndex->GetReturnValuePin();

	// Coerce the wildcard pin types
	GetIndex_Map->PinType = ForEach_Map->PinType;
	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Map, *GetIndex_Map);

	GetIndex_Return->MakeLinkTo(Set_Value);

	BreakAllNodeLinks();
}

FText UK2Node_MapForEach::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "For Each Loop (Map)");
}

FText UK2Node_MapForEach::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Loop over each element of a map");
}

FSlateIcon UK2Node_MapForEach::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon("EditorStyle", "GraphEditor.Macro.ForEach_16x");
}

void UK2Node_MapForEach::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	check(Pin);

	if (Pin->PinName == MapPinName)
	{
		RefreshWildcardPins();
	}
}

void UK2Node_MapForEach::PostEditChangeProperty(FPropertyChangedEvent &PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRefresh = false;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UK2Node_MapForEach, KeyName))
	{
		GetKeyPin()->PinFriendlyName = FText::FromString(KeyName);
		bRefresh = true;

	}
	else if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UK2Node_MapForEach, ValueName))
	{
		GetValuePin()->PinFriendlyName = FText::FromString(ValueName);
		bRefresh = true;
	}

	if (bRefresh)
	{
		// Poke the graph to update the visuals based on the above changes
		GetGraph()->NotifyGraphChanged();
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	}
}

bool UK2Node_MapForEach::CheckForErrors(const FKismetCompilerContext& CompilerContext)
{
	bool bError = false;

	if (GetMapPin()->LinkedTo.IsEmpty())
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingMap_Error", "For Each (Map) node @@ must have a Map to iterate.").ToString(), this);
		bError = true;
	}

	return bError;
}

void UK2Node_MapForEach::RefreshWildcardPins()
{
	UEdGraphPin* MapPin = GetMapPin();
	check(MapPin);

	UEdGraphPin* KeyPin = GetKeyPin();
	check(KeyPin);

	UEdGraphPin* ValuePin = GetValuePin();
	check(ValuePin);

	const bool bIsWildcardPin = FWildcardNodeUtils::IsWildcardPin(MapPin);
	if (bIsWildcardPin && MapPin->LinkedTo.Num() > 0)
	{
		if (const UEdGraphPin* InferrablePin = FWildcardNodeUtils::FindInferrableLinkedPin(MapPin))
		{
			FWildcardNodeUtils::InferType(MapPin, InferrablePin->PinType);

			// In some contexts, the key/value pins may not be in a wildcard state (eg: paste operation).
			// We'll just leave the pins with their current type and let the compiler catch any issues (if any).
			// This also helps ensure that we don't jostle any pins that are in a split state.
			if (FWildcardNodeUtils::IsWildcardPin(KeyPin))
			{
				FWildcardNodeUtils::InferType(KeyPin, InferrablePin->PinType);
			}
			
			if (FWildcardNodeUtils::IsWildcardPin(ValuePin))
			{
				FWildcardNodeUtils::InferType(ValuePin->PinType, InferrablePin->PinType.PinValueType);
			}
		}
	}

	// If no pins are connected, then we need to reset the dependent pins back to the original wildcard state.
	if (MapPin->LinkedTo.Num() == 0)
	{
		FWildcardNodeUtils::ResetToWildcard(MapPin);
		FWildcardNodeUtils::ResetToWildcard(KeyPin);
		FWildcardNodeUtils::ResetToWildcard(ValuePin);
	}
}

#undef LOCTEXT_NAMESPACE
