// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MathExpression.h"

#include "BasicTokenParser.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EditablePinBase.h"
#include "K2Node_Tunnel.h"
#include "K2Node_VariableGet.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "MathExpressionHandler.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/DefaultValueHelper.h"
#include "Misc/Guid.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/Stack.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MathExpression)

class FExpressionParser;
class UObject;

#define LOCTEXT_NAMESPACE "K2Node"

/*******************************************************************************
 * Static Helpers
*******************************************************************************/

/**
 * Helper function for deleting all the nodes from a specified graph. Does not
 * delete any tunnel in/out nodes (to preserve the tunnel).
 * 
 * @param  Graph	The graph you want to delete nodes in.
 */
static void DeleteGeneratedNodesInGraph(UEdGraph* Graph)
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraphChecked(Graph);
	for (int32 NodeIndex = 0; NodeIndex < Graph->Nodes.Num(); )
	{
		UEdGraphNode* Node = Graph->Nodes[NodeIndex];
		if (ExactCast<UK2Node_Tunnel>(Node) != NULL)
		{
			++NodeIndex;
		}
		else
		{
			FBlueprintEditorUtils::RemoveNode(Blueprint, Node, true);
		}
	}
}

/**
 * If the specified type is a "byte" type, then this will modify it to 
 * an "int". Helps when trying to match function signatures.
 * 
 * @param  InOutType	The type you want to attempt to promote.
 * @return True if the type was modified, false if not.
 */
static bool PromoteByteToInt(FEdGraphPinType& InOutType)
{
	if (InOutType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		InOutType.PinCategory = UEdGraphSchema_K2::PC_Int;
		InOutType.PinSubCategoryObject = nullptr;
		return true;
	}
	return false;
}

/**
* If the specified type is a "int" type, then this will modify it to
* a "double". Helps when trying to match function signatures.
*
* @param  InOutType	The type you want to attempt to promote.
* @return True if the type was modified, false if not.
*/
static bool PromoteIntToDouble(FEdGraphPinType& InOutType)
{
	if (InOutType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		InOutType.PinCategory = UEdGraphSchema_K2::PC_Real;
		InOutType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
		InOutType.PinSubCategoryObject = nullptr;
		return true;
	}
	return false;
}

/**
 * Sets or clears the error on a specific node. If the ErrorText is empty, then
 * it resets the error state on the node. If actual error text is supplied, 
 * then the node is flagged as having an error, and the string is appended to 
 * the node's error message.
 * 
 * @param  Node			The node you wish to modify.
 * @param  ErrorText	The text you want to append to the node's error message (empty if you want to clear any errors).
 */
static void SetNodeError(UEdGraphNode* Node, FText const& ErrorText)
{
	if (ErrorText.IsEmpty())
	{
		Node->ErrorMsg.Empty();
		Node->ErrorType = EMessageSeverity::Info;
		Node->bHasCompilerMessage = false;
	}
	else if (Node->bHasCompilerMessage)
	{
		Node->ErrorMsg += TEXT("\n") + ErrorText.ToString();
		Node->ErrorType = EMessageSeverity::Error;
	}
	else
	{
		Node->ErrorMsg = ErrorText.ToString();
		Node->ErrorType = EMessageSeverity::Error;
		Node->bHasCompilerMessage = true;
	}
}

/*******************************************************************************
 * FExpressionVisitor
*******************************************************************************/

/** 
 * This is the base class used for IFExpressionNode tree traversal (set up to 
 * handle different node types... new node types should have a Visit() method 
 * added for them).
 */
class FExpressionVisitor
{
public:
	virtual ~FExpressionVisitor() { }
	
	/** 
	 * FExpressionNodes determine when a traverser (FExpressionVisitor) has access 
	 * to the node. There are a couple hook points, allowing the traverser to pick 
	 * either a pre-order or post-order tree traversal. These values let the 
	 * FExpressionVisitor where we are in the tree search.
	 */
	enum EVisitPhase
	{
		VISIT_Pre,  // the node being visited has yet to visit its children (and will next, starting with the left)
		VISIT_Post, // the node being visited has finished visiting its children (and is about to return up, to its parent)
		VISIT_Leaf  // the node being visited is a leaf (no children will be visited)
	};

	/**
	 * Intended to be overridden by sub-classes, for special handling of 
	 * explicit node types (new node types should have one added for them).
	 * 
	 * @param  Node		The current node in the tree's traversal.
	 * @param  Phase	Where we currently are in traversing the specified node (before children vs. after)
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Visit(class IFExpressionNode&  Node, EVisitPhase Phase)    { return VisitUnhandled(Node, Phase); }
	virtual bool Visit(class FTokenWrapperNode& Node, EVisitPhase Phase)    { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }
	virtual bool Visit(class FBinaryOperator&   Node, EVisitPhase Phase)    { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }
	virtual bool Visit(class FUnaryOperator&    Node, EVisitPhase Phase)    { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }
	virtual bool Visit(class FConditionalOperator& Node, EVisitPhase Phase) { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }
	virtual bool Visit(class FExpressionList&   Node, EVisitPhase Phase)    { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }
	virtual bool Visit(class FFunctionExpression&  Node, EVisitPhase Phase) { return VisitUnhandled((class IFExpressionNode&)Node, Phase); }

protected:
	/**
	 * Called by all the base Visit() methods, a good point for sub-classes to 
	 * hook into for handling EVERY IFExpressionNode type (unless they override
	 * a Visit method)
	 * 
	 * @param  Node		The current node in the tree's traversal.
	 * @param  Phase	Where we currently are in traversing the specified node (before children vs. after)	
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool VisitUnhandled(class IFExpressionNode& Node, EVisitPhase Phase)
	{
		// if we end up here, then the subclass decided not to handle the 
		// specific node type, and therefore doesn't care about it
		return true;
	}
};

/*******************************************************************************
 * IFExpressionNode Types
*******************************************************************************/

/**
 * Base class for all expression-tree nodes that are generated from parsing an
 * expression string. Represents either a single value/variable, or an operation
 * on other IFExpressionNode(s).
 */
class IFExpressionNode
{
public:
	/** 
	 * Entry point for traversing the expression-tree, should either pass the 
	 * Visitor along to sub child nodes (in the case of a branch node), or 
	 * simply let the Visitor "visit" the leaf node.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) = 0;

	/** 
	 * For debug purposes, intended to help visualize what this node represents 
	 * (for reconstructing a pseudo expression).
	 */
	virtual FString ToString() const = 0;

	/**
	 * Variable Guid's are stored in the internal expression and must be 
	 * converted back to their name when showing the expression in the node's title
	 */
	virtual FString ToDisplayString(UBlueprint* InBlueprint) const { return ToString(); };

protected:
	IFExpressionNode() {}
};

/** 
 * Leaf node for the expression-tree. Encapsulates either a literal constant 
 * (FBasicToken::TOKEN_Const), or a variable identifier (FBasicToken::TOKEN_Identifier).
 */
class FTokenWrapperNode : public IFExpressionNode
{
public:
	FTokenWrapperNode(FBasicToken InToken)
		: Token(InToken)
	{
	}

	virtual ~FTokenWrapperNode() {}

	/** 
	 * Gives the FExpressionVisitor access to this node (lets it "visit" this, 
	 * in tree traversal terms).
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Leaf);
	}

	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		if (Token.TokenType == FBasicToken::TOKEN_Identifier || Token.TokenType == FBasicToken::TOKEN_Guid)
		{
			return FString::Printf(TEXT("%s"), Token.Identifier);
		}
		else if (Token.TokenType == FBasicToken::TOKEN_Const)
		{
			return FString::Printf(TEXT("%s"), *Token.GetConstantValue());
		}
		else
		{
			return FString::Printf(TEXT("(UnexpectedTokenType)%s"), Token.Identifier);
		}
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		if (Token.TokenType == FBasicToken::TOKEN_Guid)
		{
			FGuid VariableGuid;
			if(FGuid::Parse(FString(Token.Identifier), VariableGuid))
			{
				FName VariableName = FBlueprintEditorUtils::FindMemberVariableNameByGuid(InBlueprint, VariableGuid);

				if (VariableName.IsNone())
				{
					VariableName = FBlueprintEditorUtils::FindLocalVariableNameByGuid(InBlueprint, VariableGuid);
				}
				return VariableName.ToString();
			}
		}
		return ToString();
	}
public:
	/** The base token which this leaf node represents */
	FBasicToken Token;
};


/**
 * Branch node that represents a binary operation, where its children are the 
 * left and right operands:
 *
 *                 <operator>
 *                 /        \
 * <left-expression>        <right-expression>
 */
class FBinaryOperator : public IFExpressionNode
{
public:
	FBinaryOperator(const FString& InOperator, TSharedRef<IFExpressionNode> InLHS, TSharedRef<IFExpressionNode> InRHS)
		: Operator(InOperator)
		, LHS(InLHS)
		, RHS(InRHS)
	{
	}

	virtual ~FBinaryOperator() {}

	/** 
	 * Gives the FExpressionVisitor access to this node, and passes it along to 
	 * traverse the children.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		bool bAbort = !Visitor.Visit(*this, FExpressionVisitor::VISIT_Pre);
		if (bAbort || !LHS->Accept(Visitor) || !RHS->Accept(Visitor))
		{
			return false;
		}
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Post);
	}

	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		const FString LeftStr = LHS->ToString();
		const FString RightStr = RHS->ToString();
		return FString::Printf(TEXT("(%s %s %s)"), *LeftStr, *Operator, *RightStr);
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		const FString LeftStr = LHS->ToDisplayString(InBlueprint);
		const FString RightStr = RHS->ToDisplayString(InBlueprint);
		return FString::Printf(TEXT("(%s %s %s)"), *LeftStr, *Operator, *RightStr);
	}

public:
	FString Operator;
	TSharedRef<IFExpressionNode> LHS;
	TSharedRef<IFExpressionNode> RHS;
};


/**
 * Branch node that represents a unary (prefix) operation, where its child is  
 * the right operand:
 *
 *     <unary-operator>
 *                    \
 *                    <operand-expression>
 */
class FUnaryOperator : public IFExpressionNode
{
public:
	FUnaryOperator(const FString& InOperator, TSharedRef<IFExpressionNode> InRHS)
		: Operator(InOperator)
		, RHS(InRHS)
	{
	}

	virtual ~FUnaryOperator() {}

	/** 
	 * Gives the FExpressionVisitor access to this node, and passes it along to 
	 * traverse its child.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		bool bAbort = !Visitor.Visit(*this, FExpressionVisitor::VISIT_Pre);
		if (bAbort || !RHS->Accept(Visitor))
		{
			return false;
		}
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Post);
	}

	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		const FString RightStr = RHS->ToString();
		return FString::Printf(TEXT("(%s%s)"), *Operator, *RightStr);
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		const FString RightStr = RHS->ToDisplayString(InBlueprint);
		return FString::Printf(TEXT("(%s%s)"), *Operator, *RightStr);
	}
public:
	FString Operator;
	TSharedRef<IFExpressionNode> RHS;
};


/**
 * Branch node that represents a ternary conditional (if-then-else) operation 
 * (c ? a : b), where its children are the condition, the "then" expression,   
 * and the "else" expression:
 *
 *             <conditional-operator>
 *             /          |         \
 *  <condition>   <then-exression>   <else-expression>
 */
class FConditionalOperator : public IFExpressionNode
{
public:
	FConditionalOperator(TSharedRef<IFExpressionNode> InCondition, TSharedRef<IFExpressionNode> InTruePart, TSharedRef<IFExpressionNode> InFalsePart)
		: Condition(InCondition)
		, TruePart(InTruePart)
		, FalsePart(InFalsePart)
	{
	}

	virtual ~FConditionalOperator() {}

	/** 
	 * Gives the FExpressionVisitor access to this node, and passes it along to 
	 * traverse the children.
	 *
	 * @return True to continue traversing the tree, false to abort.  
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		bool bAbort = !Visitor.Visit(*this, FExpressionVisitor::VISIT_Pre);
		// @TODO: what about the Condition?
		if (bAbort || !TruePart->Accept(Visitor) || !FalsePart->Accept(Visitor))
		{
			return false;
		}
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Post);
	}

	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		const FString ConditionStr = Condition->ToString();
		const FString TrueStr = TruePart->ToString();
		const FString FalseStr = FalsePart->ToString();
		return FString::Printf(TEXT("(%s ? %s : %s)"), *ConditionStr, *TrueStr, *FalseStr);
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		const FString ConditionStr = Condition->ToDisplayString(InBlueprint);
		const FString TrueStr = TruePart->ToDisplayString(InBlueprint);
		const FString FalseStr = FalsePart->ToDisplayString(InBlueprint);
		return FString::Printf(TEXT("(%s ? %s : %s)"), *ConditionStr, *TrueStr, *FalseStr);
	}
public:
	TSharedRef<IFExpressionNode> Condition;
	TSharedRef<IFExpressionNode> TruePart;
	TSharedRef<IFExpressionNode> FalsePart;
};

/**
 * Branch node that represents an n-dimentional list of sub-expressions (like 
 * for vector parameter lists, etc.). Each child is a separate sub-expression:
 *
 *                 <list-node>
 *                 /    |    \
 * <sub-expression0>    |    <sub-expression2>
 *                      |
 *              <sub-expression1>              
 */
class FExpressionList : public IFExpressionNode
{
public:
	/** 
	 * Gives the FExpressionVisitor access to this node, and passes it along to 
	 * traverse all children.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		bool bAbort = !Visitor.Visit(*this, FExpressionVisitor::VISIT_Pre);
		for (TSharedRef<IFExpressionNode> Child : Children)
		{
			if (bAbort || !Child->Accept(Visitor))
			{
				return false;
			}
		}
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Post);
	}
    
	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		FString AsString("(");
		if (Children.Num() > 0)
		{
			for (TSharedRef<IFExpressionNode> Child : Children)
			{
				AsString += Child->ToString();
				if (Child == Children.Last())
				{
					AsString += ")";
				}
				else 
				{
					AsString += ", ";
				}
			}
		}
		else
		{
			AsString += ")";
		}
		return AsString;
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		FString AsString("(");
		if (Children.Num() > 0)
		{
			for (TSharedRef<IFExpressionNode> Child : Children)
			{
				AsString += Child->ToDisplayString(InBlueprint);
				if (Child == Children.Last())
				{
					AsString += ")";
				}
				else 
				{
					AsString += ", ";
				}
			}
		}
		else
		{
			AsString += ")";
		}
		return AsString;
	}

	virtual ~FExpressionList() {}

public:
	TArray< TSharedRef<IFExpressionNode> > Children;
};

/**
 * Branch node that represents some function (like sin(), cos(), etc.), could 
 * also represent some structure (conceptually the constructor), like vector, 
 * rotator, etc. Its child is a single FExpressionList (which wraps all the params).         
 */
class FFunctionExpression : public IFExpressionNode
{
public:
	FFunctionExpression(const FString& InFuncName, TSharedRef<FExpressionList> InParamList)
		: FuncName(InFuncName)
		, ParamList(InParamList)
	{
	}

	virtual ~FFunctionExpression() {}

	/** 
	 * Gives the FExpressionVisitor access to this node, and passes it along to 
	 * traverse its child.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool Accept(FExpressionVisitor& Visitor) override
	{
		bool bAbort = !Visitor.Visit(*this, FExpressionVisitor::VISIT_Pre);
		if (bAbort || !ParamList->Accept(Visitor))
		{
			return false;
		}
		return Visitor.Visit(*this, FExpressionVisitor::VISIT_Post);
	}

	/** For debug purposes, constructs a textual representation of this expression */
	virtual FString ToString() const override
	{
		const FString ParamsString = ParamList->ToString();
		return FString::Printf(TEXT("(%s%s)"), *FuncName, *ParamsString);
	}

	virtual FString ToDisplayString(UBlueprint* InBlueprint) const
	{
		const FString ParamsString = ParamList->ToDisplayString(InBlueprint);
		return FString::Printf(TEXT("(%s%s)"), *FuncName, *ParamsString);
	}
public:
	FString FuncName;
	TSharedRef<FExpressionList> ParamList;
};

/*******************************************************************************
 * FLayoutVisitor
*******************************************************************************/

/**
 * This class is utilized to help layout math expression nodes by traversing the
 * expression tree and cataloging each expression node's depth. From the tree's 
 * depth we can determine the width of the the graph (an where to place each K2 node):
 *
 *    _
 *   |            [_]---[_]
 *   |           /
 * height   [_]--       [_]--[_]---[_]
 *   |           \     /
 *   |_           [_]---[_]
 *
 *		    ^-------depth/width-------^
 */
class FLayoutVisitor : public FExpressionVisitor
{
public:
	/** Tracks the horizontal (depth) placement for each expression node encountered */
	TMap<IFExpressionNode*, int32> DepthChart;
	/** Tracks the vertical (height) placement for each expression node encountered */
	TMap<IFExpressionNode*, int32> HeightChart;
	/** Tracks the total height (value) at each depth (key) */
	TMap<int32, int32> DepthHeightLookup;

	/** */
	FLayoutVisitor()
		: CurrentDepth(0)
		, MaximumDepth(0)
	{
	}

	/**
	 * Retrieves the total depth (or graph width) of the previously traversed
	 * expression tree.
	 */
	int32 GetMaximumDepth() const
	{
		return MaximumDepth;
	}

	/**
	 * Resets this tree visitor so that it can accurately parse another 
	 * expression tree (else the results would stack up).
	 */
	void Clear() 
	{
		CurrentDepth = 0;
		MaximumDepth = 0;
		DepthChart.Empty();
		HeightChart.Empty();
		DepthHeightLookup.Empty();
	}

private:
	/**
	 * From the FExpressionVisitor interface, a generic choke point for visiting 
	 * all expression nodes.
	 *
	 * @return True to continue traversing the tree, false to abort. 
	 */
	virtual bool VisitUnhandled(class IFExpressionNode& Node, EVisitPhase Phase) override
	{
		if (Phase == FExpressionVisitor::VISIT_Pre)
		{
			++CurrentDepth;
			MaximumDepth = FMath::Max(CurrentDepth, MaximumDepth);
		}
		else
		{
			if (Phase == FExpressionVisitor::VISIT_Post)
			{
				--CurrentDepth;
			}
			// else leaf

			// CurrentHeight represents how many nodes have already been placed 
			// at this depth
			int32& CurrentHeight = DepthHeightLookup.FindOrAdd(CurrentDepth);

			DepthChart.Add(&Node, CurrentDepth);
			HeightChart.Add(&Node, CurrentHeight);

			// since we just placed another node at this depth, increase the 
			// height count
			++CurrentHeight;
		}

		// let the tree traversal continue! don't abort it!
		return true;
	}

private:
	int32 CurrentDepth;
	int32 MaximumDepth;
};

/*******************************************************************************
 * FOperatorTable
*******************************************************************************/

/** 
 * This class acts as a lookup table for mapping operator strings (like "+", 
 * "*", etc.) to corresponding functions that can be turned into blueprint 
 * nodes. It builds itself (so users don't have to add mappings themselves).
 */
class FOperatorTable
{
public:
	FOperatorTable() { Rebuild(); }

	/** 
	 * Checks to see if there are any functions associated with the specified 
	 * operator. 
	 */
	bool Contains(const FString& Operator) const
	{
		return LookupTable.Contains(Operator);
	}	

	/**
	 * Attempts to lookup a function matching the supplied signature (where 
	 * 'Operator' identifies the function's name and 'InputTypeList' defines
	 * the desired parameters). If one can't be found, it attempts to find a
	 * match by promoting the input types (like from int to double, etc.)
	 * 
	 * @param  Operator			The operator you want to find a function for.
	 * @param  InputTypeList	A list of parameter types you want to feed the function.
	 * @return A pointer to the matching function (if one was found), otherwise nullptr.
	 */
	UFunction* FindMatchingFunction(const FString& Operator, const TArray<FEdGraphPinType>& InputTypeList) const
	{
		// make a local copy of the desired input types so that we can promote 
		// those types as needed
		TArray<FEdGraphPinType> ParamTypeList = InputTypeList;

		// try to find the function
		UFunction* MatchingFunc = FindFunctionInternal(Operator, ParamTypeList);

		// if we didn't find a function that matches the supplied function 
		// signature, then try to promote the parameters (like from int to 
		// double), and see if we can lookup a function with those types
		for (int32 promoterIndex = 0; (promoterIndex < OrderedTypePromoters.Num()) && (MatchingFunc == NULL); ++promoterIndex)
		{
			const FTypePromoter& PromotionOperator = OrderedTypePromoters[promoterIndex];

			// Apply the promotion operator to any values that match
			bool bMadeChanges = false;
			for (FEdGraphPinType& ParamType : ParamTypeList)
			{
				bMadeChanges |= PromotionOperator.Execute(ParamType);
			}

			// since we've promoted some of the params, attempt to find the 
			// function again (maybe there's one that matches these param types)
			if (bMadeChanges)
			{
				MatchingFunc = FindFunctionInternal(Operator, ParamTypeList);
				// if we found a function to match this time around, no need to 
				// continue with 
				if (MatchingFunc != NULL)
				{
					break;
				}
			}
		}

		return MatchingFunc;
	}

	/** 
	 * Flags the specified function as one associated with the supplied 
	 * operator.
	 */
	void Add(const FString& Operator, UFunction* OperatorFunc)
	{
		LookupTable.FindOrAdd(Operator).Add(OperatorFunc);
	}

	/**
	 * Rebuilds the lookup table, mapping operator strings (like "+" or "*") to
	 * associated functions (searches through function libraries for operator 
	 * functions).
	 */
	void Rebuild()
	{
		LookupTable.Empty();
		OrderedTypePromoters.Empty();

		// run through all blueprint function libraries and build up a list of 
		// functions that have good operator info
		for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
		{
			UClass* TestClass = *ClassIt;
			if (TestClass->IsChildOf(UBlueprintFunctionLibrary::StaticClass()) && (!TestClass->HasAnyClassFlags(CLASS_Abstract)))
			{
				for (TFieldIterator<UFunction> FuncIt(TestClass, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
				{
					UFunction* TestFunction = *FuncIt;
					
					if (!TestFunction->HasAnyFunctionFlags(FUNC_BlueprintPure) || (TestFunction->GetReturnProperty() == nullptr))
					{
						continue;
					}
					
					FString FunctionName = TestFunction->GetName();
					const TArray<FString>& OperatorAliases = GetOperatorAliases(FunctionName);
					
					// if there are aliases, use those instead of the function's standard name
					if (OperatorAliases.Num() > 0)
					{
						for (const FString& Alias : OperatorAliases)
						{
							Add(Alias, TestFunction);
						}
					}
					else
					{
						if (TestFunction->HasMetaData(FBlueprintMetadata::MD_CompactNodeTitle))
						{
							FunctionName = TestFunction->GetMetaData(FBlueprintMetadata::MD_CompactNodeTitle);
						}
						else if (TestFunction->HasMetaData(FBlueprintMetadata::MD_DisplayName))
						{
							FunctionName = TestFunction->GetMetaData(FBlueprintMetadata::MD_DisplayName);
						}

						// Remove spaces from display name as the parser cannot handle it
						FunctionName = FDefaultValueHelper::RemoveWhitespaces(FunctionName);

						Add(FunctionName, TestFunction);
					}
				}
			}
		}

		FTypePromoter ByteToIntPromoter;
		ByteToIntPromoter.BindStatic(&PromoteByteToInt);
		OrderedTypePromoters.Add(ByteToIntPromoter);

		FTypePromoter IntToDoublePromoter;
		IntToDoublePromoter.BindStatic(&PromoteIntToDouble);
		OrderedTypePromoters.Add(IntToDoublePromoter);
	}

private:
	/**
	 * Attempts to lookup a function matching the supplied signature (where 
	 * 'Operator' identifies the function's name and 'InputTypeList' defines
	 * the desired parameters into that function).
	 * 
	 * @param  Operator			The operator you want to find a function for.
	 * @param  InputTypeList	A list of parameter types you want to feed the function.
	 * @return A pointer to the matching function (if one was found), otherwise nullptr.
	 */
	UFunction* FindFunctionInternal(const FString& Operator, const TArray<FEdGraphPinType>& InputTypeList) const
	{
		UFunction* MatchedFunction = nullptr;

		const FFunctionsList* OperatorFunctions = LookupTable.Find(Operator);
		if (OperatorFunctions != nullptr)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			for (UFunction* TestFunction : *OperatorFunctions)
			{
				int32 ArgumentIndex = 0;
				for (TFieldIterator<FProperty> PropIt(TestFunction); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
				{
					FProperty* Param = *PropIt;
					if (!Param->HasAnyPropertyFlags(CPF_ReturnParm))
					{
						if (ArgumentIndex < InputTypeList.Num())
						{
							FEdGraphPinType ParamType;
							if (K2Schema->ConvertPropertyToPinType(Param, /*out*/ParamType))
							{
								const FEdGraphPinType& TypeToMatch = InputTypeList[ArgumentIndex];
								if (!K2Schema->ArePinTypesCompatible(TypeToMatch, ParamType))
								{
									break; // type mismatch
								}
							}
							else
							{
								break; // function has a non-K2 type as a parameter
							}
						}
						else
						{
							break; // ran out of arguments; no match
						}
						++ArgumentIndex;
					}
				}

				if (ArgumentIndex == InputTypeList.Num())
				{
					// success!
					MatchedFunction = TestFunction;
					break;
				}
			}
		}

		return MatchedFunction;
	}
	
	/**
	 * Here we overwrite and map multiplies names to specific functions (for 
	 * example "MultiplyMultiply_FloatFloat" and "^2" are not the sort of names
	 * we'd expect a user to input in a mathematical expression). We can 
	 * replace a function name with a single value, or a series of values 
	 * (could setup "asin" and "arcsin" both as aliases for the ASin() method).
	 *
	 * @param  FunctionName		The raw name of the function you're looking to replace (not the friendly name)
	 * @return A reference to the array of aliases for the specified function (an empty array if none were found).
	 */
	static const TArray<FString>& GetOperatorAliases(const FString& FunctionName)
	{
#define FUNC_ALIASES_BEGIN(FuncName) \
		if (FunctionName == FString(TEXT(FuncName))) \
		{ \
			static TArray<FString> AliasTable; \
			if (AliasTable.Num() == 0) \
			{
#define ADD_ALIAS(AliasStr) \
				AliasTable.Add(TEXT(AliasStr));
#define FUNC_ALIASES_END \
			} \
			return AliasTable; \
		}
				
		FUNC_ALIASES_BEGIN("BooleanAND")
			ADD_ALIAS("&&")
		FUNC_ALIASES_END
				
		FUNC_ALIASES_BEGIN("BooleanOR")
			ADD_ALIAS("||")
		FUNC_ALIASES_END
				
		FUNC_ALIASES_BEGIN("BooleanXOR")
			ADD_ALIAS("^")
		FUNC_ALIASES_END
				
		FUNC_ALIASES_BEGIN("Not_PreBool")
			ADD_ALIAS("!")
		FUNC_ALIASES_END
		
		// keep the compact node title of "^2" from being the required key
		FUNC_ALIASES_BEGIN("Square")
			ADD_ALIAS("SQUARE")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("FClamp")
			ADD_ALIAS("CLAMP")
		FUNC_ALIASES_END
				
		FUNC_ALIASES_BEGIN("MultiplyMultiply_FloatFloat")
			ADD_ALIAS("POWER")
			ADD_ALIAS("POW")
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("FCEIL")
			ADD_ALIAS("FCEIL")
			ADD_ALIAS("CEIL")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("ASin")
			// have to add "ASin" back, because this overwrites the function's
			// name and we still want it as a viable option
			ADD_ALIAS("ASIN")
			ADD_ALIAS("ARCSIN")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("ACos")
			ADD_ALIAS("ACOS")
			ADD_ALIAS("ARCCOS")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("ATan")
			ADD_ALIAS("ATAN")
			ADD_ALIAS("ARCTAN")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("DegAtan")
			ADD_ALIAS("DEGATAN")
			ADD_ALIAS("DEGARCTAN")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("DegAtan2")
			ADD_ALIAS("DEGATAN2")
			ADD_ALIAS("DEGARCTAN2")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("DegreesToRadians")
			ADD_ALIAS("DEGTORAD")
			ADD_ALIAS("D2R")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("RadiansToDegrees")
			ADD_ALIAS("RADTODEG")
			ADD_ALIAS("R2D")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("ATan2")
			ADD_ALIAS("ATAN2")
			ADD_ALIAS("ARCTAN2")
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("MakeVector")
			ADD_ALIAS("VECTOR")
			ADD_ALIAS("VEC")
			ADD_ALIAS("VECT")
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("MakeVector2D")
			ADD_ALIAS("VECTOR2D")
			ADD_ALIAS("VEC2D")
			ADD_ALIAS("VECT2D")
		FUNC_ALIASES_END
	
		FUNC_ALIASES_BEGIN("MakeRotator")
			ADD_ALIAS("ROTATOR")
			ADD_ALIAS("ROT")
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("MakeTransform")
			ADD_ALIAS("TRANSFORM")
			ADD_ALIAS("XFORM")
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("MakeColor")
			ADD_ALIAS("COLOR")
			ADD_ALIAS("LINEARCOLOR")
			ADD_ALIAS("COLOUR") // long live the empire!
		FUNC_ALIASES_END
		
		FUNC_ALIASES_BEGIN("RandomFloat")
			ADD_ALIAS("RandomFloat")
			ADD_ALIAS("RAND")
			ADD_ALIAS("RANDOM")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("Dot_VectorVector")
			ADD_ALIAS("Dot")
		FUNC_ALIASES_END

		FUNC_ALIASES_BEGIN("Cross_VectorVector")
			ADD_ALIAS("Cross")
		FUNC_ALIASES_END
				
		// if none of the above aliases returned, then we don't have any for
		// this function (use its regular name)
		static TArray<FString> NoAliases;
		return NoAliases;
				
#undef FUNC_ALIASES_END
#undef ADD_ALIAS
#undef FUNC_ALIASES_END
	}

private:
	/** 
	 * A single operator can have multiple functions associated with it; usually 
	 * for handling different types (int*int, vs. int*vector), hence this array.
	 */
	typedef TArray<UFunction*> FFunctionsList;
	/** 
	 * A lookup table, mapping operator strings (like "+", "*", etc.) to a list 
	 * of associated functions. 
	 */
	TMap<FString, FFunctionsList> LookupTable;

	/**
	 * When looking to match parameters, there are some implicit conversions we
	 * can make to try and find a match (like converting from int to double). 
	 * This holds an ordered list of delegates that will try and promote the 
	 * supplied types.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FTypePromoter, FEdGraphPinType&);
	TArray<FTypePromoter> OrderedTypePromoters;
};

/*******************************************************************************
 * FCodeGenFragments
*******************************************************************************/

/**
 * FCodeGenFragments facilitate the making of pin connections/defaults. When 
 * turning an expression tree into a network of UK2Nodes, you traverse the tree,
 * working backwards from the expression's result node. This means that when you
 * spawn a UK2Node, you don't have the node (or literals) that should be plugged 
 * into it, that is why these fragments are created (to track the associated 
 * UK2Node/literal, and provide an easy interface for connecting it later with
 * other fragments/nodes).
 */
class FCodeGenFragment
{
public:
	FCodeGenFragment(FEdGraphPinType InType)
		: FragmentType(InType)
	{
	}

	virtual ~FCodeGenFragment()
	{
	}

    /**
     * Takes the input to some other fragment, and plugs the result of this one 
	 * into it.
     *
     * @param  InputPin	 Either an input into some parent expression, or the final output for the entire math expression.
	 * @return True is the connection was made, otherwise false.
     */
	virtual bool ConnectToInput(UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog) = 0;

	/**
	 * As it stands, all the math nodes/literals that can be generated have a 
	 * singular output (hence why we have a basic "connect this fragment to an  
	 * input" function). This message retrieves that output type.
	 * 
	 * @return The pin type of this fragment's output.
	 */
	const FEdGraphPinType& GetOutputType() const
	{
		return FragmentType;
	}

protected:
	/**
	 * Utility method for sub-classes to use when attempting a connection 
	 * between two pins. Tries to connect two pins, verifying the type/etc, and
	 * reporting a failure if there is one.
	 * 
	 * @param  OutputPin	The output pin (probably from this fragment).
	 * @param  InputPin		The input pin (probably from some other fragment). 
	 * @return True if the connection was made, false if the pins weren't compatible.
	 */
	bool SafeConnectPins(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog)
	{
		const UEdGraphSchema* Schema = InputPin->GetSchema();
		bool bSuccess = Schema->TryCreateConnection(OutputPin, InputPin);

		if (!bSuccess)
		{
			MessageLog.Error(*LOCTEXT("PinsNotCompatible", "Output pin '@@ 'is not compatible with input: '@@'").ToString(),
				OutputPin, InputPin);
		}

		return bSuccess;
	}

private:
	/** All fragments have a singular output type, this is considered the fragment's type as a whole. */
	FEdGraphPinType FragmentType;
};

/**
 * If the user uses a variable name that already exists in the blueprint, then
 * we use that instead of adding an extra input. This fragment wraps a
 * VariableGet node that was generated in that scenario.
 */
class FCodeGenFragment_VariableGet : public FCodeGenFragment
{
public:
	FCodeGenFragment_VariableGet(UK2Node_VariableGet* InNode, const FEdGraphPinType& InType)
		: FCodeGenFragment(InType)
		, GeneratedNode(InNode)
	{
		check(GeneratedNode != nullptr);
	}

	virtual ~FCodeGenFragment_VariableGet() {}

	/// Begin FCodeGenFragment Interface
	virtual bool ConnectToInput(UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog) override
	{
		bool bSuccess = false;
		if (UEdGraphPin* VariablePin = GeneratedNode->FindPin(GeneratedNode->VariableReference.GetMemberName()))
		{
			bSuccess = SafeConnectPins(VariablePin, InputPin, MessageLog);
		}
		else
		{
			FText ErrorText = FText::Format(LOCTEXT("NoVariablePin", "Failed to find the '{0}' pin for: '@@'"),
				FText::FromName(GeneratedNode->VariableReference.GetMemberName()));
			MessageLog.Error(*ErrorText.ToString(), GeneratedNode);
		}
		return bSuccess;
	}
	/// End FCodeGenFragment Interface

private:
	UK2Node_VariableGet* GeneratedNode;
};

/**
 * All operators in the mathematical expression correspond to library functions, 
 * which in turn generate CallFunction nodes. This fragment wraps one of those 
 * operation nodes and connects it with the given input (when prompted to).
 */
class FCodeGenFragment_FuntionCall : public FCodeGenFragment
{
public:
	FCodeGenFragment_FuntionCall(UK2Node_CallFunction* InNode, const FEdGraphPinType& InType)
		: FCodeGenFragment(InType)
		, GeneratedNode(InNode)
	{
		check(GeneratedNode != nullptr);
	}

	virtual ~FCodeGenFragment_FuntionCall() {}

	/// Begin FCodeGenFragment Interface
	virtual bool ConnectToInput(UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog) override
	{
		bool bSuccess = false;
		if (UEdGraphPin* ResultPin = GeneratedNode->GetReturnValuePin())
		{
			bSuccess = SafeConnectPins(ResultPin, InputPin, MessageLog);
		}
		else
		{
			MessageLog.Error(*LOCTEXT("NoRetValPin", "Failed to find an output pin for: '@@'").ToString(),
				GeneratedNode);
		}
		return bSuccess;
	}
	/// End FCodeGenFragment Interface

private:
	UK2Node_CallFunction* GeneratedNode;
};

/**
 * This fragment doesn't have a corresponding UK2Node, instead it represents a
 * constant value that should be entered into another node's input field. When
 * "connected", it modifies the connecting pin's DefaultValue.
 */
class FCodeGenFragment_Literal : public FCodeGenFragment
{
public:
	FCodeGenFragment_Literal(const FString& LiteralVal, const FEdGraphPinType& ResultType)
		: FCodeGenFragment(ResultType) 
		, DefaultValue(LiteralVal)
	{}

	virtual ~FCodeGenFragment_Literal() {}

	/// Begin FCodeGenFragment Interface
	virtual bool ConnectToInput(UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog) override
	{
		const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(InputPin->GetSchema());
		bool bSuccess = true;//K2Schema->ArePinTypesCompatible(GetOutputType(), InputPin->PinType);
		if (bSuccess)
		{
			K2Schema->TrySetDefaultValue(*InputPin, DefaultValue, false);
		}
		else 
		{
			const FText ErrorText = FText::Format(LOCTEXT("LiteralNotCompatible", "Literal type ({0}) is incompatible with pin: '@@'"), 
				FText::FromName(GetOutputType().PinCategory));
			MessageLog.Error(*ErrorText.ToString(), InputPin);
		}
		return bSuccess;
	}
	/// End FCodeGenFragment Interface

private: 
	FString DefaultValue;
};

/**
 * This fragment corresponds to an input pin that was added to the 
 * MathExpression node. Input pins are generated when the user enters variable 
 * names (like "x", or "y"... ones that aren't variables on the blueprint).
 */
class FCodeGenFragment_InputPin : public FCodeGenFragment
{
public:
	FCodeGenFragment_InputPin(UEdGraphPin* InTunnelInputPin)
		: FCodeGenFragment(InTunnelInputPin->PinType)
		, TunnelInputPin(InTunnelInputPin)
	{
	}

	virtual ~FCodeGenFragment_InputPin() {}

	/// Begin FCodeGenFragment Interface
	virtual bool ConnectToInput(UEdGraphPin* InputPin, FCompilerResultsLog& MessageLog) override
	{
		return SafeConnectPins(TunnelInputPin, InputPin, MessageLog);
	}
	/// End FCodeGenFragment Interface
	
private:
	UEdGraphPin* TunnelInputPin;
};

/*******************************************************************************
 * FMathGraphGenerator
*******************************************************************************/

/**
 * Takes the root of an expression tree and instantiates blueprint nodes/pins   
 * for the specified UK2Node_MathExpression (which is a tunnel node, similar to 
 * how collapsed composite nodes work).
 */
class FMathGraphGenerator final : public FExpressionVisitor
{
public:
	FMathGraphGenerator(UK2Node_MathExpression* InNode)
		: CompilingNode(InNode)
		, TargetBlueprint(FBlueprintEditorUtils::FindBlueprintForGraphChecked(InNode->BoundGraph))
		, ActiveMessageLog(nullptr)
	{
	}

	/**
	 * Takes an expression tree and converts expression nodes into UK2Nodes, 
	 * connecting them, and adding them under the math expression node that
     * this was instantiated with.
	 * 
	 * @param  ExpressionRoot   The root of the expression tree that you want converted into a UK2Node network.
	 */
	bool GenerateCode(TSharedRef<IFExpressionNode> ExpressionRoot, FCompilerResultsLog& MessageLog)
	{
		ActiveMessageLog = &MessageLog;
		// want to track if we generated any errors from this pass, so we need to know how many we started with
		int32 StartingErrorCount = MessageLog.NumErrors;
		
		InputPinNames.Empty();

		LayoutMapper.Clear();
		// map the depth/height of expression tree (so we can position nodes prettily)
		ExpressionRoot->Accept(LayoutMapper);
		// reset the bounds tracking, so we can adjust it as we spawn nodes
		GraphXBounds.X = +LayoutMapper.GetMaximumDepth();
		GraphXBounds.Y = -LayoutMapper.GetMaximumDepth();
		
		// traverse the expression tree, spawning nodes as we go along
		ExpressionRoot->Accept(*this);

		UK2Node_Tunnel* EntryNode = CompilingNode->GetEntryNode();
		UK2Node_Tunnel* ExitNode  = CompilingNode->GetExitNode();

		TSharedPtr<FCodeGenFragment> RootFragment = CompiledFragments.FindRef(&ExpressionRoot.Get());
		if (RootFragment.IsValid())
		{
			// connect the final node of the expression with the math-node's output
			UEdGraphPin* ReturnPin = ExitNode->CreateUserDefinedPin(UEdGraphSchema_K2::PN_ReturnValue, RootFragment->GetOutputType(), EGPD_Input);
			if (!RootFragment->ConnectToInput(ReturnPin, MessageLog))
			{
				MessageLog.Error(*LOCTEXT("ResultConnectError", "Failed to connect the generated nodes with expression's result pin: '@@'").ToString(),
					ReturnPin);
			}
		}
		else
		{
			if (MessageLog.NumErrors == 0)
			{
				MessageLog.Error(*LOCTEXT("NoGraphGenerated", "No root node generated from the expression: '@@'").ToString(), CompilingNode);
			}
		}

		// position the entry and exit nodes somewhere sane
		{
			const FVector2D EntryPos = GetNodePosition(static_cast<int32>(GraphXBounds.X - 1), 0);
			EntryNode->NodePosX = static_cast<int32>(EntryPos.X);
			EntryNode->NodePosY = static_cast<int32>(EntryPos.Y);

			const FVector2D ExitPos = GetNodePosition(static_cast<int32>(GraphXBounds.Y + 1), 0);
			ExitNode->NodePosX = static_cast<int32>(ExitPos.X);
			ExitNode->NodePosY = static_cast<int32>(ExitPos.Y);
		}
				
		bool bHasErrors = ((MessageLog.NumErrors - StartingErrorCount) > 0);
		ActiveMessageLog = nullptr;
		
		return !bHasErrors;
	}

	/**
	 * When the node gen is over, we need to clear any old pins that weren't 
	 * reused. This query method helps in identifying those that were utilized.
	 *
	 * @return True if the pin's name was used in the most recent expression, false if not.
	 */
	bool IsPinInUse(TSharedPtr<FUserPinInfo> PinInfo)
	{
		return (InputPinNames.Find(PinInfo->PinName) != INDEX_NONE);
	}
    
    /**
	 * Overloaded, part of the FExpressionVisitor interface; attempts to 
     * generate either a vaiable-get node, an input pin, or a literal fragment
     * from the supplied FTokenWrapperNode (all depends on the token's type).
	 *
	 * @return True to continue travesing the expression tree, false to stop.
	 */
	virtual bool Visit(FTokenWrapperNode& ExpressionNode, EVisitPhase Phase) override
	{
		check(ActiveMessageLog != nullptr);
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		if (ExpressionNode.Token.TokenType == FBasicToken::TOKEN_Identifier || ExpressionNode.Token.TokenType == FBasicToken::TOKEN_Guid)
		{
			const FString VariableIdentifier = ExpressionNode.Token.Identifier;
			// first we try to match up variables with existing variable properties on the blueprint

			FMemberReference VariableReference;
			FName VariableName;
			FGuid VariableGuid;

			if (ExpressionNode.Token.TokenType == FBasicToken::TOKEN_Guid && FGuid::Parse(VariableIdentifier, VariableGuid))
			{
				// First look the variable up as a Member variable
				VariableName = FBlueprintEditorUtils::FindMemberVariableNameByGuid(TargetBlueprint, VariableGuid);

				// If the variable was not found, look it up as a local variable
				if (VariableName.IsNone())
				{
					VariableName = FBlueprintEditorUtils::FindLocalVariableNameByGuid(TargetBlueprint, VariableGuid);
					VariableReference.SetLocalMember(VariableName, CompilingNode->GetGraph()->GetName(), VariableGuid);
				}
				else
				{
					VariableReference.SetSelfMember(VariableName);
				}
			}
			else 
			{
				VariableName = *VariableIdentifier;

				// First look the variable up as a Member variable
				VariableGuid = FBlueprintEditorUtils::FindMemberVariableGuidByName(TargetBlueprint, VariableName);

				// If the variable was not found, look it up as a local variable
				if (!VariableGuid.IsValid())
				{
					VariableGuid = FBlueprintEditorUtils::FindLocalVariableGuidByName(TargetBlueprint, CompilingNode->GetGraph(), VariableName);
					if (VariableGuid.IsValid())
					{
						VariableReference.SetLocalMember(VariableName, CompilingNode->GetGraph()->GetName(), VariableGuid);
					}
				}
				else
				{
					VariableReference.SetSelfMember(VariableName);
				}

				// If we found a valid Guid, change the expression's identifier to be the Guid
				if (VariableGuid.IsValid())
				{
					FCString::Strncpy(ExpressionNode.Token.Identifier, *VariableGuid.ToString(EGuidFormats::DigitsWithHyphensInBraces), NAME_SIZE);
					ExpressionNode.Token.TokenType = FBasicToken::TOKEN_Guid;
				}
			}

			if (FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(TargetBlueprint))
			{
				TSharedPtr<FCodeGenFragment_VariableGet> VariableGetFragment = GeneratePropertyFragment(ExpressionNode, VariableProperty, VariableReference, *ActiveMessageLog);
				if (VariableGetFragment.IsValid())
				{
					CompiledFragments.Add(&ExpressionNode, VariableGetFragment);
				}
			}
			// if a variable-get couldn't be created for it, it needs to be an input to the math node
			else if(ExpressionNode.Token.TokenType != FBasicToken::TOKEN_Guid)
			{
				CompiledFragments.Add(&ExpressionNode, GenerateInputPinFragment(*VariableIdentifier));
			}
			
		}
		else if (ExpressionNode.Token.TokenType == FBasicToken::TOKEN_Const)
		{
			CompiledFragments.Add(&ExpressionNode, GenerateLiteralFragment(ExpressionNode.Token, *ActiveMessageLog));
		}
		else // TOKEN_Symbol
		{
			FText ErrorText = FText::Format(LOCTEXT("UhandledTokenType", "Unhandled token '{0}' in expression: '@@'"),
				FText::FromString(ExpressionNode.Token.Identifier));

			ActiveMessageLog->Error(*ErrorText.ToString(), CompilingNode);
		}
		
		// keep traversing the expression tree... we should handle cascading
		// errors that result from ones incurred here, gathering them all as we
		// go, presenting them to the user later
		return true;
	}

	/**
	 * Overloaded, part of the FExpressionVisitor interface... On VISIT_Post, 
     * attempts to generate a UK2Node_CallFunction node for the specified
     * FBinaryOperator.
	 *
	 * @return True to continue traversing the expression tree, false to stop.
	 */
	virtual bool Visit(FBinaryOperator& ExpressionNode, EVisitPhase Phase) override
	{
		check(ActiveMessageLog != nullptr);
		
		// we only care about the "Post" visit, after the operands fragments have been generated
		if (Phase == FExpressionVisitor::VISIT_Post)
		{
			TSharedPtr<FCodeGenFragment> LHS = CompiledFragments.FindRef(&(ExpressionNode.LHS.Get()));
			TSharedPtr<FCodeGenFragment> RHS = CompiledFragments.FindRef(&(ExpressionNode.RHS.Get()));
			
			TArray< TSharedPtr<FCodeGenFragment> > ArgumentList;
			ArgumentList.Add(LHS);
			ArgumentList.Add(RHS);
			
			TSharedPtr<FCodeGenFragment_FuntionCall> FunctionFragment = GenerateFunctionFragment(ExpressionNode, ExpressionNode.Operator, ArgumentList, *ActiveMessageLog);
			if (FunctionFragment.IsValid())
			{
				CompiledFragments.Add(&ExpressionNode, FunctionFragment);
			}
        }
        
		// keep traversing the expression tree... we should handle cascading
		// errors that result from ones incurred here, gathering them all as we
		// go, presenting them to the user later
		return true;
	}
	
	/**
	 * Does nothing (but had to prevent this expression node from being flagged 
	 * as "unhandled"). Expression lists are handled by whatever expression 
	 * they're contained within.
	 *
	 * @return Always true, it is expected that cascading errors are handled (and all should be logged).
	 */
	virtual bool Visit(FExpressionList& ExpressionNode, EVisitPhase Phase) override
	{
		// no fragments are generated from a list node, it mostly acts as a link
		// from a parent node to some set of sub-expressions
		
		
		// keep traversing the expression tree... if there are any errors,
		// they'll be caught in the children nodes (or maybe in the parent)
		return true;
	}
	
	/**
	 * Overloaded, part of the FExpressionVisitor interface... On VISIT_Post, 
     * attempts to generate a UK2Node_CallFunction node for the specified
     * FFunctionExpression.
	 *
	 * @return True to continue traversing the expression tree, false to stop.
	 */
	virtual bool Visit(FFunctionExpression& ExpressionNode, EVisitPhase Phase) override
	{
		check(ActiveMessageLog != nullptr);
		
		// we only care about the "Post" visit, after the function's parameter fragments have been generated
		if (Phase == FExpressionVisitor::VISIT_Post)
		{
			TArray< TSharedPtr<FCodeGenFragment> > ArgumentList;
			for (TSharedRef<IFExpressionNode> Param : ExpressionNode.ParamList->Children)
			{
				TSharedPtr<FCodeGenFragment> ParamFragment = CompiledFragments.FindRef(&(Param.Get()));
				ArgumentList.Add(ParamFragment);
			}
			
			TSharedPtr<FCodeGenFragment_FuntionCall> FunctionFragment = GenerateFunctionFragment(ExpressionNode, ExpressionNode.FuncName, ArgumentList, *ActiveMessageLog);
			if (FunctionFragment.IsValid())
			{
				CompiledFragments.Add(&ExpressionNode, FunctionFragment);
			}
		}
		
		// keep traversing the expression tree... we should handle cascading
		// errors that result from ones incurred here, gathering them all as we
		// go, presenting them to the user later
		return true;
	}
	
	/**
	 * From the FExpressionVisitor interface; where we would handle prefixed 
	 * unary operators. Currently support for those is unimplemented, so we just
	 * log a descriptive error and return.
	 *
	 * @return Always true, it is expected that cascading errors are handled (and all should be logged).
	 */
	virtual bool Visit(FUnaryOperator& ExpressionNode, EVisitPhase Phase) override
	{
		// don't want to double up on the error message (in the "Post" phase)
		if (Phase == VISIT_Pre)
		{
			FText ErrorText = FText::Format(LOCTEXT("UnaryExpressionError", "Currently, unary operators {0} are prohibited in expressions: '@@'"),
				FText::FromString(ExpressionNode.ToString()));
			
			ActiveMessageLog->Error(*ErrorText.ToString(), CompilingNode);
		}
		
		// keep traversing the expression tree... we should handle cascading
		// errors that result from this, and gather them all to present to the user
		return true;
	}
	
	/**
	 * From the FExpressionVisitor interface; where we would handle conditional
	 * ?: operators. Currently support for those is unimplemented, so we just
	 * log a descriptive error and return.
	 *
	 * @return Always true, it is expected that cascading errors are handled (and all should be logged).
	 */
	virtual bool Visit(FConditionalOperator& ExpressionNode, EVisitPhase Phase) override
	{
		check(ActiveMessageLog != nullptr);
		
		// don't want to double up on the error message (in the "Post" phase)
		if (Phase == VISIT_Pre)
		{
			FText ErrorText = FText::Format(LOCTEXT("ConditionalExpressionError", "Currently, conditional operators {0} are prohibited in expressions: '@@'"),
				FText::FromString(ExpressionNode.ToString()));
			
			ActiveMessageLog->Error(*ErrorText.ToString(), CompilingNode);
		}
		
		// keep traversing the expression tree... we should handle cascading
		// errors that result from this, and gather them all to present to the user
		return true;
	}
	
private:
	/**
	 * Another FExpressionVisitor interface function... A Generic catch all for 
	 * any expression nodes that we don't explicitly handle. Simply logs an 
	 * error, and returns.
	 *
	 * @return Always true, it is expected that cascading errors can be handled (and all should be logged).
	 */
	virtual bool VisitUnhandled(IFExpressionNode& ExpressionNode, EVisitPhase Phase) override
	{
		check(ActiveMessageLog != nullptr);
		if (Phase == VISIT_Leaf || Phase == VISIT_Pre)
		{
			FText ErrorText = FText::Format(LOCTEXT("UnhandledExpressionNode", "Unsupported operation ({0}) in the expression: '@@'"),
				FText::FromString(ExpressionNode.ToString()));
		
			ActiveMessageLog->Error(*ErrorText.ToString(), CompilingNode);
		}
		
		// keep traversing the expression tree... we should handle cascading
		// errors that result from this, and gather them all to present to the user
		return true;
	}
	
	/**
     * Either adds a new pin, or finds an existing one on the MathExpression
     * node. From that, a fragment is generated (to track the pin, so
     * connections can be made later).
     *
     * @param  VariableIdentifier   The name of the pin to generate a fragment for.
     * @return A new input pin fragment (associated with a pin on the math expression's entry node).
     */
	TSharedPtr<FCodeGenFragment_InputPin> GenerateInputPinFragment(const FName VariableIdentifier)
	{
		TSharedPtr<FCodeGenFragment_InputPin> InputPinFragment;
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		
		UK2Node_Tunnel* EntryNode = CompilingNode->GetEntryNode();
		// if a pin under this name already exists, use that
		if (UEdGraphPin* InputPin = EntryNode->FindPin(VariableIdentifier))
		{
			InputPinFragment = MakeShareable(new FCodeGenFragment_InputPin(InputPin));
		}
		// otherwise, a new input pin needs to be created for it
		else
		{
			// Create an input pin (using the default guessed type)
			FEdGraphPinType DefaultType;
			// currently, generated expressions ALWAYS take a double (it is the most versatile type)
			DefaultType.PinCategory = UEdGraphSchema_K2::PC_Real;
			DefaultType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
			
			UEdGraphPin* NewInputPin = EntryNode->CreateUserDefinedPin(VariableIdentifier, DefaultType, EGPD_Output);
			InputPinFragment = MakeShareable(new FCodeGenFragment_InputPin(NewInputPin));
		}
		
		// when regenerating a node, we need to clear any old pins that weren't 
		// reused (can't do this before the node gen because the user may have 
		// altered a pin to how they want it), so here we track the ones that 
		// were used by the latest expression
		InputPinNames.Add(VariableIdentifier);

		return InputPinFragment;
	}
	
    /**
     * Attempts to generate a VariableGet node for the blueprint graph. If one
     * isn't generated, then this function logs an error (and returns an empty
     * pointer). However, if one is successfully created, then a fragment
     * wrapper is created and returned (to aid in linking the node later).
     *
     * @param  ExpressionContext    The expression node that we're generating this fragment for.
	 * @param  VariableProperty     The variable that the generated UK2Node should access.
     * @param  VariableReference    Variable reference to assign to the node
     * @return An empty pointer if we failed to generate something, otherwise new variable-get fragment.
     */
	TSharedPtr<FCodeGenFragment_VariableGet> GeneratePropertyFragment(FTokenWrapperNode& ExpressionContext, FProperty* VariableProperty, FMemberReference& MemberReference, FCompilerResultsLog& MessageLog)
	{
		check(ExpressionContext.Token.TokenType == FBasicToken::TOKEN_Identifier || ExpressionContext.Token.TokenType == FBasicToken::TOKEN_Guid);
		check(VariableProperty != nullptr);
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		
		TSharedPtr<FCodeGenFragment_VariableGet> VariableGetFragment;
		
		UClass* VariableAccessClass = TargetBlueprint->SkeletonGeneratedClass;
		if (MemberReference.IsLocalScope() || UEdGraphSchema_K2::CanUserKismetAccessVariable(VariableProperty, VariableAccessClass, UEdGraphSchema_K2::CannotBeDelegate))
		{
			FEdGraphPinType VarType;
			if (K2Schema->ConvertPropertyToPinType(VariableProperty, /*out*/VarType))
			{
				UK2Node_VariableGet* NodeTemplate = NewObject<UK2Node_VariableGet>();
				NodeTemplate->VariableReference = MemberReference;
				UK2Node_VariableGet* VariableGetNode = SpawnNodeFromTemplate<UK2Node_VariableGet>(&ExpressionContext, NodeTemplate);
				
				VariableGetFragment = MakeShareable(new FCodeGenFragment_VariableGet(VariableGetNode, VarType));
			}
			else
			{
				FText ErrorText = FText::Format(LOCTEXT("IncompatibleVarError", "Blueprint '{0}' variable is incompatible with graph pins in the expression: '@@'"),
					FText::FromName(VariableProperty->GetFName()));
				MessageLog.Error(*ErrorText.ToString(), CompilingNode);
			}
		}
		else
		{
			FText ErrorText = FText::Format(LOCTEXT("InaccessibleVarError", "Cannot access the blueprint's '{0}' variable from the expression: '@@'"),
				FText::FromName(VariableProperty->GetFName()));
			MessageLog.Error(*ErrorText.ToString(), CompilingNode);
		}
		
		return VariableGetFragment;
	}
	
    /**
     * Spawns a fragment which wraps a literal value. No graph-node or pin is
	 * created for this fragment; instead, it saves the literal value for later,
	 * when this fragment is connected with another (it then enters the literal
	 * value as the connecting pin's default).
	 *
	 * @param  ExpressionNode	The expression node that we're generating this fragment for.
     * @return A new literal fragment.
     */
	TSharedPtr<FCodeGenFragment_Literal> GenerateLiteralFragment(const FBasicToken& Token, FCompilerResultsLog& MessageLog)
	{
		check(Token.TokenType == FBasicToken::TOKEN_Const);
		
		FEdGraphPinType LiteralType;
		switch (Token.ConstantType)
		{
			case CPT_Bool:
				LiteralType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
				break;
			case CPT_Double:
				LiteralType.PinCategory = UEdGraphSchema_K2::PC_Real;
				LiteralType.PinSubCategory = UEdGraphSchema_K2::PC_Double;
				break;
			case CPT_Int:
				LiteralType.PinCategory = UEdGraphSchema_K2::PC_Int;
				break;
			case CPT_String:
				LiteralType.PinCategory = UEdGraphSchema_K2::PC_String;
				break;
			default:
				MessageLog.Error(*FText::Format(LOCTEXT("UnhandledLiteralType", "Unknown literal type in expression: '@@'"),
					FText::AsNumber(Token.ConstantType)).ToString(),
					CompilingNode);
				break;
		};
		
		return MakeShareable(new FCodeGenFragment_Literal(Token.GetConstantValue(), LiteralType));
	}
	
	/**
	 * Attempts to find a coresponding fuction (in this class's FOperatorTable),
	 * one that matches the supplied operator name and the set of arguments. If
	 * a matching function is found, then a wrapping UK2Node_CallFunction is
	 * spawned and linked with the supplied arguments (otherwise, errors will
	 * be logged and an empty pointer will be returned).
	 *
	 * @param  ExpressionContext	The expression node that we're generating this fragment for.
	 * @param  FunctionName			The name of the operator (the key we're going to use looking up into FOperatorTable).
	 * @param  ArgumentList			A set of other fragments that will plug in as parameters into function.
	 * @return An empty pointer if we failed to generate something, otherwise the new function fragment.
	 */
	TSharedPtr<FCodeGenFragment_FuntionCall> GenerateFunctionFragment(IFExpressionNode& ExpressionContext, FString FunctionName, TArray< TSharedPtr<FCodeGenFragment> > ArgumentList, FCompilerResultsLog& MessageLog)
	{
		bool bMissingArgument = false;
		
		TArray<FEdGraphPinType> TypeList;
		// create a type list from the argument fragments (so we can find a matching function signature)
		for (int32 Index = 0; Index < ArgumentList.Num(); ++Index)
		{
			if (!ArgumentList[Index].IsValid())
			{
				FText ErrorText = FText::Format(LOCTEXT("MissingArgument", "Failed to generate argument #{0} for the '{1}' function, in the expression: '@@'"),
					FText::AsNumber(Index + 1),
					FText::FromString(FunctionName));
				
				MessageLog.Error(*ErrorText.ToString(), CompilingNode);
				
				bMissingArgument = true;
				continue;
			}
			TypeList.Add(ArgumentList[Index]->GetOutputType());
		}
		
		TSharedPtr<FCodeGenFragment_FuntionCall> FunctionFragment;
		if (!OperatorLookup.Contains(FunctionName))
		{
			FText ErrorText = FText::Format(LOCTEXT("UnknownFuncError", "Unknown function '{0}' in the expression: '@@'"), FText::FromString(FunctionName));
			MessageLog.Error(*ErrorText.ToString(), CompilingNode);
		}
		else if (bMissingArgument)
		{
			// don't execute the other if-branches, head them off if there is already an error
		}
		else if (UFunction* MatchingFunction = OperatorLookup.FindMatchingFunction(FunctionName, TypeList))
		{
			FProperty* ReturnProperty = MatchingFunction->GetReturnProperty();
			if (ReturnProperty == nullptr)
			{
				FText ErrorText = FText::Format(LOCTEXT("NoReturnTypeError", "The '{0}' function returns nothing, it cannot be used in the expression: '@@'"),
					FText::FromString(FunctionName));
				MessageLog.Error(*ErrorText.ToString(), CompilingNode);
			}
			else
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
				
				FEdGraphPinType ReturnType;
				if (K2Schema->ConvertPropertyToPinType(ReturnProperty, /*out*/ReturnType))
				{
					UK2Node_CallFunction* NodeTemplate = NewObject<UK2Node_CallFunction>(CompilingNode->GetGraph());
					NodeTemplate->SetFromFunction(MatchingFunction);
					UK2Node_CallFunction* FunctionCall = SpawnNodeFromTemplate<UK2Node_CallFunction>(&ExpressionContext, NodeTemplate);
					
					int32 InitialErrorCount = MessageLog.NumErrors;
					// connect this fragment to its children fragments
					int32 PinWireIndex = 0;
					for (auto PinIt = FunctionCall->Pins.CreateConstIterator(); PinIt; ++PinIt)
					{
						UEdGraphPin* InputPin = *PinIt;
						if (!K2Schema->IsMetaPin(*InputPin) && (InputPin->Direction == EGPD_Input))
						{
							if (PinWireIndex < ArgumentList.Num())
							{
								TSharedPtr<FCodeGenFragment>& ArgumentNode = ArgumentList[PinWireIndex];
								// try to make the connection (which might cause an error internally)
								if (!ArgumentNode->ConnectToInput(InputPin, MessageLog))
								{
									FText ErrorText = FText::Format(LOCTEXT("ConnectPinError", "Failed to connect parameter #{0} with input on '@@'"),
										FText::AsNumber(PinWireIndex + 1));
									MessageLog.Error(*ErrorText.ToString(), FunctionCall);
								}
							}
							else if (InputPin->DefaultValue.IsEmpty()) // there is an ErrorTolerance parameter with a default value in EqualEqual_VectorVector
							{
								// too many pins - shouldn't be possible due to the checking in FindMatchingFunction() above
								FText ErrorText = LOCTEXT("ConnectPinError_RequiresMoreParameters", "The '@@' function requires more parameters than were provided");
								MessageLog.Error(*ErrorText.ToString(), FunctionCall);
								break;
							}
							++PinWireIndex;
						}
					}
					
					bool bConnectionErrors = (InitialErrorCount < MessageLog.NumErrors);
					if (bConnectionErrors)
					{
						MessageLog.Error(*LOCTEXT("InternalExpressionError", "Internal node error for expression: '@@'").ToString(), CompilingNode);
					}
					
					FunctionFragment = MakeShareable(new FCodeGenFragment_FuntionCall(FunctionCall, ReturnType));
				}
				else
				{
					FText ErrorText = FText::Format(LOCTEXT("ReturnTypeError", "The '{0}' function's return type is incompatible with graph pins in the expression: '@@'"),
						FText::FromString(FunctionName));
					MessageLog.Error(*ErrorText.ToString(), CompilingNode);
				}
			}
		}
		else
		{
			FText ErrorText = FText::Format(LOCTEXT("OperatorParamsError", "Cannot find a '{0}' function that takes the supplied param types, for expression: '@@'"),
				FText::FromString(FunctionName));
			MessageLog.Error(*ErrorText.ToString(), CompilingNode);
		}
		
		return FunctionFragment;
	}
	
	/**
	 * Utility method to turn an FLayoutVisitor coordinate into graph coordinates.
	 * FLayoutVisitor coordinates are in terms of nodes (so a depth of 1, would
	 * mean one node to the right of the initial node).
	 * 
	 * @param  Depth	Horizontal coordinate (how many nodes away from the initial node).
	 * @param  Height	Vertical coordinate (how many nodes down from the initial node).
	 * @return A 2D graph position for the center of a node to be placed at.
	 */
	FVector2D GetNodePosition(int32 Depth, int32 Height) const
	{
		// get a count of how many nodes there are at this specific depth
		int32 TotalHeight = LayoutMapper.DepthHeightLookup.FindRef(LayoutMapper.GetMaximumDepth() - Depth);

		const float MiddleHeight  = FMath::Max(TotalHeight, 1) * 0.5f;
		const float HeightPerNode = 140.0f;
		const float DepthPerNode  = 240.0f;

		return FVector2D(Depth * DepthPerNode, (Height - MiddleHeight + 0.5f) * HeightPerNode);
	}

	/**
	 * Templatized function for turning an expression node into a UK2Node. This
	 * takes the expression node's position in the expression tree and turns it 
	 * into a blueprint graph position (placing the new UK2Node there).
	 * 
	 * @param  ForExpression	The expression node that the will UK2Node represent.
	 * @param  Template			An instance of the UK2Node type you wish to spawn.
	 * @return The newly created UK2Node.
	 */
	template<typename NodeType>
	NodeType* SpawnNodeFromTemplate(IFExpressionNode* ForExpression, NodeType* Template)
	{
		const int32 Y = LayoutMapper.HeightChart.FindRef(ForExpression);
		const int32 X = LayoutMapper.GetMaximumDepth() - LayoutMapper.DepthChart.FindRef(ForExpression);
		
		GraphXBounds.X = FMath::Min((int32)GraphXBounds.X, X);
		GraphXBounds.Y = FMath::Max((int32)GraphXBounds.Y, X);

		constexpr bool bShouldSelectNewNode = false;
		const FVector2D Location = GetNodePosition(X, Y);
		return FEdGraphSchemaAction_K2NewNode::SpawnNodeFromTemplate<NodeType>(CompilingNode->BoundGraph, Template, Location, bShouldSelectNewNode);
	}

private:
	/** The node that we're generating sub-nodes and pins for */
	UK2Node_MathExpression* CompilingNode;

	/** 
	 * The blueprint that CompilingNode belongs to (the blueprint this will 
	 * generate a graph for).
	 */
	UBlueprint* TargetBlueprint;

	/** List of known operators, and mappings from them to associated functions */
	FOperatorTable OperatorLookup;

	/** 
	 * An FLayoutVisitor that charts the depth of the expression tree (and what
	 * depth/height each expression node is at). Used to layout the graph nicely.
	 */
	FLayoutVisitor LayoutMapper;
	
	/**
	 * Supplements LayoutMapper, tracks where nodes were actually placed (sometimes
	 * the depth of an expression node doesn't map one-to-one with the fragment
	 * in the graph), so you have the min and max x locations of spawned graph nodes.
	 */
	FVector2D GraphXBounds;

	/**
	 * Fragments that represent spawned UK2Nodes or literals that were generated
	 * from traversing the expression tree... These fragments facilitate
	 * connections between each other (that's why we need to track them).
	 */
	TMap<IFExpressionNode*, TSharedPtr<FCodeGenFragment> > CompiledFragments;
	
	/** 
	 * Used so the various Visit() methods have a way to log errors, null when 
	 * not in the middle of GenerateCode().
	 */
	FCompilerResultsLog* ActiveMessageLog;

	/** 
	 * After the code generation, we want to clear any old pins that 
	 * weren't reused, so here we track the ones in use.
	 */
	TArray<FName> InputPinNames;
};

/*******************************************************************************
 * FExpressionParser
*******************************************************************************/

#define PARSE_HELPER_BEGIN(NestedRuleName) \
	TSharedRef<IFExpressionNode> LHS = NestedRuleName(); \
	Begin:

#define PARSE_HELPER_ENTRY(NestedRuleName, DesiredToken) \
	if (IsValid() && MatchSymbol(DesiredToken)) \
	{ \
		TSharedRef<IFExpressionNode> RHS = NestedRuleName(); \
		LHS = MakeShareable(new FBinaryOperator(DesiredToken, LHS, RHS)); \
		goto Begin; \
	} \

#define PARSE_HELPER_END \
	{ return LHS; }

/**
 * Recursively builds an IFExpressionNode tree, where leaf nodes represent tokens 
 * (constants, literals, or identifiers), and branch nodes represent operations
 * on the attached children. The chaining order of expression functions is what 
 * determines operator precedence.
 */
class FExpressionParser : public FBasicTokenParser
{
public:
	/**
	 * Takes a string and parses a mathematical expression out of it, returning 
	 * the head of an expression tree that was generated as a result.
	 * 
	 * @param  InExpression		The string you wish to parse.
	 * @return An expression node, which serves as the root of an expression tree representing the provided string.
	 */
	TSharedRef<IFExpressionNode> ParseExpression(FString InExpression)
	{
		ExpressionString = InExpression;
		ResetParser(*ExpressionString);

		TSharedRef<IFExpressionNode> FullExpression = Expression();
		// if we didn't parse the full expression and the parser doesn't have 
		// an error, then there is some unhandled string postfixed to the 
		// expression (something like "2.x" or "5var")
		if (InputPos < InputLen && IsValid())
		{
			FText ErrorText = FText::Format(LOCTEXT("UnhandledPostfixError", "Unhandled trailing '{0}' at the end of the expression"), 
				FText::FromString(&Input[InputPos]));
			SetError(FErrorState::ParseError, ErrorText);
		}

		return FullExpression;
	}

private:
	/**
	 * Starting point for parsing full expressions (sets off on parsing out 
	 * operations according to operator precedence)... Could be used for the 
	 * initial root expression, or various other sub-expressions (like those 
	 * encapsulated in parentheses, etc.). 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> Expression()
	{
		// AssignmentExpression has the lowest precedence, start with it (it 
		// will attempt to parse out higher precedent operations first)
		return AssignmentExpression();
	}

	/**
	 * Intended to support assignment within the expression (setting temp or 
	 * external variables equal to some value, so they can be used later in the 
	 * expression).
	 * 
	 * @TODO   Implement!
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> AssignmentExpression()
	{
		// ConditionalExpression takes precedence over a assignment operation, parse it first
		return ConditionalExpression();
	}

	/**
	 * Looks for a conditional ternary statement (c ? a : b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> ConditionalExpression()
	{
		// LogicalOrExpression takes precedence over a conditional operation, parse it first
		TSharedRef<IFExpressionNode> MainPart = LogicalOrExpression();

		if (IsValid() && MatchSymbol(TEXT("?")))
		{
			TSharedRef<IFExpressionNode> TruePart = Expression();
			RequireSymbol(TEXT(":"), TEXT("?: operator"));
			TSharedRef<IFExpressionNode> FalsePart = ConditionalExpression();

			return MakeShareable(new FConditionalOperator(MainPart, TruePart, FalsePart));
		}
		else
		{
			return MainPart;
		}
	}

	/**
	 * Looks for a binary logical-or statement (a || b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> LogicalOrExpression()
	{
		// LogicalOrExpression takes precedence over an or operation, parse it first
		PARSE_HELPER_BEGIN(LogicalAndExpression) 
		PARSE_HELPER_ENTRY(LogicalAndExpression, TEXT("||"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary logical-and statement (a && b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> LogicalAndExpression()
	{
		// InclusiveOrExpression takes precedence over an and operation, parse it first
		PARSE_HELPER_BEGIN(InclusiveOrExpression) 
		PARSE_HELPER_ENTRY(InclusiveOrExpression, TEXT("&&"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary bitwise-or statement (a | b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> InclusiveOrExpression()
	{
		// ExclusiveOrExpression takes precedence over an inclusive or operation, parse it first
		PARSE_HELPER_BEGIN(ExclusiveOrExpression) 
		PARSE_HELPER_ENTRY(ExclusiveOrExpression, TEXT("|"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary exclusive-or statement (a ^ b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> ExclusiveOrExpression()
	{
		// AndExpression takes precedence over an exclusive or operation, parse it first
		PARSE_HELPER_BEGIN(AndExpression) 
		PARSE_HELPER_ENTRY(AndExpression, TEXT("^"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary bitwise-and statement (a & b) to parse, and 
	 * tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> AndExpression()
	{
		// EqualityExpression takes precedence over an and operation, parse it first
		PARSE_HELPER_BEGIN(EqualityExpression)
		PARSE_HELPER_ENTRY(EqualityExpression, TEXT("&"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary equality statement (like [a == b], or [a != b]) to 
	 * parse, and tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> EqualityExpression()
	{
		// RelationalExpression takes precedence over an equality expression, parse it first
		PARSE_HELPER_BEGIN(RelationalExpression)
		PARSE_HELPER_ENTRY(RelationalExpression, TEXT("=="))
		PARSE_HELPER_ENTRY(RelationalExpression, TEXT("!="))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary comparison statement to parse (like [a > b], [a <= b], 
	 * etc.), and tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> RelationalExpression()
	{
		// ShiftExpression takes precedence over a relational expression, parse it first
		PARSE_HELPER_BEGIN(ShiftExpression)
		PARSE_HELPER_ENTRY(ShiftExpression, TEXT("<"))
		PARSE_HELPER_ENTRY(ShiftExpression, TEXT(">"))
		PARSE_HELPER_ENTRY(ShiftExpression, TEXT("<="))
		PARSE_HELPER_ENTRY(ShiftExpression, TEXT(">="))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary bitwise shift statement to parse (like [a << b], or 
	 * [a >> b]), and tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> ShiftExpression()
	{
		// AdditiveExpression takes precedence over a shift, parse it first
		PARSE_HELPER_BEGIN(AdditiveExpression)
		PARSE_HELPER_ENTRY(AdditiveExpression, TEXT("<<"))
		PARSE_HELPER_ENTRY(AdditiveExpression, TEXT(">>"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary addition/subtraction statement to parse ([a + b], or 
	 * [a - b]), and tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> AdditiveExpression()
	{
		// MultiplicativeExpression takes precedence over an add/subtract, parse it first
		PARSE_HELPER_BEGIN(MultiplicativeExpression)
		PARSE_HELPER_ENTRY(MultiplicativeExpression, TEXT("+"))
		PARSE_HELPER_ENTRY(MultiplicativeExpression, TEXT("-"))
		PARSE_HELPER_END
	}

	/**
	 * Looks for a binary multiplication/division/modulus statement to parse 
	 * ([a * b], [a / b], or [a % b]), and tokenizes the operands. 
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> MultiplicativeExpression()
	{
		// CastExpression takes precedence over a multiply/division/modulus, parse it first
		PARSE_HELPER_BEGIN(CastExpression)
		PARSE_HELPER_ENTRY(CastExpression, TEXT("*"))
		PARSE_HELPER_ENTRY(CastExpression, TEXT("/"))
		PARSE_HELPER_ENTRY(CastExpression, TEXT("%"))
		PARSE_HELPER_END
	}

	/**
	 * Intended to handle type-casts (like from double to int, etc.).
	 * 
	 * @TODO   Implement!
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> CastExpression()
	{
		// @TODO: support casts (currently this is too greedy, and messes up "4*(5)" interpreting (5) as a cast)
		// if (MatchSymbol(TEXT("(")))
		// {
		// 	//@TODO: Need to support qualifiers / typedefs / what have you
		// 	FBasicToken TypeName;
		// 	GetToken(TypeName);
		// 	TSharedRef<IFExpressionNode> TypeExpression = MakeShareable(new FTokenWrapperNode(TypeName));
		// 
		// 	RequireSymbol(TEXT(")"), TEXT("Closing ) in cast"));
		// 
		// 	TSharedRef<IFExpressionNode> ValueExpression = CastExpression(Context);
		// 			
		// 	return MakeShareable(new FCastOperator(TypeExpression, ValueExpression));
		// }
		// else
		{
			return UnaryExpression();
		}
	}

	/**
	 * Attempts to parse various unary statements (like positive/negative 
	 * markers, logical negation, pre increment/decrement, etc.)
	 * 
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> UnaryExpression()
	{
		// 	prefix increment:    ++<unary-expression> 
		// 	prefix decrement:    --<unary-expression> 
		// 	bitwise not:          ~<unary-expression> 
		// 	logical not:          !<unary-expression> 
		// 	positive sign:        +<unary-expression> 
		// 	negative sign:        -<unary-expression> 
		// 	reference:            &<unary-expression> 
		// 	dereference:          *<unary-expression> 
		// 	negative sign:        -<unary-expression> 
		// 	allocation:        new <unary-expression> 
		// 	deallocation:   delete <unary-expression> 
		// 	parameter pack: sizeof <unary-expression> 
		// 	C-style cast:   (type) <unary-expression> 

		//
		// check for the various prefix operators and jump back to 
		// CastExpression() for parsing the right operand...

		if (MatchSymbol(TEXT("&")))
		{
			return MakeShareable(new FUnaryOperator(TEXT("&"), CastExpression()));
		}
		else if (MatchSymbol(TEXT("+")))
		{
			// would return pre-increment operators like so: 
			//		unaryOp(+).RHS = unaryOp(+)
			return MakeShareable(new FUnaryOperator(TEXT("+"), CastExpression()));
		}
		else if (MatchSymbol(TEXT("-")))
		{
			// would return post-increment operators like so: 
			//		unaryOp(-).RHS = unaryOp(-)
			return MakeShareable(new FUnaryOperator(TEXT("-"), CastExpression()));
		}
		else if (MatchSymbol(TEXT("~")))
		{
			return MakeShareable(new FUnaryOperator(TEXT("~"), CastExpression()));
		}
		else if (MatchSymbol(TEXT("!")))
		{
			return MakeShareable(new FUnaryOperator(TEXT("!"), CastExpression()));
		}
		else
		{
			return PostfixExpression();
		}
	}

	/**
	 * Intended to handle postfix operations (like post increment/decrement, 
	 * array subscripting, member access, etc.).
	 * 
	 * @TODO   Implement!
	 * @return Root node of an expression tree that was generated from where we 
	 *         are in parsing ExpressionString (at time of calling).
	 */
	TSharedRef<IFExpressionNode> PostfixExpression()
	{
		// if (MatchSymbol(TEXT("[")))
		// {
		// 	// Array indexing
		// 	TSharedRef<IFExpressionNode> IndexExpression = Expression(Context);
		// 	RequireSymbol(TEXT("]"), TEXT("Closing ] in array indexing"));
		// 
		// 	return NullExpression; // @TODO: return a valid expression node
		// }
		// else if (MatchSymbol(TEXT("(")))
		// {
		// 	while (!PeekSymbol(TEXT(")")))
		// 	{
		// 		TSharedRef<IFExpressionNode> Item = AssignmentExpression(Context);
		// 	}
		// 
		// 	RequireSymbol(TEXT(")"), TEXT("Closing ) in function call"));
		// 
		// 	return NullExpression; // @TODO: return a valid expression node
		// }
		// else if (MatchSymbol(TEXT(".")) || MatchSymbol(TEXT("->")))
		// {
		// 	// Member reference
		// 	FBasicToken Identifier;
		// 	if (GetIdentifier(Identifier, /*bNoConsts=*/ true))
		// 	{
		// 		//@TODO: Do stuffs
		// 	}
		// 	else
		// 	{
		// 		// @TODO: error
		// 	}
		// 
		// 	return NullExpression; // @TODO: return a valid expression node
		// }
		// else
		{
			return PrimaryExpression();
		}
	}

	/**
	 * End of the line, where we attempt to generate a leaf node (an identifier,
	 * const literal, or a string). However, here we also look for the start of 
	 * a sub-expression (one encapsulated in parentheses).
	 * 
	 * @return Either a leaf node (representing a variable, or literal), or another 
	 *         branch node, representing a sub-expression encapsulated by parentheses.
	 */
	TSharedRef<IFExpressionNode> PrimaryExpression()
	{
		if (MatchSymbol(TEXT("(")))
		{
			TSharedRef<IFExpressionNode> Result = Expression();
			RequireSymbol(TEXT(")"), TEXT("group closing"));
			return Result;
		}
		else
		{
			// identifier, constant, or string
			FBasicToken Token;
			GetToken(Token);
			
			// or maybe a function call?
			if (MatchSymbol(TEXT("(")))
			{
				TSharedPtr<FExpressionList> FuncArguments;
				// if this is an empty function (takes no parameters)
				if (PeekSymbol(TEXT(")")))
				{
					FuncArguments = MakeShareable(new FExpressionList);
				}
				else
				{
					FuncArguments = ListExpression();
				}
				
				TSharedRef<IFExpressionNode> FuncExpression = MakeShareable(new FFunctionExpression(Token.Identifier, FuncArguments.ToSharedRef()));
				
				FText RequireError = FText::Format(LOCTEXT("MissingFuncClose", "'{0}' closing"), FText::FromString(Token.Identifier));
				RequireSymbol(TEXT(")"), *RequireError.ToString());
				
				return FuncExpression;
			}
			else
			{
				return MakeShareable(new FTokenWrapperNode(Token));
			}
		}
	}
    
	/**
	 * Parses out a comma separated list of sub-expressions (arguments for a 
	 * function or struct).
	 *
	 * @return A branch FExpressionList node, which holds a series of sub-expressions.
	 */
    TSharedRef<FExpressionList> ListExpression()
	{
		TSharedRef<FExpressionList> ListNode = MakeShareable(new FExpressionList);	
		do 
		{
			ListNode->Children.Add(Expression());
		} while (MatchSymbol(TEXT(",")));

		return ListNode;
	}

private:
	/** The intact expression string that this is currently in charge of parsing */
	FString ExpressionString;
};

/*******************************************************************************
 * UK2Node_MathExpression
*******************************************************************************/

//------------------------------------------------------------------------------
UK2Node_MathExpression::UK2Node_MathExpression(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// renaming the node rebuilds the expression (the node name is where they 
	// specify the math equation)
	bCanRenameNode = true;

	bMadeAfterRotChange = false;
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAll;
}

void UK2Node_MathExpression::Serialize(FArchive& Ar)
{
	UK2Node_Composite::Serialize(Ar);

	if (Ar.IsLoading() && !bMadeAfterRotChange)
	{
		// remember that this logic has been run, we only want to run it once:
		bMadeAfterRotChange = true;

		// We need to reorder the parameters to MakeRot/MakeRotator/Rotator/Rot, to filter this expensive logic I'm just searching
		// expressions for 'rot':
		if (Expression.Contains(TEXT("Rot")))
		{
			// Now parse the expression and look for function expressions to the old MakeRot function:
			FExpressionParser Parser;
			TSharedPtr<IFExpressionNode> ExpressionRoot = Parser.ParseExpression(Expression);
			
			struct FMakeRotFixupVisitor : public FExpressionVisitor
			{
				virtual bool Visit(class FFunctionExpression& Node, EVisitPhase Phase)
				{ 
					if (Phase != VISIT_Pre)
					{
						return false;
					}

					const bool bIsMakeRot = (Node.FuncName == TEXT("makerot"));
					if (bIsMakeRot ||
						Node.FuncName == TEXT("rotator") ||
						Node.FuncName == TEXT("rot"))
					{
						// reorder parameters to match new order of MakeRotator:
						if (Node.ParamList->Children.Num() == 3)
						{
							TSharedRef<IFExpressionNode> OldPitch = Node.ParamList->Children[0];
							TSharedRef<IFExpressionNode> OldYaw = Node.ParamList->Children[1];
							TSharedRef<IFExpressionNode> OldRoll = Node.ParamList->Children[2];
							
							Node.ParamList->Children[0] = OldRoll;
							Node.ParamList->Children[1] = OldPitch;
							Node.ParamList->Children[2] = OldYaw;
						}

						// MakeRot also needs to be updated to the new name:
						if (bIsMakeRot)
						{
							Node.FuncName = TEXT("MakeRotator");
						}
					}
					return true;
				}
			};

			// perform the update:
			FMakeRotFixupVisitor Fixup;
			ExpressionRoot->Accept(Fixup);

			// reform the expression with the updated parameter order/function names:
			Expression = ExpressionRoot->ToString();
		}
	}
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UK2Node_MathExpression, Expression))
	{
		RebuildExpression(Expression);
	}
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}


//------------------------------------------------------------------------------
FNodeHandlingFunctor* UK2Node_MathExpression::CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MathExpression(CompilerContext);
}

//------------------------------------------------------------------------------
bool UK2Node_MathExpression::ShouldExpandInsteadCompile() const
{
	const int32 TunnelNodesNum = 2;
	if (!BoundGraph || (TunnelNodesNum >= BoundGraph->Nodes.Num()))
	{
		return true;
	}

	if ((TunnelNodesNum + 1) == BoundGraph->Nodes.Num())
	{
		TArray<UEdGraphNode*> InnerNodes = BoundGraph->Nodes;
		InnerNodes.RemoveSingleSwap(GetEntryNode(), EAllowShrinking::No);
		InnerNodes.RemoveSingleSwap(GetExitNode(), EAllowShrinking::No);
		const bool bTheOnlyNodeIsNotAFunctionCall = (1 == InnerNodes.Num())
			&& (nullptr != InnerNodes[0])
			&& !InnerNodes[0]->IsA<UK2Node_CallFunction>();
		if (bTheOnlyNodeIsNotAFunctionCall)
		{
			return true;
		}
	}

	return false;
}

//------------------------------------------------------------------------------
TSharedPtr<class INameValidatorInterface> UK2Node_MathExpression::MakeNameValidator() const
{
	// we'll let our parser mark the node for errors after the face (once the 
	// name is submitted)... parsing it with every character could be slow
	return MakeShareable(new FDummyNameValidator(EValidatorResult::Ok));
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::OnRenameNode(const FString& NewName)
{
	RebuildExpression(NewName);
	CachedNodeTitle.MarkDirty();
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::RebuildExpression(FString InExpression)
{
	static bool bIsAlreadyRebuilding = false;
	// the rebuild can invoke a ReconstructNode(), which triggers this again,
	// so this combined with the following 
	if (!bIsAlreadyRebuilding)
	{
		TGuardValue<bool> RecursionGuard(bIsAlreadyRebuilding, true);

		ClearExpression();
		Expression = InExpression;

		// This should not be sanitized, if anything fails to occur, what the user inputed should be what is displayed
		CachedDisplayExpression.SetCachedText(FText::FromString(Expression), this);
		CachedNodeTitle.SetCachedText(GetFullTitle(CachedDisplayExpression), this);

		if (!InExpression.IsEmpty()) // @TODO: is this needed?
		{
			// build a expression tree from the string
			FExpressionParser Parser;
			TSharedPtr<IFExpressionNode> ExpressionRoot = Parser.ParseExpression(InExpression);

			// if the parser successfully chewed through the string
			if (Parser.IsValid())
			{
				FMathGraphGenerator GraphGenerator(this);
				// generate new nodes from the expression tree (could result in
				// a series of errors being attached to the node).
				if (!GraphGenerator.GenerateCode(ExpressionRoot.ToSharedRef(), *CachedMessageLog))
				{
					if (CachedMessageLog->NumErrors == 0)
					{
						CachedMessageLog->Error(*LOCTEXT("MathExprGFailedGen", "Failed to generate full expression graph for: '@@'").ToString(), this);
					}
				}
				else
				{
					Expression = ExpressionRoot->ToString();
					CachedDisplayExpression.SetCachedText(FText::FromString(SanitizeDisplayExpression(ExpressionRoot->ToDisplayString(GetBlueprint()))), this);
					CachedNodeTitle.SetCachedText(GetFullTitle(CachedDisplayExpression), this);
				}

				if (UK2Node_Tunnel* EntryNode = GetEntryNode())
				{
					// iterate backwards so we can remove as we go... we want to 
					// clear any pins that weren't used by the expression (if we
					// clear any, then they were probably remnants from the last
					// expression... we can't delete them before, because the 
					// user may have mutated one for the new expression)
					for (int32 PinIndex = EntryNode->UserDefinedPins.Num() - 1; PinIndex >= 0; --PinIndex)
					{
						TSharedPtr<FUserPinInfo> PinInfo = EntryNode->UserDefinedPins[PinIndex];
						if (!GraphGenerator.IsPinInUse(PinInfo))
						{
							EntryNode->RemoveUserDefinedPin(PinInfo);
						}
					}					
				}
			}
			else
			{
				FText ErrorText = FText::Format(LOCTEXT("MathExprParseError", "PARSE ERROR in '@@': {0}"),
					Parser.GetErrorState().Description);
				CachedMessageLog->Error(*ErrorText.ToString(), this);
			}
		}

		// notify any listeners that the bound graph has changed
		if (BoundGraph)
		{
			BoundGraph->NotifyGraphChanged();
		}

		// refresh the node since the connections may have changed, this won't be reentrant due to bool above
		ReconstructNode();

		// finally, recompile
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNodeChecked(this);
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::ClearExpression()
{
	// clear any errors 
	SetNodeError(this, FText::GetEmpty());

	// clear out old nodes
	DeleteGeneratedNodesInGraph(BoundGraph);

	// delete the old return pins (they will always be regenerated)... save the 
	// input pins though (because someone may have changed the input type to 
	// something other than a float)
	if (UK2Node_Tunnel* ExitNode = GetExitNode())
	{
		// iterate backwards so we can remove as we go
		for (int32 PinIndex = ExitNode->UserDefinedPins.Num() - 1; PinIndex >= 0; --PinIndex)
		{
			TSharedPtr<FUserPinInfo> PinInfo = ExitNode->UserDefinedPins[PinIndex];
			ExitNode->RemoveUserDefinedPin(PinInfo);
		}
	}
	
	// passing true to FCompilerResultsLog's constructor would make this the
	// primary compiler log (it is not) - the idea being that upon destruction 
	// the primary log prints a summary; well, since this isn't destructed at 
	// the end of compilation, and it blocks the full compiler log from 
	// becoming the "CurrentEventTarget", we pass false - we append logs 
	// collected by this one to the full compiler log later on anyways (so they won't be missed)
	CachedMessageLog = MakeShareable(new FCompilerResultsLog(/*bIsCompatibleWithEvents =*/false));
	CachedMessageLog->bSilentMode = true;
	
	Expression.Empty();
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	if (CachedMessageLog.IsValid())
	{
		MessageLog.Append(*CachedMessageLog, true);
	}
	// else, this may be some intermediate node in the compile, let's look at the errors from the original...
	else 
	{
		if(const UObject* SourceObject = MessageLog.FindSourceObject(this))
		{
			const UK2Node_MathExpression* MathExpression = MessageLog.FindSourceObjectTypeChecked<UK2Node_MathExpression>(this);

			// Should always be able to find the source math expression
			check(MathExpression);

			// if the expressions match, then the errors should
			check(MathExpression->Expression == Expression);

			// take the same errors from the original node (so we don't have to
			// re-parse/re-gen to fish out the same errors)
			if (MathExpression->CachedMessageLog.IsValid())
			{
				MessageLog.Append(*MathExpression->CachedMessageLog, true);
			}
		}
	}
}

//------------------------------------------------------------------------------
FText UK2Node_MathExpression::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Expression.IsEmpty() && (TitleType == ENodeTitleType::MenuTitle))
	{
		return LOCTEXT("AddMathExprMenuOption", "Add Math Expression...");
	}
	else if (TitleType != ENodeTitleType::FullTitle)
	{
		if (CachedDisplayExpression.IsOutOfDate(this))
		{
			FExpressionParser Parser;
			TSharedPtr<IFExpressionNode> ExpressionRoot = Parser.ParseExpression(Expression);
			if(Parser.IsValid())
			{
				CachedDisplayExpression.SetCachedText(FText::FromString(SanitizeDisplayExpression(ExpressionRoot->ToDisplayString(GetBlueprint()))), this);
			}
			else
			{
				// Fallback and display the expression in it's raw form
				CachedDisplayExpression.SetCachedText(FText::FromString(Expression), this);
			}
		}
		return CachedDisplayExpression;
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FExpressionParser Parser;
		TSharedPtr<IFExpressionNode> ExpressionRoot = Parser.ParseExpression(Expression);

		if(Parser.IsValid())
		{
			CachedDisplayExpression.SetCachedText(FText::FromString(SanitizeDisplayExpression(ExpressionRoot->ToDisplayString(GetBlueprint()))), this);
		}
		CachedNodeTitle.SetCachedText(GetFullTitle(CachedDisplayExpression), this);
	}
	return CachedNodeTitle;
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::PostPlacedNewNode()
{
	bMadeAfterRotChange = true;
	Super::PostPlacedNewNode();
	FEdGraphUtilities::RenameGraphToNameOrCloseToName(BoundGraph, "MathExpression");
}

//------------------------------------------------------------------------------
void UK2Node_MathExpression::ReconstructNode()
{
	if (!HasAnyFlags(RF_NeedLoad))
	{
		RebuildExpression(Expression);
	}

	// Call the super ReconstructNode, preserving our error message since we never want it automatically cleared
	const FString OldErrorMessage = ErrorMsg;
	Super::ReconstructNode();
	ErrorMsg = OldErrorMessage;

	// Mark our input pins as not saved, but we want to save orphaned output pins to show compile errors
	for (UEdGraphPin* Pin : Pins)
	{
		// Recombine the sub pins back into the OptionPin
		if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
		{
			Pin->SetSavePinIfOrphaned(false);
		}
		else
		{
			Pin->SetSavePinIfOrphaned(true);
		}
	}
}

//------------------------------------------------------------------------------
FString UK2Node_MathExpression::SanitizeDisplayExpression(FString InExpression) const
{
	// We do not want the outermost parentheses in the display expression, they add nothing to the logical comprehension
	InExpression.RemoveFromStart(TEXT("("));
	InExpression.RemoveFromEnd(TEXT(")"));

	return InExpression;
}

FText UK2Node_MathExpression::GetFullTitle(FText InExpression) const
{
	// FText::Format() is slow, so we cache this to save on performance
	return FText::Format(LOCTEXT("MathExpressionSecondTitleLine", "{0}\nMath Expression"), InExpression);
}

void UK2Node_MathExpression::FindDiffs(class UEdGraphNode* OtherNode, struct FDiffResults& Results )
{
	UK2Node_MathExpression* MathExpression1 = this;
	UK2Node_MathExpression* MathExpression2 = Cast<UK2Node_MathExpression>(OtherNode);

	// Compare the visual display of a math expression (the visual display involves consolidating variable Guid's into readable parameters)
	FText Expression1 = MathExpression1->GetNodeTitle(ENodeTitleType::EditableTitle);
	FText Expression2 = MathExpression2->GetNodeTitle(ENodeTitleType::EditableTitle);
	if (Expression1.CompareTo(Expression2) != 0)
	{
		FDiffSingleResult Diff;
		Diff.Node1 = MathExpression2;
		Diff.Node2 = MathExpression1;

		Diff.Diff = EDiffType::NODE_PROPERTY;
		FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

		FFormatNamedArguments Args;
		Args.Add(TEXT("Expression1"), Expression1);
		Args.Add(TEXT("Expression2"), Expression2);

		Diff.ToolTip =  FText::Format(LOCTEXT("DIF_MathExpressionToolTip", "Math Expression '{Expression1}' changed to '{Expression2}'"), Args);
		Diff.Category = EDiffType::MODIFICATION;
		Diff.DisplayString = FText::Format(LOCTEXT("DIF_MathExpression", "Math Expression '{Expression1}' changed to '{Expression2}'"), Args);
		Results.Add(Diff);
	}
}

#undef LOCTEXT_NAMESPACE
