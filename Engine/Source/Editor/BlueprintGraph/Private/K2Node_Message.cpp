// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Message.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/LevelScriptActor.h"
#include "Engine/LevelStreaming.h"
#include "Engine/MemberReference.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_DynamicCast.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/BlueprintMapLibrary.h"
#include "Kismet/BlueprintSetLibrary.h"
#include "Kismet/KismetArrayLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "ObjectTools.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/Script.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Message)

#define LOCTEXT_NAMESPACE "K2Node_Message"

UK2Node_Message::UK2Node_Message(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UK2Node_Message::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeName;
	if (UFunction* Function = GetTargetFunction())
	{
		if (!CachedNodeTitles.IsTitleCached(TitleType, this))
		{
			FText NodeNameText = ObjectTools::GetUserFacingFunctionName(Function);
			if (TitleType == ENodeTitleType::MenuTitle)
			{
				// FText::Format() is slow, so we cache this to save on performance
				CachedNodeTitles.SetCachedTitle(TitleType, FText::Format(LOCTEXT("ListTitle", "{0} (Message)"), NodeNameText), this);
			}
			else
			{
				// For consistency, use the function call node title format
				FText NodeTitle = Super::GetNodeTitle(TitleType);

				// FText::Format() is slow, so we cache this to save on performance
				CachedNodeTitles.SetCachedTitle(TitleType, NodeTitle, this);
			}
		}
	}
	else
	{
		return NSLOCTEXT("K2Node", "InvalidMessageNode", "Invalid Message Node");
	}
	return CachedNodeTitles.GetCachedTitle(TitleType);
}

FText UK2Node_Message::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		CachedTooltip.SetCachedText(FText::Format(LOCTEXT("MessageTooltip", "{0}\nMessage. This does nothing if the target does not implement the required interface."), Super::GetTooltipText()), this);
	}
	return CachedTooltip;
}

UEdGraphPin* UK2Node_Message::CreateSelfPin(const UFunction* Function)
{
	UEdGraphPin* SelfPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), UEdGraphSchema_K2::PN_Self);
	SelfPin->bDefaultValueIsIgnored = true;
	return SelfPin;
}

void UK2Node_Message::FixupSelfMemberContext()
{
	// Do nothing; the function either exists and works, or doesn't and doesn't
}

FName UK2Node_Message::GetCornerIcon() const
{
	return TEXT("Graph.Message.MessageIcon");
}

FNodeHandlingFunctor* UK2Node_Message::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_Message::ExpandLevelStreamingHandlers(class FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph, UEdGraphPin* InStartingExecPin, UEdGraphPin* InMessageSelfPin, UK2Node_DynamicCast* InCastToInterfaceNode)
{
	const UEdGraphSchema_K2* Schema = InCompilerContext.GetSchema();
	
	/**
	 * Create intermediate nodes
	 */

	// Create a  GetLevelScriptActor CallFunction node, this will be used if the cast to ULevelStreaming was successful
	UK2Node_CallFunction* GetLevelScriptActorNode = InCompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, InSourceGraph);
	GetLevelScriptActorNode->SetFromFunction(ULevelStreaming::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(ULevelStreaming, GetLevelScriptActor)));
	GetLevelScriptActorNode->AllocateDefaultPins();

	UEdGraphPin* PinToCastToInterface = nullptr;
	{
		PinToCastToInterface = InMessageSelfPin;

		// Move all pin connections from the message self pin to the GetLevelScriptActorNode's self pin
		{
			InCompilerContext.MovePinLinksToIntermediate(*InMessageSelfPin, *Schema->FindSelfPin(*GetLevelScriptActorNode, EGPD_Input));

			// The last pin on the function node is the ALevelScriptActor
			UEdGraphPin* FuncResultPin = GetLevelScriptActorNode->Pins[GetLevelScriptActorNode->Pins.Num() - 1];
			ensure(!FuncResultPin->PinType.PinSubCategoryObject->IsA(ALevelScriptActor::StaticClass()));

			// We will want to cast the resulting level Blueprint to the appropriate interface
			PinToCastToInterface = FuncResultPin;
		}

		// Move all connections from the starting exec pin to the cast node
		InCompilerContext.MovePinLinksToIntermediate(*InStartingExecPin, *InCastToInterfaceNode->GetExecPin());
	}

	// Connect the Interface cast node to the generated pins
	{
		// The source pin of the Interface cast node connects to the temporary variable (storing either an ALevelScriptActor or another UObject)
		UEdGraphPin* CastToInterfaceSourceObjectPin = InCastToInterfaceNode->GetCastSourcePin();
		Schema->TryCreateConnection(PinToCastToInterface, CastToInterfaceSourceObjectPin);
	}
}

static bool IsLevelStreamingClass(UClass* InClass)
{
	if (InClass)
	{
		UClass* Parent = ULevelStreaming::StaticClass();
		return (Parent == InClass || InClass->IsChildOf(Parent));
	}
	return false;
}

void UK2Node_Message::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UEdGraphPin* ExecPin = Schema->FindExecutionPin(*this, EGPD_Input);
	const bool bExecPinConnected = ExecPin && (ExecPin->LinkedTo.Num() > 0);
	UEdGraphPin* ThenPin = Schema->FindExecutionPin(*this, EGPD_Output);
	const bool bThenPinConnected = ThenPin && (ThenPin->LinkedTo.Num() > 0);

	// Skip ourselves if our exec isn't wired up
	if (bExecPinConnected)
	{
		UClass* InterfaceClass = FunctionReference.GetMemberParentClass(GetBlueprintClassFromNode());

		// Make sure our interface is valid
		if (InterfaceClass == nullptr)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MessageNodeInvalid_Error", "Message node @@ has an invalid interface.").ToString(), this);
			return;
		}

		UFunction* MessageNodeFunction = GetTargetFunction();
		if (MessageNodeFunction == nullptr)
		{
			//@TODO: Why do this here in the compiler, it's already done on AllocateDefaultPins() during on-load node reconstruction
			MessageNodeFunction = FMemberReference::FindRemappedField<UFunction>(InterfaceClass, FunctionReference.GetMemberName());
		}

		if (MessageNodeFunction == nullptr)
		{
			CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("MessageNodeInvalidFunction_ErrorFmt", "Unable to find function with name {0} for Message node @@."), FText::FromString(FunctionReference.GetMemberName().ToString())).ToString(), this);
			return;
		}

		const bool bIsStaticFunc = MessageNodeFunction->HasAllFunctionFlags(FUNC_Static);
		if (!bIsStaticFunc && !InterfaceClass->HasAnyClassFlags(CLASS_Interface))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MessageNodeClassNotAnInterface_Error", "Message node @@ uses a class that isn't an interface.").ToString(), this);
			return;
		}

		// Check to make sure we have a target
		UEdGraphPin* MessageSelfPin = Schema->FindSelfPin(*this, EGPD_Input);

		// If we've intentionally hidden the 'self' pin (e.g. we're a static function), the pin can be redirected
		bool bSelfPinCanBeRedirected = MessageSelfPin->bHidden;
		if (bSelfPinCanBeRedirected)
		{
			// Build up a list of names that could correspond to this self pin (e.g. SomeClass.SomeFunc.self)
			TArray<FString> PossibleRedirectNames;
			GetRedirectPinNames(*MessageSelfPin, PossibleRedirectNames);

			// Check for any redirects, and if we have one, try to find that pin
			FName OutSelfPinName;
			ERedirectType RedirectType = ShouldRedirectParam(PossibleRedirectNames, OutSelfPinName, this);
			if (RedirectType != ERedirectType::ERedirectType_None)
			{
				MessageSelfPin = FindPin(OutSelfPinName, EEdGraphPinDirection::EGPD_Input);
			}
		}

		if( !MessageSelfPin || MessageSelfPin->LinkedTo.Num() == 0 )
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MessageNodeSelfPin_Error", "Message node @@ must have a valid target or reference to self.").ToString(), this);
			return;
		}

		// First, create an internal cast-to-interface node
		UK2Node_DynamicCast* CastToInterfaceNode = CompilerContext.SpawnIntermediateNode<UK2Node_DynamicCast>(this, SourceGraph);
		CastToInterfaceNode->TargetType = MessageNodeFunction->GetOuterUClass()->GetAuthoritativeClass();
		if (bSelfPinCanBeRedirected)
		{
			if (UObject* PinSubCategoryObj = MessageSelfPin->PinType.PinSubCategoryObject.Get())
			{
				UClass* TargetTypeClass = Cast<UClass>(PinSubCategoryObj);
				if (!TargetTypeClass)
				{
					TargetTypeClass = PinSubCategoryObj->GetClass();
				}
				CastToInterfaceNode->TargetType = TargetTypeClass->GetAuthoritativeClass();
			}
		}
		CastToInterfaceNode->SetPurity(false);
		CastToInterfaceNode->AllocateDefaultPins();

		UEdGraphPin* CastToInterfaceResultPin = CastToInterfaceNode->GetCastResultPin();
		if( !CastToInterfaceResultPin )
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("InvalidInterfaceClass_Error", "Node @@ has an invalid target interface class").ToString(), this);
			return;
		}

		// If the message pin is linked and what it is linked to is not a ULevelStreaming pin reference but is a child of ULevelStreaming, 
		// then we need to leave the possibility that it could be a ULevelStreaming and will need to make an attempt at casting to one
		if (MessageSelfPin->LinkedTo.Num() > 0 && IsLevelStreamingClass(Cast<UClass>(MessageSelfPin->LinkedTo[0]->PinType.PinSubCategoryObject.Get())))
		{
			ExpandLevelStreamingHandlers(CompilerContext, SourceGraph, ExecPin, MessageSelfPin, CastToInterfaceNode);
		}
		else
		{
			// Move the connections on the Message node's self pin to the ULevelStreaming Cast node's source pin
			{
				CompilerContext.MovePinLinksToIntermediate(*MessageSelfPin, *CastToInterfaceNode->GetCastSourcePin());
				CastToInterfaceNode->NotifyPinConnectionListChanged(CastToInterfaceNode->GetCastSourcePin());
			}

			// Connect the incoming exec pin to the ULevelStreaming cast node's exec pin, which is the exec flow's entry into this
			if (ExecPin != nullptr)
			{
				// Wire up the connections
				CompilerContext.MovePinLinksToIntermediate(*ExecPin, *CastToInterfaceNode->GetExecPin());
			}
		}

		CastToInterfaceResultPin->PinType.PinSubCategoryObject = *CastToInterfaceNode->TargetType;

		// Next, create the function call node
		UK2Node_CallFunction* FunctionCallNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
		FunctionCallNode->FunctionReference = FunctionReference;
		FunctionCallNode->AllocateDefaultPins();

		UEdGraphPin* CastToInterfaceValidPin = CastToInterfaceNode->GetValidCastPin();
		check(CastToInterfaceValidPin != nullptr);

		UEdGraphPin* LastOutCastFaildPin   = CastToInterfaceNode->GetInvalidCastPin();
		check(LastOutCastFaildPin != nullptr);
		UEdGraphPin* LastOutCastSuccessPin = CastToInterfaceValidPin;

		// Wire up the connections
		if (UEdGraphPin* CallFunctionExecPin = Schema->FindExecutionPin(*FunctionCallNode, EGPD_Input))
		{
			// Connect the CallFunctionExecPin to both Assignment nodes for the assignment of the UObject cast
			CallFunctionExecPin->MakeLinkTo(CastToInterfaceValidPin);
			LastOutCastSuccessPin = Schema->FindExecutionPin(*FunctionCallNode, EGPD_Output);
		}
		
		// Function's actual 'self' pin (since Self Pin can be redirected, let's use its matching name)
		UEdGraphPin* FunctionCallSelfPin = FunctionCallNode->FindPin(MessageSelfPin->GetName(), EGPD_Input);
		if (!ensureMsgf(FunctionCallSelfPin, TEXT("Could not find suitable SelfPin '%s' matching input pin on (intermediate) node %s"), *MessageSelfPin->GetName(), *FunctionCallNode->GetDescriptiveCompiledName()))
		{
			// Fallback to old code with hardcoded Self name
			FunctionCallSelfPin = Schema->FindSelfPin(*FunctionCallNode, EGPD_Input);
		}
		CastToInterfaceResultPin->MakeLinkTo(FunctionCallSelfPin);

		UFunction* ArrayClearFunction = UKismetArrayLibrary::StaticClass()->FindFunctionByName(FName(TEXT("Array_Clear")));
		check(ArrayClearFunction);
		UFunction* SetClearFunction = UBlueprintSetLibrary::StaticClass()->FindFunctionByName(FName(TEXT("Set_Clear")));
		check(SetClearFunction);
		UFunction* MapClearFunction = UBlueprintMapLibrary::StaticClass()->FindFunctionByName(FName(TEXT("Map_Clear")));
		check(MapClearFunction);

		const bool bIsPureMessageFunc = Super::IsNodePure();

		// Variable pins - Try to associate variable inputs to the message node with the variable inputs and outputs to the call function node
		for( int32 i = 0; i < Pins.Num(); i++ )
		{
			UEdGraphPin* CurrentPin = Pins[i];
			if( CurrentPin && (CurrentPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec) && (CurrentPin->PinName != UEdGraphSchema_K2::PN_Self) )
			{
				// Try to find a match for the pin on the function call node
				UEdGraphPin* FunctionCallPin = FunctionCallNode->FindPin(CurrentPin->PinName);
				if( FunctionCallPin )
				{
					// Move pin links if the pin is connected...
					CompilerContext.MovePinLinksToIntermediate(*CurrentPin, *FunctionCallPin);

					// when cast fails all return values must be cleared.
					if (EEdGraphPinDirection::EGPD_Output == CurrentPin->Direction)
					{
						UEdGraphPin* VarOutPin = FunctionCallPin;
						if (bIsPureMessageFunc)
						{
							// since we cannot directly use the output from the
							// function call node (since it is pure, and invoking
							// it with a null target would cause an error), we 
							// have to use a temporary variable in it's place...
							UK2Node_TemporaryVariable* TempVar = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
							TempVar->VariableType = CurrentPin->PinType;
							TempVar->AllocateDefaultPins();

							VarOutPin = TempVar->GetVariablePin();
							// nodes using the function's outputs directly, now
							// use this TempVar node instead
							CompilerContext.MovePinLinksToIntermediate(*FunctionCallPin, *VarOutPin);

							// on a successful cast, the temp var is filled with
							// the function's value, on a failed cast, the var 
							// is filled with a default value (DefaultValueNode, 
							// below)... this is the node for the success case:
							UK2Node_AssignmentStatement* AssignTempVar = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
							AssignTempVar->AllocateDefaultPins();
							
							// assign the output from the pure function node to
							// the TempVar (either way, this message node is 
							// returning the TempVar's value, so on a successful 
							// cast we want it to have the function's result)
							UEdGraphPin* ValueInPin = AssignTempVar->GetValuePin();
							Schema->TryCreateConnection(FunctionCallPin, ValueInPin);
							AssignTempVar->PinConnectionListChanged(ValueInPin);

							UEdGraphPin* VarInPin = AssignTempVar->GetVariablePin();
							Schema->TryCreateConnection(VarOutPin, VarInPin);
							AssignTempVar->PinConnectionListChanged(VarInPin);
							// fold this AssignTempVar node into the cast's 
							// success execution chain
							Schema->TryCreateConnection(AssignTempVar->GetExecPin(), LastOutCastSuccessPin);
							LastOutCastSuccessPin = AssignTempVar->GetThenPin();
						}

						UK2Node* DefaultValueNode = nullptr;
						UEdGraphPin* DefaultValueThenPin = nullptr;
						if (CurrentPin->PinType.IsArray())
						{
							UK2Node_CallArrayFunction* ClearArray = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
							DefaultValueNode = ClearArray;
							ClearArray->SetFromFunction(ArrayClearFunction);
							ClearArray->AllocateDefaultPins();

							UEdGraphPin* ArrayPin = ClearArray->GetTargetArrayPin();
							check(ArrayPin);
							Schema->TryCreateConnection(ArrayPin, VarOutPin);
							ClearArray->PinConnectionListChanged(ArrayPin);

							DefaultValueThenPin = ClearArray->GetThenPin();
						} 
						else if(CurrentPin->PinType.IsSet())
						{
							UK2Node_CallFunction* ClearSet = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
							DefaultValueNode = ClearSet;
							ClearSet->SetFromFunction(SetClearFunction);
							ClearSet->AllocateDefaultPins();

							UEdGraphPin* SetPin = FEdGraphUtilities::FindSetParamPin(SetClearFunction, ClearSet);
							Schema->TryCreateConnection(SetPin, VarOutPin);
							ClearSet->PinConnectionListChanged(SetPin);

							DefaultValueThenPin = ClearSet->GetThenPin();
						}
						else if(CurrentPin->PinType.IsMap())
						{
							UK2Node_CallFunction* ClearMap = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
							DefaultValueNode = ClearMap;
							ClearMap->SetFromFunction(MapClearFunction);
							ClearMap->AllocateDefaultPins();

							UEdGraphPin* MapPin = FEdGraphUtilities::FindMapParamPin(MapClearFunction, ClearMap);
							Schema->TryCreateConnection(MapPin, VarOutPin);
							ClearMap->PinConnectionListChanged(MapPin);

							DefaultValueThenPin = ClearMap->GetThenPin();
						}
						else
						{
							UK2Node_AssignmentStatement* AssignDefaultValue = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
							DefaultValueNode = AssignDefaultValue;
							AssignDefaultValue->AllocateDefaultPins();

							Schema->TryCreateConnection(AssignDefaultValue->GetVariablePin(), VarOutPin);
							AssignDefaultValue->PinConnectionListChanged(AssignDefaultValue->GetVariablePin());
							Schema->SetPinAutogeneratedDefaultValueBasedOnType(AssignDefaultValue->GetValuePin());

							DefaultValueThenPin = AssignDefaultValue->GetThenPin();
						}

						UEdGraphPin* DefaultValueExecPin = DefaultValueNode->GetExecPin();
						check(DefaultValueExecPin);
						Schema->TryCreateConnection(DefaultValueExecPin, LastOutCastFaildPin);
						LastOutCastFaildPin = DefaultValueThenPin;
						check(LastOutCastFaildPin);
					}
				}
				else
				{
					UE_LOGF(LogK2Compiler, Log, "%ls", *LOCTEXT("NoPinConnectionFound_Error", "Unable to find connection for pin!  Check AllocateDefaultPins() for consistency!").ToString());
				}
			}
		}

		if( bThenPinConnected )
		{
			check(LastOutCastFaildPin != nullptr);
			// Failure case for the cast runs straight through to the exit
			CompilerContext.CopyPinLinksToIntermediate(*ThenPin, *LastOutCastFaildPin);

			check(LastOutCastSuccessPin != nullptr);
			// Copy all links from the invalid cast case above to the call function node
			CompilerContext.MovePinLinksToIntermediate(*ThenPin, *LastOutCastSuccessPin);
		}
	}

	// Break all connections to the original node, so it will be pruned
	BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
