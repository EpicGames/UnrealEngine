// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MultiGate.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_TemporaryVariable.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetNodeHelperLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MultiGate)

#define LOCTEXT_NAMESPACE "K2Node_MultiGate"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_MultiGate

class FKCHandler_MultiGate : public FNodeHandlingFunctor
{
protected:
	// Map to a bool that determines if we're in the first execution of the node or not
	TMap<UEdGraphNode*, FBPTerminal*> FirstRunTermMap;
	// Map to an int used to keep track of which outputs have been used
	TMap<UEdGraphNode*, FBPTerminal*> DataTermMap;

	/** 
	 * These are locals that don't need to be independent for each node, instead 
	 * they can be shared between each multigate node in the graph.
	 */
	struct FFunctionScopedTerms
	{
		FFunctionScopedTerms() : GenericBoolTerm(nullptr), IndexTerm(nullptr) {}

		// Generic bool term used for run-time conditions
		FBPTerminal* GenericBoolTerm;
		// Index term used for run-time index determination
		FBPTerminal* IndexTerm;
	};
	TMap<UFunction*, FFunctionScopedTerms> FunctionTermMap;

public:
	FKCHandler_MultiGate(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
	{
		FNodeHandlingFunctor::RegisterNets(Context, Node);

		const FString BaseNetName = Context.NetNameMap->MakeValidName(Node);

		// Create a term to store a bool that determines if we're in the first execution of the node or not
		FBPTerminal* FirstRunTerm = Context.CreateLocalTerminal();
		FirstRunTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		FirstRunTerm->Source = Node;
		FirstRunTerm->Name = BaseNetName + TEXT("_FirstRun");
		FirstRunTermMap.Add(Node, FirstRunTerm);

		UK2Node_MultiGate* GateNode = Cast<UK2Node_MultiGate>(Node);
		// If there is already a data node from expansion phase
		if (!GateNode || !GateNode->DataNode)
		{
			FBPTerminal* DataTerm = Context.CreateLocalTerminal();
			DataTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Int;
			DataTerm->Source = Node;
			DataTerm->Name = BaseNetName + TEXT("_Data");
			DataTermMap.Add(Node, DataTerm);
		}

		FFunctionScopedTerms& FuncLocals = FunctionTermMap.FindOrAdd(Context.Function);

		// Create a local scratch bool for run-time if there isn't already one
		if (!FuncLocals.GenericBoolTerm)
		{
			FuncLocals.GenericBoolTerm = Context.CreateLocalTerminal();
			FuncLocals.GenericBoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
			FuncLocals.GenericBoolTerm->Source = Node;
			FuncLocals.GenericBoolTerm->Name = BaseNetName + TEXT("_ScratchBool");
		}

		// Create a local scratch int for run-time index tracking if there isn't already one
		if (!FuncLocals.IndexTerm)
		{
			FuncLocals.IndexTerm = Context.CreateLocalTerminal();
			FuncLocals.IndexTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Int;
			FuncLocals.IndexTerm->Source = Node;
			FuncLocals.IndexTerm->Name = BaseNetName + TEXT("_ScratchIndex");
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
	{
		/////////////////////////////////////////////////////////////////////////////////////
		// Get the node, retrieve the helper functions, and create a local "Index" variable
		/////////////////////////////////////////////////////////////////////////////////////

		// Get the multi gate node and the helper functions
		UK2Node_MultiGate* GateNode = Cast<UK2Node_MultiGate>(Node);

		// Get function names and class pointers to helper nodes
		FName MarkBitFunctionName = "";
		UClass* MarkBitFunctionClass = NULL;
		GateNode->GetMarkBitFunction(MarkBitFunctionName, &MarkBitFunctionClass);
		UFunction* MarkBitFunction = FindUField<UFunction>(MarkBitFunctionClass, MarkBitFunctionName);

		FName HasUnmarkedBitFunctionName = "";
		UClass* HasUnmarkedBitFunctionClass = NULL;
		GateNode->GetHasUnmarkedBitFunction(HasUnmarkedBitFunctionName, &HasUnmarkedBitFunctionClass);
		UFunction* HasUnmarkedBitFunction = FindUField<UFunction>(HasUnmarkedBitFunctionClass, HasUnmarkedBitFunctionName);

		FName GetUnmarkedBitFunctionName = "";
		UClass* GetUnmarkedBitFunctionClass = NULL;
		GateNode->GetUnmarkedBitFunction(GetUnmarkedBitFunctionName, &GetUnmarkedBitFunctionClass);
		UFunction* GetUnmarkedBitFunction = FindUField<UFunction>(GetUnmarkedBitFunctionClass, GetUnmarkedBitFunctionName);

		FName ConditionalFunctionName = "";
		UClass* ConditionalFunctionClass = NULL;
		GateNode->GetConditionalFunction(ConditionalFunctionName, &ConditionalFunctionClass);
		UFunction* ConditionFunction = FindUField<UFunction>(ConditionalFunctionClass, ConditionalFunctionName);

		FName EqualityFunctionName = "";
		UClass* EqualityFunctionClass = NULL;
		GateNode->GetEqualityFunction(EqualityFunctionName, &EqualityFunctionClass);
		UFunction* EqualityFunction = FindUField<UFunction>(EqualityFunctionClass, EqualityFunctionName);

		FName BoolNotEqualFunctionName = "";
		UClass* BoolNotEqualFunctionClass = NULL;
		GateNode->GetBoolNotEqualFunction(BoolNotEqualFunctionName, &BoolNotEqualFunctionClass);
		UFunction* BoolNotEqualFunction = FindUField<UFunction>(BoolNotEqualFunctionClass, BoolNotEqualFunctionName);

		FName PrintStringFunctionName = "";
		UClass* PrintStringFunctionClass = NULL;
		GateNode->GetPrintStringFunction(PrintStringFunctionName, &PrintStringFunctionClass);
		UFunction* PrintFunction = FindUField<UFunction>(PrintStringFunctionClass, PrintStringFunctionName);

		FName ClearBitsFunctionName = "";
		UClass* ClearBitsFunctionClass = NULL;
		GateNode->GetClearAllBitsFunction(ClearBitsFunctionName, &ClearBitsFunctionClass);
		UFunction* ClearBitsFunction = FindUField<UFunction>(ClearBitsFunctionClass, ClearBitsFunctionName);

		// Find the data terms if there is already a data node from expansion phase
		FBPTerminal* DataTerm = NULL;
		if (GateNode->DataNode)
		{
			UEdGraphPin* PinToTry = FEdGraphUtilities::GetNetFromPin(GateNode->DataNode->GetVariablePin());
			FBPTerminal** DataTermPtr = Context.NetMap.Find(PinToTry);
			DataTerm = (DataTermPtr != NULL) ? *DataTermPtr : NULL;
		}
		// Else we built it in the net registration, so find it
		else
		{
			DataTerm = DataTermMap.FindRef(GateNode);
		}
		check(DataTerm);

		// Used for getting all the nets from pins
		UEdGraphPin* PinToTry = NULL;

		// The StartIndex passed into the multi gate node
		PinToTry = FEdGraphUtilities::GetNetFromPin(GateNode->GetStartIndexPin());
		FBPTerminal** StartIndexPinTerm = Context.NetMap.Find(PinToTry);

		// Get the bRandom pin as a kismet term from the multi gate node
		PinToTry = FEdGraphUtilities::GetNetFromPin(GateNode->GetIsRandomPin());
		FBPTerminal** RandomTerm = Context.NetMap.Find(PinToTry);

		// Get the Loop pin as a kismet term from the multi gate node
		PinToTry = FEdGraphUtilities::GetNetFromPin(GateNode->GetLoopPin());
		FBPTerminal** LoopTerm = Context.NetMap.Find(PinToTry);

		// Find the local boolean for use in determining if this is the first run of the node or not
		FBPTerminal* FirstRunBoolTerm = FirstRunTermMap.FindRef(GateNode);

		// Create a literal pin that represents a -1 value
		FBPTerminal* InvalidIndexTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		InvalidIndexTerm->bIsLiteral = true;
		InvalidIndexTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Int;
		InvalidIndexTerm->Name = TEXT("-1");

		// Create a literal pin that represents a true value
		FBPTerminal* TrueBoolTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		TrueBoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		TrueBoolTerm->bIsLiteral = true;
		TrueBoolTerm->Name = TEXT("true");

		// Get the out pins and create a literal describing how many logical outs there are
		TArray<UEdGraphPin*> OutPins;
		GateNode->GetOutPins(OutPins);
		FBPTerminal* NumOutsTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		NumOutsTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Int;
		NumOutsTerm->bIsLiteral = true;
		NumOutsTerm->Name = FString::Printf(TEXT("%d"), OutPins.Num());

		///////////////////////////////////////////////////
		// See if this is the first time in
		///////////////////////////////////////////////////

		FFunctionScopedTerms& FuncLocals = FunctionTermMap.FindChecked(Context.Function);
		check(FuncLocals.GenericBoolTerm != nullptr);

		// (bIsNotFirstTime != true)
		FBlueprintCompiledStatement& BoolNotEqualStatement = Context.AppendStatementForNode(Node);
		BoolNotEqualStatement.Type = KCST_CallFunction;
		BoolNotEqualStatement.FunctionToCall = BoolNotEqualFunction;
		BoolNotEqualStatement.FunctionContext = NULL;
		BoolNotEqualStatement.bIsParentContext = false;
		// Set the params
		BoolNotEqualStatement.LHS = FuncLocals.GenericBoolTerm;
		BoolNotEqualStatement.RHS.Add(FirstRunBoolTerm);
		BoolNotEqualStatement.RHS.Add(TrueBoolTerm);

		// if (bIsNotFirstTime == false)
		// {
		FBlueprintCompiledStatement& IfFirstTimeStatement = Context.AppendStatementForNode(Node);
		IfFirstTimeStatement.Type = KCST_GotoIfNot;
		IfFirstTimeStatement.LHS = FuncLocals.GenericBoolTerm;

		///////////////////////////////////////////////////////////////////
		// This is the first time in... set the bool and the start index
		///////////////////////////////////////////////////////////////////

		// bIsNotFirstTime = true;
		FBlueprintCompiledStatement& AssignBoolStatement = Context.AppendStatementForNode(Node);
		AssignBoolStatement.Type = KCST_Assignment;
		AssignBoolStatement.LHS = FirstRunBoolTerm;
		AssignBoolStatement.RHS.Add(TrueBoolTerm);

		//////////////////////////////////////////////////////////////////////
		// See if the StartIndex is greater than -1 (they supplied an index)
		//////////////////////////////////////////////////////////////////////

		// (StartIndex > -1)
		FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
		Statement.Type = KCST_CallFunction;
		Statement.FunctionToCall = ConditionFunction;
		Statement.FunctionContext = NULL;
		Statement.bIsParentContext = false;
		Statement.LHS = FuncLocals.GenericBoolTerm;
		Statement.RHS.Add(*StartIndexPinTerm);
		Statement.RHS.Add(InvalidIndexTerm);

		// if (StartIndex > -1)
		// {
		FBlueprintCompiledStatement& IfHasIndexStatement = Context.AppendStatementForNode(Node);
		IfHasIndexStatement.Type = KCST_GotoIfNot;
		IfHasIndexStatement.LHS = FuncLocals.GenericBoolTerm;

		///////////////////////////////////////////////////////////////////
		// They supplied a start index so set the index to it
		///////////////////////////////////////////////////////////////////

		// Index = StartIndex; // (StartIndex is from multi gate pin for it)
		FBlueprintCompiledStatement& AssignSuppliedIndexStatement = Context.AppendStatementForNode(Node);
		AssignSuppliedIndexStatement.Type = KCST_Assignment;
		AssignSuppliedIndexStatement.LHS = FuncLocals.IndexTerm;
		AssignSuppliedIndexStatement.RHS.Add(*StartIndexPinTerm);

		// Jump to index usage
		FBlueprintCompiledStatement& ElseGotoIndexUsageStatement = Context.AppendStatementForNode(Node);
		ElseGotoIndexUsageStatement.Type = KCST_UnconditionalGoto;
		// }
		// else
		// {

		///////////////////////////////////////////////////////////////////
		// They did NOT supply a start index so figure one out
		///////////////////////////////////////////////////////////////////

		check(FuncLocals.IndexTerm != nullptr);

		// Index = GetUnmarkedBit(Data, -1, bRandom);
		FBlueprintCompiledStatement& GetStartIndexStatement = Context.AppendStatementForNode(Node);
		GetStartIndexStatement.Type = KCST_CallFunction;
		GetStartIndexStatement.FunctionToCall = GetUnmarkedBitFunction;
		GetStartIndexStatement.bIsParentContext = false;
		GetStartIndexStatement.LHS = FuncLocals.IndexTerm;
		GetStartIndexStatement.RHS.Add(DataTerm);
		GetStartIndexStatement.RHS.Add(*StartIndexPinTerm);
		GetStartIndexStatement.RHS.Add(NumOutsTerm);
		GetStartIndexStatement.RHS.Add(*RandomTerm);
		// Hook the IfHasIndexStatement jump to this node
		GetStartIndexStatement.bIsJumpTarget = true;
		IfHasIndexStatement.TargetLabel = &GetStartIndexStatement;

		// Jump to index usage
		FBlueprintCompiledStatement& StartIndexGotoIndexUsageStatement = Context.AppendStatementForNode(Node);
		StartIndexGotoIndexUsageStatement.Type = KCST_UnconditionalGoto;
		// }
		// }
		// else
		// {
		////////////////////////////////////////////////////////////////////////////
		// Else this is NOT the first time in, see if there is an available index
		////////////////////////////////////////////////////////////////////////////

		// (HasUnmarkedBit())
		FBlueprintCompiledStatement& IsAvailableStatement = Context.AppendStatementForNode(Node);
		IsAvailableStatement.Type = KCST_CallFunction;
		IsAvailableStatement.FunctionToCall = HasUnmarkedBitFunction;
		IsAvailableStatement.FunctionContext = NULL;
		IsAvailableStatement.bIsParentContext = false;
		IsAvailableStatement.LHS = FuncLocals.GenericBoolTerm;
		IsAvailableStatement.RHS.Add(DataTerm);
		IsAvailableStatement.RHS.Add(NumOutsTerm);
		// Hook the IfFirstTimeStatement jump to this node
		IsAvailableStatement.bIsJumpTarget = true;
		IfFirstTimeStatement.TargetLabel = &IsAvailableStatement;

		// if (HasUnmarkedBit())
		// {
		FBlueprintCompiledStatement& IfIsAvailableStatement = Context.AppendStatementForNode(Node);
		IfIsAvailableStatement.Type = KCST_GotoIfNot;
		IfIsAvailableStatement.LHS = FuncLocals.GenericBoolTerm;

		////////////////////////////////////////////////////////////////////////////
		// Has available index so figure it out and jump to its' usage
		////////////////////////////////////////////////////////////////////////////

		// Index = GetUnmarkedBit(Data, -1, bRandom)
		FBlueprintCompiledStatement& GetNextIndexStatement = Context.AppendStatementForNode(Node);
		GetNextIndexStatement.Type = KCST_CallFunction;
		GetNextIndexStatement.FunctionToCall = GetUnmarkedBitFunction;
		GetNextIndexStatement.bIsParentContext = false;
		GetNextIndexStatement.LHS = FuncLocals.IndexTerm;
		GetNextIndexStatement.RHS.Add(DataTerm);
		GetNextIndexStatement.RHS.Add(*StartIndexPinTerm);
		GetNextIndexStatement.RHS.Add(NumOutsTerm);
		GetNextIndexStatement.RHS.Add(*RandomTerm);

		// Goto Index usage
		FBlueprintCompiledStatement& GotoIndexUsageStatement = Context.AppendStatementForNode(Node);
		GotoIndexUsageStatement.Type = KCST_UnconditionalGoto;
		// }
		// else
		// {
		////////////////////////////////////////////////////////////////////////////
		// No available index, see if we can loop
		////////////////////////////////////////////////////////////////////////////

		// if (bLoop)
		FBlueprintCompiledStatement& IfLoopingStatement = Context.AppendStatementForNode(Node);
		IfLoopingStatement.Type = KCST_GotoIfNot;
		IfLoopingStatement.LHS = *LoopTerm;
		IfLoopingStatement.bIsJumpTarget = true;
		IfIsAvailableStatement.TargetLabel = &IfLoopingStatement;
		// {
		////////////////////////////////////////////////////////////////////////////
		// Reset the data and jump back up to "if (HasUnmarkedBit())"
		////////////////////////////////////////////////////////////////////////////

		// Clear the data
		// Data = 0;
		FBlueprintCompiledStatement& ClearDataStatement = Context.AppendStatementForNode(Node);
		ClearDataStatement.Type = KCST_CallFunction;
		ClearDataStatement.FunctionToCall = ClearBitsFunction;
		ClearDataStatement.bIsParentContext = false;
		ClearDataStatement.RHS.Add(DataTerm);

		// Goto back up to attempt an index again
		FBlueprintCompiledStatement& RetryStatement = Context.AppendStatementForNode(Node);
		RetryStatement.Type = KCST_UnconditionalGoto;
		IsAvailableStatement.bIsJumpTarget = true;
		RetryStatement.TargetLabel = &IsAvailableStatement;
		// }
		// else
		// {
		////////////////////////////////////////////////////////////////////////////
		// Dead... Jump to end of thread
		////////////////////////////////////////////////////////////////////////////
		FBlueprintCompiledStatement& NoLoopStatement = Context.AppendStatementForNode(Node);
		NoLoopStatement.Type = KCST_EndOfThread;
		NoLoopStatement.bIsJumpTarget = true;
		IfLoopingStatement.TargetLabel = &NoLoopStatement;
		// }
		// }
		// }

		//////////////////////////////////////
		// We have a valid index so mark it
		//////////////////////////////////////

		// MarkBit(Data, Index);
		FBlueprintCompiledStatement& MarkIndexStatement = Context.AppendStatementForNode(Node);
		MarkIndexStatement.Type = KCST_CallFunction;
		MarkIndexStatement.FunctionToCall = MarkBitFunction;
		MarkIndexStatement.bIsParentContext = false;
		MarkIndexStatement.LHS = FuncLocals.IndexTerm;
		MarkIndexStatement.RHS.Add(DataTerm);
		MarkIndexStatement.RHS.Add(FuncLocals.IndexTerm);

		// Setup jump label
		MarkIndexStatement.bIsJumpTarget = true;
		GotoIndexUsageStatement.TargetLabel = &MarkIndexStatement;
		ElseGotoIndexUsageStatement.TargetLabel = &MarkIndexStatement;
		StartIndexGotoIndexUsageStatement.TargetLabel = &MarkIndexStatement;

		/////////////////////////////////////////////////////////////////////////
		// We have a valid index so mark it, then find the correct exec out pin
		/////////////////////////////////////////////////////////////////////////

		// Call the correct exec pin out of the multi gate node
		FBlueprintCompiledStatement* PrevIndexEqualityStatement = NULL;
		FBlueprintCompiledStatement* PrevIfIndexMatchesStatement = NULL;
		for (int32 OutIdx = 0; OutIdx < OutPins.Num(); OutIdx++)
		{
			// (Index == OutIdx)
			FBlueprintCompiledStatement& IndexEqualityStatement = Context.AppendStatementForNode(Node);
			IndexEqualityStatement.Type = KCST_CallFunction;
			IndexEqualityStatement.FunctionToCall = EqualityFunction;
			IndexEqualityStatement.FunctionContext = NULL;
			IndexEqualityStatement.bIsParentContext = false;
			// LiteralIndexTerm will be the right side of the == statemnt
			FBPTerminal* LiteralIndexTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
			LiteralIndexTerm->bIsLiteral = true;
			LiteralIndexTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Int;
			LiteralIndexTerm->Name = FString::Printf(TEXT("%d"), OutIdx);
			// Set the params
			IndexEqualityStatement.LHS = FuncLocals.GenericBoolTerm;
			IndexEqualityStatement.RHS.Add(FuncLocals.IndexTerm);
			IndexEqualityStatement.RHS.Add(LiteralIndexTerm);

			// if (Index == OutIdx)
			FBlueprintCompiledStatement& IfIndexMatchesStatement = Context.AppendStatementForNode(Node);
			IfIndexMatchesStatement.Type = KCST_GotoIfNot;
			IfIndexMatchesStatement.LHS = FuncLocals.GenericBoolTerm;
			// {
			//////////////////////////////////////
			// Found a match - Jump there
			//////////////////////////////////////

			GenerateSimpleThenGoto(Context, *GateNode, OutPins[OutIdx]);
			// }
			// else
			// {
			////////////////////////////////////////////////////
			// Not a match so loop will attempt the next index
			////////////////////////////////////////////////////

			if (PrevIndexEqualityStatement && PrevIfIndexMatchesStatement)
			{
				// Attempt next index
				IndexEqualityStatement.bIsJumpTarget = true;
				PrevIfIndexMatchesStatement->TargetLabel = &IndexEqualityStatement;
			}
			// }

			PrevIndexEqualityStatement = &IndexEqualityStatement;
			PrevIfIndexMatchesStatement = &IfIndexMatchesStatement;
		}

		check(PrevIfIndexMatchesStatement);

		// Should have jumped to proper index, print error (should never happen)
		// Create a CallFunction statement for doing a print string of our error message
		FBlueprintCompiledStatement& PrintStatement = Context.AppendStatementForNode(Node);
		PrintStatement.Type = KCST_CallFunction;
		PrintStatement.bIsJumpTarget = true;
		PrintStatement.FunctionToCall = PrintFunction;
		PrintStatement.FunctionContext = NULL;
		PrintStatement.bIsParentContext = false;
		// Create a local int for use in the equality call function below (LiteralTerm = the right hand side of the EqualEqual_IntInt or NotEqual_BoolBool statement)
		FBPTerminal* LiteralStringTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		LiteralStringTerm->bIsLiteral = true;
		LiteralStringTerm->Type.PinCategory = UEdGraphSchema_K2::PC_String;
		LiteralStringTerm->Name = FText::Format(LOCTEXT("MultiGateNode IndexWarningFmt", "MultiGate Node failed! Out of bounds indexing of the out pins. There are only {0} outs available."), OutPins.Num()).ToString();
		PrintStatement.RHS.Add(LiteralStringTerm);
		// Hook the IfNot statement's jump target to this statement
		PrevIfIndexMatchesStatement->TargetLabel = &PrintStatement;
	}
};

UK2Node_MultiGate::UK2Node_MultiGate(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UK2Node_MultiGate::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "MultiGate_Tooltip", "Executes a series of pins in order");
}

FLinearColor UK2Node_MultiGate::GetNodeTitleColor() const
{
	return FLinearColor::White;
}

FText UK2Node_MultiGate::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "MultiGate", "MultiGate");
}

bool UK2Node_MultiGate::CanAddPin() const
{
	// Since an int32 is used to track which gates have been used, we can only have 32 outputs
	return GetNumOutPins() < 32;
}

void UK2Node_MultiGate::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, TEXT("Reset"));
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, TEXT("IsRandom"));
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, TEXT("Loop"));
	UEdGraphPin* IndexPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, TEXT("StartIndex"));
	K2Schema->SetPinAutogeneratedDefaultValue(IndexPin, TEXT("-1"));
}

void UK2Node_MultiGate::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, TEXT("Reset"));
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, TEXT("IsRandom"));
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, TEXT("Loop"));
	UEdGraphPin* IndexPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, TEXT("StartIndex"));
	K2Schema->SetPinAutogeneratedDefaultValue(IndexPin, TEXT("-1"));
}

UEdGraphPin* UK2Node_MultiGate::GetResetPin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("Reset"));
	check(Pin != NULL);
	return Pin;
}

UEdGraphPin* UK2Node_MultiGate::GetIsRandomPin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("IsRandom"));
	check(Pin != NULL);
	return Pin;
}

UEdGraphPin* UK2Node_MultiGate::GetLoopPin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("Loop"));
	check(Pin != NULL);
	return Pin;
}

UEdGraphPin* UK2Node_MultiGate::GetStartIndexPin() const
{
	UEdGraphPin* Pin = FindPin(TEXT("StartIndex"));
	check(Pin != NULL);
	return Pin;
}

void UK2Node_MultiGate::GetOutPins(TArray<UEdGraphPin*>& OutPins) const
{
	OutPins.Reset(Pins.Num());

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Output)
		{
			OutPins.Add(Pin);
		}
	}
}

int32 UK2Node_MultiGate::GetNumOutPins() const
{
	int32 NumOutPins = 0;

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->Direction == EGPD_Output)
		{
			++NumOutPins;
		}
	}

	return NumOutPins;
}

void UK2Node_MultiGate::GetMarkBitFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, MarkBit);
	*FunctionClass = UKismetNodeHelperLibrary::StaticClass();
}

void UK2Node_MultiGate::GetHasUnmarkedBitFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, HasUnmarkedBit);
	*FunctionClass = UKismetNodeHelperLibrary::StaticClass();
}

void UK2Node_MultiGate::GetUnmarkedBitFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, GetUnmarkedBit);
	*FunctionClass = UKismetNodeHelperLibrary::StaticClass();
}

/** Gets the name and class of the Greater_IntInt function from the KismetMathLibrary */
void UK2Node_MultiGate::GetConditionalFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Greater_IntInt);
	*FunctionClass = UKismetMathLibrary::StaticClass();
}

void UK2Node_MultiGate::GetEqualityFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, EqualEqual_IntInt);
	*FunctionClass = UKismetMathLibrary::StaticClass();
}

/** Gets the name and class of the NotEqual_BoolBool function from the KismetMathLibrary */
void UK2Node_MultiGate::GetBoolNotEqualFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, NotEqual_BoolBool);
	*FunctionClass = UKismetMathLibrary::StaticClass();
}

/** Gets the name and class of the PrintString function */
void UK2Node_MultiGate::GetPrintStringFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, PrintWarning);
	*FunctionClass = UKismetSystemLibrary::StaticClass();
}

void UK2Node_MultiGate::GetClearAllBitsFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetNodeHelperLibrary, ClearAllBits);
	*FunctionClass = UKismetNodeHelperLibrary::StaticClass();
}

FName UK2Node_MultiGate::GetPinNameGivenIndex(int32 Index) const
{
	return *FString::Printf(TEXT("Out %d"), Index);
}

FNodeHandlingFunctor* UK2Node_MultiGate::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_MultiGate(CompilerContext);
}

void UK2Node_MultiGate::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	/////////////////////////////
	// Handle the "Reset"
	/////////////////////////////

	// Redirect the reset pin if linked to
	UEdGraphPin* ResetPin = GetResetPin();
	if (ResetPin->LinkedTo.Num() > 0)
	{
		/////////////////////////////
		// Temporary Variable node
		/////////////////////////////

		// Create the node
		UK2Node_TemporaryVariable* TempVarNode = SourceGraph->CreateIntermediateNode<UK2Node_TemporaryVariable>();
		TempVarNode->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
		TempVarNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(TempVarNode, this);
		// Give a reference of the variable node to the multi gate node
		DataNode = TempVarNode;

		/////////////////////////////
		// Assignment node
		/////////////////////////////

		// Create the node
		UK2Node_AssignmentStatement* AssignmentNode = SourceGraph->CreateIntermediateNode<UK2Node_AssignmentStatement>();
		AssignmentNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(AssignmentNode, this);

		// Coerce the wildcards pin types (set the default of the value to 0)
		AssignmentNode->GetVariablePin()->PinType = TempVarNode->GetVariablePin()->PinType;
		AssignmentNode->GetVariablePin()->MakeLinkTo(TempVarNode->GetVariablePin());
		AssignmentNode->GetValuePin()->PinType = TempVarNode->GetVariablePin()->PinType;
		AssignmentNode->GetValuePin()->DefaultValue = TEXT("0");

		// Move the "Reset" link to the Assignment node
		CompilerContext.MovePinLinksToIntermediate(*ResetPin, *AssignmentNode->GetExecPin());
	}
}

void UK2Node_MultiGate::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

#undef LOCTEXT_NAMESPACE
