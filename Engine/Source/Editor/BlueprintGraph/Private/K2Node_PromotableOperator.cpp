// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PromotableOperator.h"

#include "BlueprintEditorSettings.h"
#include "BlueprintTypePromotion.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "EngineLogs.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Kismet/BlueprintTypeConversions.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/WildcardNodeUtils.h"
#include "KismetCompiler.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuDelegates.h"
#include "ToolMenuSection.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "EdGraphSchema_K2_Actions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PromotableOperator)

#define LOCTEXT_NAMESPACE "PromotableOperatorNode"

static TAutoConsoleVariable<bool> CVarAllowConversionOfComparisonOps(
	TEXT("BP.bAllowConversionOfComparisonOps"),
	true,
	TEXT("If true, then allow the user to convert between comparison operators on the UK2Node_PromotableOperator"),
	ECVF_Default);

///////////////////////////////////////////////////////////
// Pin names for default construction

static const FName InputPinA_Name = FName(TEXT("A"));
static const FName InputPinB_Name = FName(TEXT("B"));
static const FName TolerancePin_Name = FName(TEXT("ErrorTolerance"));
static const int32 NumFunctionInputs = 2;

///////////////////////////////////////////////////////////
// UK2Node_PromotableOperator

UK2Node_PromotableOperator::UK2Node_PromotableOperator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UpdateOpName();
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAllButExec;
	NumAdditionalInputs = 0;
}

///////////////////////////////////////////////////////////
// UEdGraphNode interface

namespace PromotableOpUtils
{
	bool FindTolerancePinType(const UFunction* Func, FEdGraphPinType& OutPinType)
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		// Check if there is a tolerance field that we need to add
		// If UFunction has a third input param 
		int32 InputArgsFound = 0;
		for (TFieldIterator<FProperty> PropIt(Func); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;

			// We don't care about the return property, and we are looking for
			// any additional input param that isn't in the normal function input range
			if (Param->HasAnyPropertyFlags(CPF_ReturnParm) || (InputArgsFound++ < NumFunctionInputs))
			{
				continue;
			}

			// Found the tolerance type!
			FEdGraphPinType ParamType;
			if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
			{
				OutPinType = ParamType;
				return true;
			}
		}
		return false;
	}
}

void UK2Node_PromotableOperator::AllocateDefaultPins()
{
	FWildcardNodeUtils::CreateWildcardPin(this, InputPinA_Name, EGPD_Input);
	FWildcardNodeUtils::CreateWildcardPin(this, InputPinB_Name, EGPD_Input);

	UEdGraphPin* OutPin = FWildcardNodeUtils::CreateWildcardPin(this, UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);

	const UFunction* Func = GetTargetFunction();
	// For comparison functions we always want a bool output, so make it visually so
	if (Func && FTypePromotion::IsComparisonFunc(Func))
	{
		OutPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;

		// If we need a tolerance pin then we need to add it here, but not if we are in a wildcard state
		FEdGraphPinType ToleranceType;
		if (PromotableOpUtils::FindTolerancePinType(Func, ToleranceType))
		{
			UEdGraphPin* TolerancePin = CreatePin(EGPD_Input, ToleranceType, TolerancePin_Name);
		}
	}

	// Update the op name so that if there is a blank wildcard node left on the graph we
	// can ensure that it is correct
	UpdateOpName();
	ensureMsgf((OperationName != TEXT("NO_OP")), TEXT("Invalid operation name on Promotable Operator node!"));

	// Create any additional input pin. Their appropriate type is determined in ReallocatePinsDuringReconstruction
	// because we cannot get a promoted type with no links to the pin.
	for (int32 i = NumFunctionInputs; i < (NumAdditionalInputs + NumFunctionInputs); ++i)
	{
		AddInputPinImpl(i);
	}
}

void UK2Node_PromotableOperator::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	static const FName PromotableOperatorNodeName = FName("PromotableOperator");
	static const FText PromotableOperatorStr = LOCTEXT("PromotableOperatorNode", "Operator Node");
	
	if (CVarAllowConversionOfComparisonOps.GetValueOnAnyThread() && CanConvertComparisonOperatorNodeType(Context->Node))
	{
		static const FName ConvertComparisonOpName = FName("ConvertComparisonOp");
		static const FText ConvertComparisonOpStr = LOCTEXT("ConvertComparisonOp", "Convert Operator");
		
		FToolMenuSection& Section = Menu->AddSection(ConvertComparisonOpName, ConvertComparisonOpStr);
		for (const FName& PossibleConversionOpName : FTypePromotion::GetComparisonOpNames())
		{
			// Don't display our current operator type
			if (PossibleConversionOpName == OperationName)
			{
				continue;
			}

			FFormatNamedArguments Args;
			Args.Add(TEXT("NewOpName"), FTypePromotion::GetUserFacingOperatorName(PossibleConversionOpName));
			Args.Add(TEXT("CurrentOpName"), FTypePromotion::GetUserFacingOperatorName(OperationName));

			const FText PinConversionName = FText::Format(LOCTEXT("ConvertOpName_Tooltip", "Convert to {NewOpName}"), Args);

			Section.AddMenuEntry(
				FName(PinConversionName.ToString()),
				PinConversionName,
				FText::Format(LOCTEXT("ConvertOperator_ToType_Tooltip", "Convert this node operation from '{CurrentOpName}' to '{NewOpName}'"), Args),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateStatic(&UK2Node_PromotableOperator::ConvertComparisonOperatorNode, const_cast<UEdGraphNode*>(Context->Node.Get()), PossibleConversionOpName)
				)
			);
		}
	}
	
	// Add the option to remove a pin via the context menu
	if (CanRemovePin(Context->Pin))
	{
		FToolMenuSection& Section = Menu->AddSection(PromotableOperatorNodeName, PromotableOperatorStr);
		Section.AddMenuEntry(
			"RemovePin",
			LOCTEXT("RemovePin", "Remove pin"),
			LOCTEXT("RemovePinTooltip", "Remove this input pin"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_PromotableOperator*>(this), &UK2Node_PromotableOperator::RemoveInputPin, const_cast<UEdGraphPin*>(Context->Pin))
			)
		);
	}
	else if (CanAddPin())
	{
		FToolMenuSection& Section = Menu->AddSection(PromotableOperatorNodeName, PromotableOperatorStr);
		Section.AddMenuEntry(
			"AddPin",
			LOCTEXT("AddPin", "Add pin"),
			LOCTEXT("AddPinTooltip", "Add another input pin"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_PromotableOperator*>(this), &UK2Node_PromotableOperator::AddInputPin)
			)
		);
	}

	// Add the pin conversion sub menu
	if (CanConvertPinType(Context->Pin))
	{
		static const FName ConvNodeName = FName("PromotableOperatorPinConvs");
		static const FText ConvNodeStr = LOCTEXT("PromotableOperatorPinConvs", "Pin Conversions");

		FToolMenuSection& ConversionSection = Menu->AddSection(ConvNodeName, ConvNodeStr);

		// Give the user an option to reset this node to wildcard
		if (!FWildcardNodeUtils::IsWildcardPin(Context->Pin))
		{
			const FText ResetName = LOCTEXT("ResetFunction", "To Wildcard");

			ConversionSection.AddMenuEntry(
				FName(ResetName.ToString()),
				ResetName,
				LOCTEXT("ResetToWildcardTooltip", "Break all connections and reset this node to a wildcard state."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_PromotableOperator*>(this), &UK2Node_PromotableOperator::ConvertPinType, const_cast<UEdGraphPin*>(Context->Pin), FWildcardNodeUtils::GetDefaultWildcardPinType())
				)
			);
		}

		CreateConversionMenu(ConversionSection, (UEdGraphPin*)Context->Pin);
	}
}

void UK2Node_PromotableOperator::CreateConversionMenu(FToolMenuSection& ConversionSection, UEdGraphPin* PinToConvert) const
{
	check(PinToConvert);

	// Gather what pin types could possibly be used for a conversion with this operator
	TArray<UFunction*> AvailableFunctions;
	FTypePromotion::GetAllFuncsForOp(OperationName, AvailableFunctions);
	TArray<FEdGraphPinType> PossiblePromos;
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// If this is a split pin, then we need to convert the parent pin, not the child.
	if (PinToConvert->ParentPin != nullptr)
	{
		PinToConvert = PinToConvert->ParentPin;
	}

	// If we have a pin that matches our current type, then we can use it to see if we can still get a valid function
	FEdGraphPinType OriginalContextType = PinToConvert->PinType;

	for (const UFunction* Func : AvailableFunctions)
	{
		for (TFieldIterator<FProperty> PropIt(Func); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			const FProperty* Param = *PropIt;
			FEdGraphPinType ParamType;

			if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
			{
				if (FWildcardNodeUtils::IsWildcardPin(PinToConvert) || FTypePromotion::IsValidPromotion(ParamType, PinToConvert->PinType) || FTypePromotion::IsValidPromotion(PinToConvert->PinType, ParamType))
				{
					PossiblePromos.AddUnique(ParamType);
				}
			}
		}
	}

	// Don't display the conversion menu if there are no valid conversions
	if (AvailableFunctions.Num() == 0)
	{
		return;
	}

	// Add the options to the context menu
	for (const FEdGraphPinType& PinType : PossiblePromos)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NewPinType"), Schema->TypeToText(PinType));
		Args.Add(TEXT("CurrentPinType"), Schema->TypeToText(OriginalContextType));

		const FText PinConversionName = FText::Format(LOCTEXT("Convert_Pin_To_Type_Tooltip", "To {NewPinType}"), Args);

		ConversionSection.AddMenuEntry(
			FName(PinConversionName.ToString()),
			PinConversionName,
			FText::Format(LOCTEXT("ConvertPinTypeTooltip", "Convert this pin type from '{CurrentPinType}' to '{NewPinType}'"), Args),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(const_cast<UK2Node_PromotableOperator*>(this), &UK2Node_PromotableOperator::ConvertPinType, const_cast<UEdGraphPin*>(PinToConvert), PinType)
			)
		);
	}
}

bool UK2Node_PromotableOperator::CanConvertPinType(const UEdGraphPin* Pin) const
{
	// You can convert any pin except for output pins on comparison functions and the tolerance pin
	return
		Pin &&
		!(Pin->Direction == EGPD_Output &&
		FTypePromotion::IsComparisonFunc(GetTargetFunction())) &&
		!IsTolerancePin(*Pin);
}

bool UK2Node_PromotableOperator::CanConvertComparisonOperatorNodeType(const UEdGraphNode* Node)
{
	if (!Node)
	{
		return false;
	}
	
	if (const UK2Node_PromotableOperator* OpNode = Cast<UK2Node_PromotableOperator>(Node))
	{
		// Only allow conversions between simple comparison types. If we have a tolerance pin, don't allow conversion
		// because not all operators for those types have a tolerance.
		const UEdGraphPin* TolerancePin = OpNode->FindTolerancePin();
		const bool bTolerancePinAcceptable = !TolerancePin || TolerancePin->bHidden;
		
		const UFunction* TargetFunc = OpNode->GetTargetFunction();
		
		// If the target function is null then it's a wildcard node and we can convert it
		return (TargetFunc == nullptr || FTypePromotion::IsComparisonFunc(TargetFunc)) && bTolerancePinAcceptable;
	}

	return false;
}

void UK2Node_PromotableOperator::ConvertComparisonOperatorNode(UEdGraphNode* Node, const FName NewOpName)
{
	UK2Node_PromotableOperator* OpNode = Cast<UK2Node_PromotableOperator>(Node);
	// You can only convert to comparison operators, nothing else, because then we can be sure the number of pins to consider will be correct
	// when finding the best matching function. Other math ops may behave in undefined ways if we allowed it.
	if (!OpNode || !FTypePromotion::IsComparisonOpName(NewOpName))
	{
		return;
	}

	TArray<UEdGraphPin*> PinsToConsider;
	OpNode->GetPinsToConsider(PinsToConsider);

	// In the case of this node having no pins to consider (it is a wildcard, with no default values or connections)
	// we will just use the boolean output pin to ensure that we get a valid UFunction to use.
	if (PinsToConsider.IsEmpty())
	{
		UEdGraphPin* OutPin = OpNode->GetOutputPin();
		// Because we only allow this conversion to happen on comparison operators, we can be sure the output pin will be a 
		// simple boolean output and that it is there
		ensure(OutPin && OutPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
		PinsToConsider.Add(OutPin);		
	}

	// For nodes with connections we have to find the best function again that matches for them
	if (const UFunction* BestMatchingFunc = FTypePromotion::FindBestMatchingFunc(NewOpName, PinsToConsider))
	{
		FScopedTransaction Transaction(LOCTEXT("PromotableOperatorComparisonOpConversion", "Convert operator node"));
		OpNode->Modify();
		
		OpNode->OperationName = NewOpName;
		
		// Only allow this with comparison functions
		ensure(FTypePromotion::IsComparisonFunc(BestMatchingFunc));
		UE_LOGF(LogBlueprint, Verbose, "Converting node '%ls' from '%ls' to '%ls'...", *GetNameSafe(OpNode), *OpNode->OperationName.ToString(), *NewOpName.ToString());
		
		OpNode->SetFromFunction(BestMatchingFunc);
		OpNode->ReconstructNode();
	}
	else
	{
		UE_LOGF(LogBlueprint, Error, "Failed to convert Convert node from '%ls' to '%ls'", *OpNode->OperationName.ToString(), *NewOpName.ToString());
	}
}

FText UK2Node_PromotableOperator::GetTooltipText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("FunctionTooltip"), !HasAnyConnectionsOrDefaults() ? FTypePromotion::GetUserFacingOperatorName(OperationName) : Super::GetTooltipText());

	return FText::Format(LOCTEXT("PromotableOpTooltipText", "{FunctionTooltip}\n\nTo go back to old math nodes, uncheck 'Enable Type Promotion' in the Blueprint Editor Settings. Both node types are supported."), Args);
}

void UK2Node_PromotableOperator::PinDefaultValueChanged(UEdGraphPin* Pin)
{
	Super::PinDefaultValueChanged(Pin);

	if (bDefaultValueReentranceGuard)
	{
		return;
	}

	// Re-entrance Guard just in case this function gets called from any notify triggers in the schema
	// to prevent possible recursive calls from ResetPinToAutogeneratedDefaultValue when breaking
	// all links to this node
	TGuardValue<bool> ReentranceGuard(bDefaultValueReentranceGuard, true);

	// If this default value resets to the default one on the pin, and there are no other
	// connections or default values, then we should just reset the whole node to a wildcard
	if (!HasAnyConnectionsOrDefaults())
	{
		ResetNodeToWildcard();
	}
}

void UK2Node_PromotableOperator::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	// This will handle the case of dragging off of this node, and connecting to a node via typing
	// in the context menu. Without updating in this case, our pins would be left as wildcards!
	if(HasAnyConnectionsOrDefaults())
	{
		UpdateOpName();
		UpdateFromBestMatchingFunction();

		// Get correct default value boxes
		GetGraph()->NotifyNodeChanged(this);
	}
}

void UK2Node_PromotableOperator::PostPasteNode()
{
	Super::PostPasteNode();

	// If we have copied a node with additional pins then we need to make sure 
	// they get reset to wildcard as well, otherwise their type will persist
	if (!HasAnyConnectionsOrDefaults())
	{
		ResetNodeToWildcard();
	}
}

///////////////////////////////////////////////////////////
// UK2Node interface

void UK2Node_PromotableOperator::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const bool bValidOpName = UpdateOpName();
	if (!bValidOpName)
	{
		UE_LOGF(LogBlueprint, Error, "Could not find matching operation name for this function!");
		CompilerContext.MessageLog.Error(TEXT("Could not find matching operation on '@@'!"), this);
		return;
	}

	UEdGraphPin* OriginalOutputPin = GetOutputPin();
	TArray<UEdGraphPin*> OriginalInputPins = GetInputPins();

	// Our operator function has been determined on pin connection change
	UFunction* OpFunction = GetTargetFunction();

	if (!OpFunction)
	{
		UE_LOGF(LogBlueprint, Error, "Could not find matching op function during expansion!");
		CompilerContext.MessageLog.Error(TEXT("Could not find matching op function during expansion on '@@'!"), this);
		return;
	}
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	/** Helper struct to gather the necessary pins we need to create redirections */
	struct FIntermediateCastPinHelper
	{
		UEdGraphPin* InputA = nullptr;
		UEdGraphPin* InputB = nullptr;
		UEdGraphPin* OutputPin = nullptr;
		UEdGraphPin* SelfPin = nullptr;
		UEdGraphPin* TolerancePin = nullptr;

		explicit FIntermediateCastPinHelper(UK2Node_CallFunction* NewOperator)
		{
			check(NewOperator);
			SelfPin = NewOperator->FindPin(UEdGraphSchema_K2::PN_Self);
			TolerancePin = NewOperator->FindPin(TolerancePin_Name, EGPD_Input);

			// Find inputs and outputs
			for (UEdGraphPin* Pin : NewOperator->Pins)
			{
				if (Pin == SelfPin || Pin == TolerancePin)
				{
					continue;
				}

				if (Pin->Direction == EEdGraphPinDirection::EGPD_Input)
				{
					if (!InputA)
					{
						InputA = Pin;
					}
					else if (!InputB)
					{
						InputB = Pin;
					}
				}
				else if (Pin->Direction == EEdGraphPinDirection::EGPD_Output)
				{
					OutputPin = Pin;
				}
			}
		}

		~FIntermediateCastPinHelper() = default;
	};

	UK2Node_CallFunction* PrevIntermediateNode = nullptr;
	UEdGraphPin* PrevOutputPin = nullptr;
	UEdGraphPin* MyTolerancePin = FindTolerancePin();

	// Create cast from original 2 inputs to the first intermediate node
	{
		UFunction* BestFunc = OpFunction;
		// Look for a best matching function if we possibly don't have one
		if(!BestFunc)
		{
			TArray<UEdGraphPin*> PinsToConsider =
			{
				OriginalInputPins[0],
				OriginalInputPins[1],
				OriginalOutputPin
			};

			if (UFunction* Func = FTypePromotion::FindBestMatchingFunc(OperationName, PinsToConsider))
			{
				BestFunc = Func;
			}
		}

		PrevIntermediateNode = CreateIntermediateNode(this, BestFunc, CompilerContext, SourceGraph);
		FIntermediateCastPinHelper NewOpHelper(PrevIntermediateNode);
		PrevOutputPin = PrevIntermediateNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);

		const bool bPinASuccess = UK2Node_PromotableOperator::CreateIntermediateCast(this, CompilerContext, SourceGraph, OriginalInputPins[0], NewOpHelper.InputA);
		const bool bPinBSuccess = UK2Node_PromotableOperator::CreateIntermediateCast(this, CompilerContext, SourceGraph, OriginalInputPins[1], NewOpHelper.InputB);
		// Attempt to connect the tolerance pin if both nodes have one
		const bool bToleranceSuccess = MyTolerancePin && NewOpHelper.TolerancePin ? UK2Node_PromotableOperator::CreateIntermediateCast(this, CompilerContext, SourceGraph, MyTolerancePin, NewOpHelper.TolerancePin) : true;

		if (!bPinASuccess || !bPinBSuccess || !bToleranceSuccess)
		{
			CompilerContext.MessageLog.Error(TEXT("'@@' could not successfuly expand pins!"), PrevIntermediateNode);
		}
	}

	// Loop through all the additional inputs, create a new node of this function and connecting inputs as necessary 
	for (int32 i = NumFunctionInputs; i < NumAdditionalInputs + NumFunctionInputs; ++i)
	{
		check(i > 0 && i < OriginalInputPins.Num());
		FIntermediateCastPinHelper PrevNodeHelper(PrevIntermediateNode);

		// Find the best matching function that this intermediate node should use
		// so that we can avoid unnecessary conversion nodes and casts
		UFunction* BestMatchingFunc = OpFunction;
		{
			TArray<UEdGraphPin*> PinsToConsider =
			{
				PrevNodeHelper.OutputPin,
				OriginalInputPins[i],
				OriginalOutputPin
			};

			if (UFunction* Func = FTypePromotion::FindBestMatchingFunc(OperationName, PinsToConsider))
			{
				BestMatchingFunc = Func;
			}
		}

		UK2Node_CallFunction* NewIntermediateNode = CreateIntermediateNode(PrevIntermediateNode, BestMatchingFunc, CompilerContext, SourceGraph);
		FIntermediateCastPinHelper NewOpHelper(NewIntermediateNode);

		// Connect the output pin of the previous intermediate node, to the input of the new one
		const bool bPinASuccess = CreateIntermediateCast(PrevIntermediateNode, CompilerContext, SourceGraph, NewOpHelper.InputA, PrevOutputPin);

		// Connect the original node's pin to the newly created intermediate node's B Pin
		const bool bPinBSuccess = CreateIntermediateCast(this, CompilerContext, SourceGraph, OriginalInputPins[i], NewOpHelper.InputB);
		
		// Make a connection to a tolerance pin if both nodes have one
		const bool bToleranceSuccess = MyTolerancePin && NewOpHelper.TolerancePin ? UK2Node_PromotableOperator::CreateIntermediateCast(this, CompilerContext, SourceGraph, MyTolerancePin, NewOpHelper.TolerancePin) : true;

		if (!bPinASuccess || !bPinBSuccess || !bToleranceSuccess)
		{
			CompilerContext.MessageLog.Error(TEXT("'@@' could not successfuly expand additional pins!"), PrevIntermediateNode);
		}

		// Track what the previous node is so that we can connect it's output appropriately
		PrevOutputPin = NewOpHelper.OutputPin;
		PrevIntermediateNode = NewIntermediateNode;
	}

	// Make the final output connection that we need
	if (OriginalOutputPin && PrevOutputPin)
	{
		CompilerContext.MovePinLinksToIntermediate(*OriginalOutputPin, *PrevOutputPin);
		// If there is no link to the output pin then the connection response called for a conversion node,
		// but one was not available. This can occur when there is a connection to the output pin that 
		// is smaller than an input and cannot be casted. We throw a compiler error instead of auto-breaking
		// pins because that can be confusing to the user.
		if (PrevOutputPin->LinkedTo.Num() == 0)
		{
			CompilerContext.MessageLog.Error(
				*FText::Format(LOCTEXT("FailedOutputConnection_ErrorFmt",
				"Output pin type '{1}' is not compatible with input type of '{0}' on '@@'"),
				Schema->TypeToText(PrevOutputPin->PinType),
				Schema->TypeToText(OriginalOutputPin->PinType)).ToString(),
				PrevIntermediateNode
			);
		}
	}
}

void UK2Node_PromotableOperator::NotifyPinConnectionListChanged(UEdGraphPin* ChangedPin)
{
	Super::NotifyPinConnectionListChanged(ChangedPin);

	EvaluatePinsFromChange(ChangedPin);
}

void UK2Node_PromotableOperator::PostReconstructNode()
{
	Super::PostReconstructNode();

	// We only need to set the function if we have connections, otherwise we should stick in a wildcard state
	if (HasAnyConnectionsOrDefaults())
	{
		if (UEdGraphPin* TolerancePin = FindTolerancePin())
		{
			FEdGraphPinType ToleranceType;
			TolerancePin->bHidden = !PromotableOpUtils::FindTolerancePinType(GetTargetFunction(), ToleranceType);
		}

		// Allocate default pins will have been called before this, which means we are reset to wildcard state
		// We need to Update the pins to be the proper function again
		UpdatePinsFromFunction(GetTargetFunction());

		for (UEdGraphPin* AddPin : Pins)
		{
			if (ensure(AddPin) && IsAdditionalPin(*AddPin) && AddPin->LinkedTo.Num() > 0)
			{
				FEdGraphPinType TypeToSet = FTypePromotion::GetPromotedType(AddPin->LinkedTo);
				AddPin->PinType = TypeToSet;
			}
		}
	}
	else if (UEdGraphPin* TolerancePin = FindTolerancePin())
	{
		TolerancePin->bHidden = true;
	}
}

bool UK2Node_PromotableOperator::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	check(MyPin && OtherPin);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Container types cannot be promotable operators as they have their own special case wildcard propagation
	if (OtherPin->PinType.IsContainer())
	{
		OutReason = LOCTEXT("NoExecPinsAllowed", "Promotable Operator nodes cannot have containers or references.").ToString();
		return true;
	}
	// The output pin on comparison operators is always a boolean, and cannot have its type changed
	// so we need to just treat it normally as a regular K2CallFunction node would
	else if (MyPin == GetOutputPin() && FTypePromotion::IsComparisonFunc(GetTargetFunction()) && OtherPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Boolean)
	{
		return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
	}
	// If the pins are the same type then there is no reason to check for a promotion
	else if (MyPin->PinType == OtherPin->PinType)
	{
		return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
	}
	// Enums need to be casted to a byte manually before we can do math with them, like in C++
	else if (!FWildcardNodeUtils::IsWildcardPin(MyPin) && MyPin->Direction == EGPD_Input && OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Byte && OtherPin->PinType.PinSubCategoryObject != nullptr)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("OtherPinType"), K2Schema->TypeToText(OtherPin->PinType));

		OutReason = FText::Format(LOCTEXT("NoCompatibleEnumConv", "'{OtherPinType}' must be converted to a numeric type before being connected"), Args).ToString();
		return true;
	}
	else if (FWildcardNodeUtils::IsWildcardPin(MyPin) && !FWildcardNodeUtils::IsWildcardPin(OtherPin))
	{
		TArray<UEdGraphPin*> PinsToConsider;
		PinsToConsider.Add(const_cast<UEdGraphPin*>(OtherPin));

		if (!FTypePromotion::FindBestMatchingFunc(OperationName, PinsToConsider))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("OtherPinType"), K2Schema->TypeToText(OtherPin->PinType));
			Args.Add(TEXT("OpName"), FText::FromName(OperationName));

			const TSet<FName>& DenyList = GetDefault<UBlueprintEditorSettings>()->TypePromotionPinDenyList;
			if (DenyList.Contains(OtherPin->PinType.PinCategory))
			{
				OutReason = FText::Format(LOCTEXT("NoCompatibleStructConv_Denied", "No matching '{OpName}' function for '{OtherPinType}'. This type is listed as denied by TypePromotionPinDenyList in the blueprint editor settings."), Args).ToString();
			}
			else
			{
				OutReason = FText::Format(LOCTEXT("NoCompatibleStructConv", "No matching '{OpName}' function for '{OtherPinType}'"), Args).ToString();
			}

			return true;
		}
	}

	const bool bHasStructPin = MyPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct || OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct;

	// If the other pin can be promoted to my pin type, than allow the connection
	if (FTypePromotion::IsValidPromotion(OtherPin->PinType, MyPin->PinType))
	{
		if (bHasStructPin)
		{
			// Compare the directions
			const UEdGraphPin* InputPin = nullptr;
			const UEdGraphPin* OutputPin = nullptr;

			if (!K2Schema->CategorizePinsByDirection(MyPin, OtherPin, /*out*/ InputPin, /*out*/ OutputPin))
			{
				OutReason = LOCTEXT("DirectionsIncompatible", "Pin directions are not compatible!").ToString();
				return true;
			}

			if (!FTypePromotion::HasStructConversion(InputPin, OutputPin))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("MyPinType"), K2Schema->TypeToText(MyPin->PinType));
				Args.Add(TEXT("OtherPinType"), K2Schema->TypeToText(OtherPin->PinType));

				OutReason = FText::Format(LOCTEXT("NoCompatibleOperatorConv", "No compatible operator functions between '{MyPinType}' and '{OtherPinType}'"), Args).ToString();
				return true;
			}
		}
		return false;
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

void UK2Node_PromotableOperator::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// We need to fix up any additional pins that may have been created as a wildcard pin
	int32 AdditionalPinsFixed = 0;

	// Additional Pin creation here? Check for orphan pins here and see if we can re-create them
	for (UEdGraphPin* OldPin : OldPins)
	{
		if (ensure(OldPin))
		{
			if (IsAdditionalPin(*OldPin))
			{
				if (UEdGraphPin* AddPin = GetAdditionalPin(AdditionalPinsFixed + NumFunctionInputs))
				{
					AddPin->PinType = OldPin->PinType;
					AddPin->DefaultValue = OldPin->DefaultValue;
					++AdditionalPinsFixed;
				}
			}

			// Copy the default value and pin type for pins without any links to preseve it
			UEdGraphPin* CurrentPin = FindPin(OldPin->GetFName(), OldPin->Direction);

			if (CurrentPin && OldPin->LinkedTo.Num() == 0 && !OldPin->DoesDefaultValueMatchAutogenerated())
			{
				CurrentPin->PinType = OldPin->PinType;
				CurrentPin->DefaultValue = OldPin->DefaultValue;
			}
		}
	}
}

void UK2Node_PromotableOperator::AutowireNewNode(UEdGraphPin* ChangedPin)
{
	Super::AutowireNewNode(ChangedPin);

	EvaluatePinsFromChange(ChangedPin);
}

void UK2Node_PromotableOperator::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	Super::GetPinHoverText(Pin, HoverTextOut);

	static const FText RightClickToConv = LOCTEXT("RightClickToConvTooltip", "\n\nRight click this pin to convert its type.");
	HoverTextOut += RightClickToConv.ToString();
}

///////////////////////////////////////////////////////////
// IK2Node_AddPinInterface

void UK2Node_PromotableOperator::AddInputPin()
{
	if (CanAddPin())
	{
		FScopedTransaction Transaction(LOCTEXT("AddPinPromotableOperator", "AddPin"));
		Modify();

		AddInputPinImpl(NumFunctionInputs + NumAdditionalInputs);
		++NumAdditionalInputs;

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

bool UK2Node_PromotableOperator::CanAddPin() const
{
	return ((NumAdditionalInputs + NumFunctionInputs) < GetMaxInputPinsNum()) &&
		!FTypePromotion::IsComparisonFunc(GetTargetFunction());
}

bool UK2Node_PromotableOperator::CanRemovePin(const UEdGraphPin* Pin) const
{
	return (
		Pin && Pin->ParentPin == nullptr &&
		NumAdditionalInputs > 0 &&
		INDEX_NONE != Pins.IndexOfByKey(Pin) &&
		Pin->Direction == EEdGraphPinDirection::EGPD_Input
		);
}

void UK2Node_PromotableOperator::RemoveInputPin(UEdGraphPin* Pin)
{
	if (CanRemovePin(Pin))
	{
		FScopedTransaction Transaction(LOCTEXT("RemovePinPromotableOperator", "RemovePin"));
		Modify();

		if (RemovePin(Pin))
		{
			--NumAdditionalInputs;

			int32 NameIndex = 0;
			const UEdGraphPin* SelfPin = FindPin(UEdGraphSchema_K2::PN_Self);

			for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
			{
				UEdGraphPin* LocalPin = Pins[PinIndex];

				if (LocalPin && (LocalPin->Direction != EGPD_Output) && (LocalPin != SelfPin) && LocalPin->ParentPin == nullptr)
				{
					const FName PinName = GetNameForAdditionalPin(NameIndex);
					if (PinName != LocalPin->PinName)
					{
						LocalPin->Modify();
						LocalPin->PinName = PinName;
					}
					NameIndex++;
				}
			}
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
		}
	}

	// If the pin that was removed makes this node empty then we can just reset it 
	// and have no need to reevaluate any pins
	if (!HasAnyConnectionsOrDefaults())
	{
		ResetNodeToWildcard();
	}
}

UEdGraphPin* UK2Node_PromotableOperator::GetAdditionalPin(int32 PinIndex) const
{
	const FName PinToFind = GetNameForAdditionalPin(PinIndex);

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->PinName == PinToFind)
		{
			return Pin;
		}
	}

	return nullptr;
}

UEdGraphPin* UK2Node_PromotableOperator::FindTolerancePin() const
{
	return FindPin(TolerancePin_Name, EGPD_Input);
}

///////////////////////////////////////////////////////////
// UK2Node_CallFunction interface
void UK2Node_PromotableOperator::SetFromFunction(const UFunction* Function)
{
	if(Function)
	{
		OperationName = FTypePromotion::GetOpNameFromFunction(Function);
	}

	// During compilation of an Anim BP, if this node gets pruned then the outer will be the /Engine/Transient/
	// package which will result in this node not having a valid SelfContext, so there is no need
	// to set any further information based on this function as it will not be used. 
	if(GetBlueprintClassFromNode())
	{
		Super::SetFromFunction(Function);
	}
}

///////////////////////////////////////////////////////////
// UK2Node_PromotableOperator

bool UK2Node_PromotableOperator::IsTolerancePin(const UEdGraphPin& Pin) const
{
	return Pin.PinName == TolerancePin_Name && Pin.Direction == EGPD_Input;
}

UEdGraphPin* UK2Node_PromotableOperator::AddInputPinImpl(int32 PinIndex)
{
	const FName NewPinName = GetNameForAdditionalPin(PinIndex);

	UEdGraphPin* NewPin = FWildcardNodeUtils::CreateWildcardPin(this, NewPinName, EGPD_Input);
	check(NewPin);

	// Determine a default type for this pin if we have other input connections
	const TArray<UEdGraphPin*> InputPins = GetInputPins(/* bIncludeLinks = */ true);
	check(InputPins.Num());
	FEdGraphPinType PromotedType = FTypePromotion::GetPromotedType(InputPins);

	NewPin->PinType = PromotedType;

	return NewPin;
}

bool UK2Node_PromotableOperator::IsAdditionalPin(const UEdGraphPin& Pin) const
{
	// Quickly check if this input pin is one of the two default input pins
	bool bIsDefaultPinA =
		(Pin.PinName == InputPinA_Name) ||
		(Pin.ParentPin && (Pin.ParentPin->PinName == InputPinA_Name));

	bool bIsDefaultPinB =
		(Pin.PinName == InputPinB_Name) ||
		(Pin.ParentPin && (Pin.ParentPin->PinName == InputPinB_Name));

	return (Pin.Direction == EGPD_Input) && !bIsDefaultPinA && !bIsDefaultPinB && !IsTolerancePin(Pin);
}

bool UK2Node_PromotableOperator::HasAnyConnectionsOrDefaults() const
{
	if(FTypePromotion::IsComparisonOpName(OperationName))
	{
		return HasDeterminingComparisonTypes();
	}

	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin->LinkedTo.Num() > 0 || !Pin->DoesDefaultValueMatchAutogenerated())
		{
			return true;
		}
	}
	return false;
}

bool UK2Node_PromotableOperator::HasDeterminingComparisonTypes() const
{
	if(!FTypePromotion::IsComparisonOpName(OperationName))
	{
		return false;
	}

	// For comparison ops, we only want to check the input pins as the output
	// will always be a bool
	for (const UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->Direction == EGPD_Input && (Pin->LinkedTo.Num() > 0 || !Pin->DoesDefaultValueMatchAutogenerated()))
		{
			return true;
		}
	}
	return false;
}

void UK2Node_PromotableOperator::EvaluatePinsFromChange(UEdGraphPin* ChangedPin, const bool bFromConversion /* = false*/)
{
	UpdateOpName();

	if (!ensureMsgf(ChangedPin, TEXT("UK2Node_PromotableOperator::EvaluatePinsFromChange failed to evaluate on a null pin!")))
	{
		return;
	}

	// True if the pin that has changed now has zero connections
	const bool bWasAFullDisconnect = (ChangedPin->LinkedTo.Num() == 0) && !HasAnyConnectionsOrDefaults();
	const bool bIsComparisonOp = FTypePromotion::IsComparisonOpName(OperationName);

	// If we have been totally disconnected and don't have any non-default inputs, 
	// than we just reset the node to be a regular wildcard
	if (!bFromConversion && (bWasAFullDisconnect || (bIsComparisonOp && !HasDeterminingComparisonTypes())))
	{
		ResetNodeToWildcard();
		return;
	}
	// If the pin that was connected is linked to a wildcard pin, then we should make it a wildcard
	// and do nothing else.
	else if (ChangedPin->GetOwningNodeUnchecked() == this && FWildcardNodeUtils::IsLinkedToWildcard(ChangedPin))
	{
		return;
	}
	// Changing the tolerance pin doesn't effect the rest of the function signature, so don't attempt to update the func
	else if (IsTolerancePin(*ChangedPin))
	{
		return;
	}
	// If we have connected the output of our comparison operator (which is always a bool) then 
	// there is no need to update the other pins
	else if (bIsComparisonOp && ChangedPin == GetOutputPin())
	{
		return;
	}

	// If the newest connection to this this pin is a different type,
	// then we need to break all other type connections that are not the same
	if(ChangedPin->LinkedTo.Num() > 1)
	{
		const FEdGraphPinType& MostRecentConnection = ChangedPin->LinkedTo.Last()->PinType;
		for(int32 i = ChangedPin->LinkedTo.Num() - 1; i >= 0; --i)
		{
			UEdGraphPin* Link = ChangedPin->LinkedTo[i];

			const bool bIsRealNumberConnection =
				(Link->PinType.PinCategory == UEdGraphSchema_K2::PC_Real) &&
				(MostRecentConnection.PinCategory == UEdGraphSchema_K2::PC_Real);

			// Real numbers are an exception to the normal link breaking rules.
			// Even if the subcategories don't match, they're still valid connections
			// since we implicitly cast between float and double types.
			if (bIsRealNumberConnection)
			{
				continue;
			}

			if( Link->PinType.PinCategory != MostRecentConnection.PinCategory ||
				Link->PinType.PinSubCategory != MostRecentConnection.PinSubCategory ||
				Link->PinType.PinSubCategoryObject != MostRecentConnection.PinSubCategoryObject
			)
			{
				ChangedPin->BreakLinkTo(Link);				
			}
		}
	}
	
	// Gather all pins and their links so we can determine the highest type
	TArray<UEdGraphPin*> PinsToConsider;
	GetPinsToConsider(PinsToConsider);

	if (bFromConversion)
	{
		PinsToConsider.AddUnique(ChangedPin);
	}

	const UFunction* BestMatchingFunc = FTypePromotion::FindBestMatchingFunc(OperationName, PinsToConsider);

	// Store these other function options for later so that the user can convert to them later
	UpdatePinsFromFunction(BestMatchingFunc, ChangedPin, bFromConversion);
}

bool UK2Node_PromotableOperator::UpdateOpName()
{

	// If the function is null then return false, because we did not successfully update it. 
	// This could be possible during node reconstruction/refresh, and we don't want to set the 
	// op name to "Empty" incorrectly. 
	if (const UFunction* Func = GetTargetFunction())
	{
		OperationName = FTypePromotion::GetOpNameFromFunction(Func);
		return true;
	}
	return false;
}

UK2Node_CallFunction* UK2Node_PromotableOperator::CreateIntermediateNode(UK2Node_CallFunction* PreviousNode, const UFunction* const OpFunction, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	// Spawn an intermediate UK2Node_CallFunction node of the function type we need
	UK2Node_CallFunction* NewOperator = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
	NewOperator->SetFromFunction(OpFunction);
	NewOperator->AllocateDefaultPins();

	// Move this node next to the thing it was linked to
	NewOperator->NodePosY = PreviousNode->NodePosY + 50;
	NewOperator->NodePosX = PreviousNode->NodePosX + 8;

	CompilerContext.MessageLog.NotifyIntermediateObjectCreation(NewOperator, this);

	return NewOperator;
}

bool UK2Node_PromotableOperator::CreateIntermediateCast(UK2Node_CallFunction* SourceNode, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, UEdGraphPin* InputPin, UEdGraphPin* OutputPin)
{
	check(InputPin && OutputPin);
	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	// If the pin types are the same, than no casts are needed and we can just connect.
	// This includes real number types that may have a different subcategory, which we implicitly convert.
	const bool bPinTypesMatch =
		(InputPin->PinType == OutputPin->PinType) ||
		((InputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real) && (OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Real));

	if (bPinTypesMatch)
	{
		// If SourceNode is 'this' then we need to move the pin links instead of just 
		// creating the connection because the output is not another new node, but 
		// just the intermediate expansion node.
		if (SourceNode == this)
		{
			return !CompilerContext.MovePinLinksToIntermediate(*InputPin, *OutputPin).IsFatal();
		}
		else
		{
			return Schema->TryCreateConnection(InputPin, OutputPin);
		}
	}

	UK2Node* TemplateConversionNode = nullptr;
	TSubclassOf<UK2Node> ConversionNodeClass;

	if (TOptional<UEdGraphSchema_K2::FSearchForAutocastFunctionResults> AutoCastResults = Schema->SearchForAutocastFunction(InputPin->PinType, OutputPin->PinType))
	{
		// Create a new call function node for the casting operator
		UK2Node_CallFunction* TemplateNode = SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
		TemplateNode->FunctionReference.SetExternalMember(AutoCastResults->TargetFunction, AutoCastResults->FunctionOwner);
		TemplateConversionNode = TemplateNode;
		TemplateNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(TemplateNode, this);
	}
	else if (TOptional<UEdGraphSchema_K2::FFindSpecializedConversionNodeResults> ConversionNodeResults = Schema->FindSpecializedConversionNode(InputPin->PinType, *OutputPin, true))
	{
		FVector2D AverageLocation = UEdGraphSchema_K2::CalculateAveragePositionBetweenNodes(InputPin, OutputPin);		
		TemplateConversionNode = FEdGraphSchemaAction_K2NewNode::SpawnNodeFromTemplate<UK2Node>(SourceGraph, ConversionNodeResults->TargetNode, AverageLocation);
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(TemplateConversionNode, this);
	}

	bool bInputSuccessful = false;
	bool bOutputSuccessful = false;

	if (TemplateConversionNode)
	{
		UEdGraphPin* ConversionInput = nullptr;
		UEdGraphPin* ConversionOutput = TemplateConversionNode->FindPin(UEdGraphSchema_K2::PN_ReturnValue, EGPD_Output);

		for (UEdGraphPin* ConvPin : TemplateConversionNode->Pins)
		{
			if (ConvPin)
			{
				if (!ConversionInput && ConvPin->Direction == EGPD_Input && ConvPin->PinName != UEdGraphSchema_K2::PSC_Self)
				{
					ConversionInput = ConvPin;
				}
				else if (!ConversionOutput && ConvPin->Direction == EGPD_Output)
				{
					ConversionOutput = ConvPin;
				}
			}
		}

		ensure(ConversionInput && ConversionOutput);

		// Connect my input to the conversion node directly if we have links, otherwise we need to move the intermediate version of it
		if (InputPin->LinkedTo.Num() > 0)
		{
			bInputSuccessful = Schema->TryCreateConnection(InputPin->LinkedTo[0], ConversionInput);
		}
		else if (InputPin && ConversionInput)
		{
			bInputSuccessful = !CompilerContext.MovePinLinksToIntermediate(*InputPin, *ConversionInput).IsFatal();
		}

		// Connect conversion node output to the input of the new operator
		bOutputSuccessful = Schema->TryCreateConnection(ConversionOutput, OutputPin);

		// Move this node next to the thing it was linked to
		TemplateConversionNode->NodePosY = SourceNode->NodePosY;
		TemplateConversionNode->NodePosX = SourceNode->NodePosX + 4;
	}
	else
	{
		CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("NoValidPromotion", "Cannot find appropriate promotion from '{0}' to '{1}' on '@@'"),
			Schema->TypeToText(InputPin->PinType),
			Schema->TypeToText(OutputPin->PinType)).ToString(),
			SourceNode
		);
	}

	return bInputSuccessful && bOutputSuccessful;
}

void UK2Node_PromotableOperator::ResetNodeToWildcard()
{
	RecombineAllSplitPins();

	// Reset type to wildcard
	const FEdGraphPinType& WildType = FWildcardNodeUtils::GetDefaultWildcardPinType();
	const UEdGraphSchema* Schema = GetSchema();

	for (UEdGraphPin* Pin : Pins)
	{
		// Ensure this pin is not a split pin
		if (Pin && Pin->ParentPin == nullptr)
		{
			Pin->PinType = WildType;
			Schema->ResetPinToAutogeneratedDefaultValue(Pin);
		}
	}

	const UFunction* Func = GetTargetFunction();

	if(Func && FTypePromotion::IsComparisonFunc(Func))
	{
		// Set output pins to have a bool output flag by default
		if (UEdGraphPin* OutPin = GetOutputPin())
		{
			OutPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		}

		// If we have a tolerance pin, and we reset to wildcard then we should hide it
		if (UEdGraphPin* TolerancePin = FindTolerancePin())
		{
			TolerancePin->bHidden = true;
		}
	}

	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_PromotableOperator::RecombineAllSplitPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Gather what pins need to be recombined from a split pin	
	for (int32 Index = 0; Index < Pins.Num(); ++Index)
	{
		if (Pins[Index] && Pins[Index]->SubPins.Num() > 0)
		{
			K2Schema->RecombinePin(Pins[Index]);
		}
	}
}

void UK2Node_PromotableOperator::UpdateFromBestMatchingFunction()
{
	// Gather all pins and their links so we can determine the highest type that the user could want
	TArray<UEdGraphPin*> PinsToConsider;
	GetPinsToConsider(PinsToConsider);

	// We need to update the pins from our function if have a new connection
	if (const UFunction* BestMatchingFunc = FTypePromotion::FindBestMatchingFunc(OperationName, PinsToConsider))
	{
		UpdatePinsFromFunction(BestMatchingFunc);
	}
}

TArray<UEdGraphPin*> UK2Node_PromotableOperator::GetInputPins(bool bIncludeLinks /** = false */) const
{
	TArray<UEdGraphPin*> InputPins;
	for (UEdGraphPin* Pin : Pins)
	{
		// Exclude split pins from this
		if (Pin && Pin->Direction == EEdGraphPinDirection::EGPD_Input && Pin->ParentPin == nullptr)
		{
			InputPins.Add(Pin);
			if (bIncludeLinks)
			{
				for (UEdGraphPin* Link : Pin->LinkedTo)
				{
					InputPins.Emplace(Link);
				}
			}
		}
	}
	return InputPins;
}

void UK2Node_PromotableOperator::GetPinsToConsider(TArray<UEdGraphPin*>& OutArray) const
{
	const bool bIsComparisonOp = FTypePromotion::IsComparisonOpName(OperationName);

	for (UEdGraphPin* Pin : Pins)
	{
		if (ensure(Pin))
		{
			// Tolerance pins don't factor into what types we should be matching 
			// for the function, so we should not consider them
			if (IsTolerancePin(*Pin))
			{
				continue;
			}

			// If this is a comparison operator then we don't need to consider the boolean output
			// pin because every comparison function will have a boolean output!
			if (bIsComparisonOp && Pin->Direction == EGPD_Output)
			{
				continue;
			}

			// If this pin has links, then use those instead of the actual pin because we could be process of changing it
			// which means that it would still have it's old pin type, and could be inaccurate
			if (Pin->LinkedTo.Num() > 0)
			{
				// If this is from a split pin, then we care about the parent type
				if (Pin->ParentPin != nullptr)
				{
					OutArray.Add(Pin->ParentPin);
				}

				// as well as all the links to it
				for (UEdGraphPin* Link : Pin->LinkedTo)
				{
					OutArray.Emplace(Link);
				}
			}
			else if (!FWildcardNodeUtils::IsWildcardPin(Pin))
			{
				// If this is from a split pin, then we care about the type of the parent, not this pin
				if (Pin->ParentPin != nullptr)
				{
					OutArray.Add(Pin->ParentPin);
				}
				else
				{
					OutArray.Add(Pin);
				}
			}
		}
	}
}

void UK2Node_PromotableOperator::UpdatePinsFromFunction(const UFunction* Function, UEdGraphPin* ChangedPin /* = nullptr */, bool bIsFromConversion /*= false*/)
{
	if (!Function)
	{
		return;
	}

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// Gather the pin types of the properties on the function we want to convert to
	FEdGraphPinType FunctionReturnType;
	FEdGraphPinType HighestFuncInputType;
	FEdGraphPinType ToleranceType;
	TArray<FEdGraphPinType> FunctionInputTypes;
	{
		for (TFieldIterator<FProperty> PropIt(Function); PropIt && (PropIt->PropertyFlags & CPF_Parm); ++PropIt)
		{
			FProperty* Param = *PropIt;
			FEdGraphPinType ParamType;

			if (Schema->ConvertPropertyToPinType(Param, /* out */ ParamType))
			{
				if (Param->HasAnyPropertyFlags(CPF_ReturnParm))
				{
					FunctionReturnType = ParamType;
				}
				else
				{
					// Track the highest input pin type that we have
					if (FTypePromotion::GetHigherType(HighestFuncInputType, ParamType) == FTypePromotion::ETypeComparisonResult::TypeBHigher)
					{
						HighestFuncInputType = ParamType;
					}
					FunctionInputTypes.Add(ParamType);
				}
			}
		}
	}

	auto ConformPinLambda = [&ChangedPin, bIsFromConversion](const FEdGraphPinType& FunctionPinType, UEdGraphPin* NodePin)
	{
		using namespace UE::Kismet::BlueprintTypeConversions;

		// If the pin types are already equal, then we don't have to do any work
		// If this is linked to wildcard pins, then we can just ignore it and handle it on expansion
		if (!NodePin || FWildcardNodeUtils::IsLinkedToWildcard(NodePin))
		{
			return;
		}

		// Pins that are underdoing a conversion will have already had their types changed
		if (NodePin == ChangedPin && bIsFromConversion)
		{
			return;
		}

		// By default, conform to the type of the function param
		FEdGraphPinType ConformingType = FunctionPinType;
		const FEdGraphPinType HighestLinkedType = NodePin->LinkedTo.Num() > 0 ? FTypePromotion::GetPromotedType(NodePin->LinkedTo) : NodePin->PinType;
		const UScriptStruct* NodePinStruct = Cast<UScriptStruct>(HighestLinkedType.PinSubCategoryObject);
		const UScriptStruct* FunctionPinStruct = Cast<UScriptStruct>(FunctionPinType.PinSubCategoryObject);
		const bool bHasImplicitConversionFunction = FStructConversionTable::Get().GetConversionFunction(NodePinStruct, FunctionPinStruct).IsSet();
		const bool bDifferingStructs =
			HighestLinkedType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
			FunctionPinType.PinCategory == UEdGraphSchema_K2::PC_Struct &&
			HighestLinkedType.PinSubCategoryObject != FunctionPinType.PinSubCategoryObject &&
			!bHasImplicitConversionFunction;

		const bool bHasDeterminingType = NodePin->LinkedTo.Num() > 0 || !NodePin->DoesDefaultValueMatchAutogenerated();

		// If the highest type is the same as the function type, just continue on with life
		if (bHasDeterminingType && (HighestLinkedType.PinCategory != FunctionPinType.PinCategory || bDifferingStructs))
		{
			NodePin->PinType = FunctionPinType;
			const bool bValidPromo = 
				FTypePromotion::IsValidPromotion(HighestLinkedType, FunctionPinType) || 
				(NodePin->LinkedTo.Num() > 0 && FTypePromotion::HasStructConversion(NodePin, NodePin->LinkedTo[0]));

			// If the links cannot be promoted to the function type, then we need to break them
			// We don't want to break the pin if it is the one that the user has dragged on to though,
			// because that would result in the node breaking connection as soon as the user lets go
			if ((NodePin != ChangedPin) && !FWildcardNodeUtils::IsWildcardPin(NodePin) && !bValidPromo)
			{
				NodePin->BreakAllPinLinks();
			}
			else
			{
				ConformingType = HighestLinkedType;
			}
		}

		// Conform the pin type appropriately
		NodePin->PinType = ConformingType;
	};

	// Check if we need to add a tolerance pin or not with this new function
	if (FTypePromotion::IsComparisonFunc(Function))
	{
		UEdGraphPin* ExistingTolPin = FindTolerancePin();
		const bool bHasTolerancePin = PromotableOpUtils::FindTolerancePinType(Function, ToleranceType);

		if (!ExistingTolPin)
		{
			ExistingTolPin = CreatePin(EGPD_Input, ToleranceType, TolerancePin_Name);
		}

		// Set the tolerance pin to visible if it is currently on the node
		ExistingTolPin->bHidden = !bHasTolerancePin;
	}

	int32 CurPinIndex = 0;
	for (UEdGraphPin* CurPin : Pins)
	{
		if (ensure(CurPin))
		{
			// We don't want to try and conform split pin, because we will already have conformed the parent pin
			if (CurPin->ParentPin != nullptr)
			{
				continue;
			}

			if (IsAdditionalPin(*CurPin))
			{
				// Conform to the highest input pin on the function
				ConformPinLambda(HighestFuncInputType, CurPin);
			}
			else if (CurPin->Direction == EGPD_Output)
			{
				// Match to the output pin
				ConformPinLambda(FunctionReturnType, CurPin);
			}
			// Creation and conformation of the tolerance pin is handled before all others to 
			// ensure that connections are not broken accidentally
			else if (IsTolerancePin(*CurPin))
			{
				ConformPinLambda(ToleranceType, CurPin);
			}
			else
			{
				// Match to the appropriate function input type
				ConformPinLambda(FunctionInputTypes[CurPinIndex], CurPin);
				++CurPinIndex;
			}
		}
	}

	// Update the function reference and the FUNC_BlueprintPure/FUNC_Const appropriately
	SetFromFunction(Function);

	// Invalidate the tooltips
	CachedTooltip.MarkDirty();

	// We need to notify the graph that the node has changed to get 
	// the correct default value text boxes on the node
	GetGraph()->NotifyNodeChanged(this);
}

UEdGraphPin* UK2Node_PromotableOperator::GetOutputPin() const
{
	for (UEdGraphPin* Pin : Pins)
	{
		if (Pin && Pin->Direction == EGPD_Output)
		{
			return Pin;
		}
	}

	return nullptr;
}

void UK2Node_PromotableOperator::ConvertPinType(UEdGraphPin* PinToChange, const FEdGraphPinType NewPinType)
{
	// No work to be done here!
	if (!PinToChange || PinToChange->PinType == NewPinType)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("PromotableOperatorPinConvert", "Convert pin type"));
	Modify();

	if(NewPinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
	{
		const bool bIsComparison = FTypePromotion::IsComparisonFunc(GetTargetFunction());
		// Break all input connections, but only break an output if it is not a comparison
		// because the bool pin type will not change.
		for(UEdGraphPin* Pin : Pins)
		{
			if(Pin->Direction == EGPD_Input || !bIsComparison)
			{
				Pin->BreakAllPinLinks();
			}
		}		
		
		ResetNodeToWildcard();
		return;
	}

	// Break any pin links to this node because the type will be different
	PinToChange->BreakAllPinLinks();

	// Recombine any split pins that this type has before changing its type
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if (PinToChange->SubPins.Num() > 0)
	{
		Schema->RecombinePin(PinToChange);
	}

	PinToChange->PinType = NewPinType;

	EvaluatePinsFromChange(PinToChange, true);

	InvalidatePinTooltips();

	// Reset the default value on pins that have been converted
	Schema->ResetPinToAutogeneratedDefaultValue(PinToChange, false);
}

#undef LOCTEXT_NAMESPACE	// "PromotableOperatorNode"
