// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_PureAssignmentStatement.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"
#include "VariableSetHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PureAssignmentStatement)

struct FBPTerminal;

#define LOCTEXT_NAMESPACE "K2Node_PureAssignmentStatement"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_PureAssignmentStatement

class FKCHandler_PureAssignmentStatement : public FKCHandler_VariableSet
{
public:
	FKCHandler_PureAssignmentStatement(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_VariableSet(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		auto PureAssignmentNode = CastChecked<UK2Node_PureAssignmentStatement>(Node);
		{
			auto VariablePin = PureAssignmentNode->GetVariablePin();
			if (VariablePin->LinkedTo.Num() == 0)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("NoVariableConnected_Error", "A variable needs to be connected to @@").ToString(), VariablePin);
				return;
			}

			auto OutputPin = PureAssignmentNode->GetOutputPin();
			if (OutputPin->LinkedTo.Num() == 0)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("NoOutputConnected_Error", "A output pin needs to be connected to @@").ToString(), OutputPin);
				return;
			}

			// At the moment, a term for variable should be already registered
			auto VariableTermPtr = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(VariablePin));
			if (!VariableTermPtr || !*VariableTermPtr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("NoVarriableTerm_Error", "ICE: no variable term found in @@").ToString(), Node);
				return;
			}
			FBPTerminal* VariableTerm = *VariableTermPtr; // we must take a copy here because Add takes a reference and the map might be resized
			Context.NetMap.Add(OutputPin, VariableTerm);
		}

		auto ValuePin = PureAssignmentNode->GetValuePin();
		ValidateAndRegisterNetIfLiteral(Context, ValuePin);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		auto PureAssignmentNode = CastChecked<UK2Node_PureAssignmentStatement>(Node);
		auto VariablePin = PureAssignmentNode->GetVariablePin();
		auto ValuePin = PureAssignmentNode->GetValuePin();

		InnerAssignment(Context, Node, VariablePin, ValuePin);
	}
};

// The node is (was originally) needed in expansion step for handling CreateAutoRefTerm parameters in pure functions 

FName UK2Node_PureAssignmentStatement::VariablePinName = TEXT("Variable");
FName UK2Node_PureAssignmentStatement::ValuePinName = TEXT("Value");
FName UK2Node_PureAssignmentStatement::OutputPinName = UEdGraphSchema_K2::PN_ReturnValue;


UK2Node_PureAssignmentStatement::UK2Node_PureAssignmentStatement(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_PureAssignmentStatement::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, VariablePinName);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, ValuePinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, OutputPinName);

	Super::AllocateDefaultPins();
}

void UK2Node_PureAssignmentStatement::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	if (Pin->LinkedTo.Num() > 0)
	{
		const FEdGraphPinType PinType = Pin->LinkedTo[0]->PinType;

		UEdGraphPin* VariablePin = GetVariablePin();
		VariablePin->PinType = PinType;
		UEdGraphSchema_K2::ValidateExistingConnections(VariablePin);

		UEdGraphPin* OutputPin = GetOutputPin();
		OutputPin->PinType = PinType;
		UEdGraphSchema_K2::ValidateExistingConnections(OutputPin);

		UEdGraphPin* ValuePin = GetValuePin();
		ValuePin->PinType = PinType;
		ValuePin->PinType.bIsReference = false;
		UEdGraphSchema_K2::ValidateExistingConnections(ValuePin);
	}
}

UEdGraphPin* UK2Node_PureAssignmentStatement::GetOutputPin() const
{
	return FindPinChecked(OutputPinName);
}

UEdGraphPin* UK2Node_PureAssignmentStatement::GetVariablePin() const
{
	return FindPinChecked(VariablePinName);
}

UEdGraphPin* UK2Node_PureAssignmentStatement::GetValuePin() const
{
	return FindPinChecked(ValuePinName);
}

FNodeHandlingFunctor* UK2Node_PureAssignmentStatement::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_PureAssignmentStatement(CompilerContext);
}

#undef LOCTEXT_NAMESPACE
