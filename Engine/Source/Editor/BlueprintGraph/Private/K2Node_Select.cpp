// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Select.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Select)

struct FLinearColor;

#define LOCTEXT_NAMESPACE "K2Node_Select"

namespace
{
	FName NAME_bPickOption0(TEXT("bPickOption0"));
	FName NAME_Index(TEXT("Index"));
	FName NAME_Option_0(TEXT("Option 0"));
	FName NAME_Option_1(TEXT("Option 1"));
}

//////////////////////////////////////////////////////////////////////////
// FKCHandler_Select

class FKCHandler_Select : public FNodeHandlingFunctor
{
protected:
	TMap<UEdGraphNode*, FBPTerminal*> DefaultTermMap;

public:
	FKCHandler_Select(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Select* SelectNode = Cast<UK2Node_Select>(Node);
		UEdGraphPin* ReturnPin = SelectNode ? SelectNode->GetReturnValuePin() : nullptr;
		if (!ReturnPin)
		{
			Context.MessageLog.Error(*LOCTEXT("Error_NoReturnPin", "No return pin in @@").ToString(), Node);
			return;
		}

		// return inline term
		if (Context.NetMap.Find(ReturnPin))
		{
			Context.MessageLog.Error(*LOCTEXT("Error_ReturnTermAlreadyRegistered", "ICE: Return term is already registered @@").ToString(), Node);
			return;
		}

		{
			FBPTerminal* Term = new FBPTerminal();
			Context.InlineGeneratedValues.Add(Term);
			Term->CopyFromPin(ReturnPin, Context.NetNameMap->MakeValidName(ReturnPin));
			Context.NetMap.Add(ReturnPin, Term);
		}

		//Register Default term
		{
			TArray<UEdGraphPin*> OptionPins;
			SelectNode->GetOptionPins(OptionPins);
			if (!OptionPins.Num())
			{
				Context.MessageLog.Error(*LOCTEXT("Error_NoOptionPin", "No option pin in @@").ToString(), Node);
				return;
			}

			FString DefaultTermName = Context.NetNameMap->MakeValidName(Node, TEXT("Default"));
			FBPTerminal* DefaultTerm = Context.CreateLocalTerminalFromPinAutoChooseScope(OptionPins[0], DefaultTermName);
			DefaultTermMap.Add(Node, DefaultTerm);
		}

		FNodeHandlingFunctor::RegisterNets(Context, Node);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Select* SelectNode = CastChecked<UK2Node_Select>(Node);
		FBPTerminal* DefaultTerm = nullptr;
		FBPTerminal* ReturnTerm = nullptr;
		FBPTerminal* IndexTerm = nullptr;

		{
			UEdGraphPin* IndexPin = SelectNode->GetIndexPin();
			UEdGraphPin* IndexPinNet = IndexPin ? FEdGraphUtilities::GetNetFromPin(IndexPin) : nullptr;
			FBPTerminal** IndexTermPtr = IndexPinNet ? Context.NetMap.Find(IndexPinNet) : nullptr;
			IndexTerm = IndexTermPtr ? *IndexTermPtr : nullptr;

			UEdGraphPin* ReturnPin = SelectNode->GetReturnValuePin();
			UEdGraphPin* ReturnPinNet = ReturnPin ? FEdGraphUtilities::GetNetFromPin(ReturnPin) : nullptr;
			FBPTerminal** ReturnTermPtr = ReturnPinNet ? Context.NetMap.Find(ReturnPinNet) : nullptr;
			ReturnTerm = ReturnTermPtr ? *ReturnTermPtr : nullptr;

			FBPTerminal** DefaultTermPtr = DefaultTermMap.Find(SelectNode);
			DefaultTerm = DefaultTermPtr ? *DefaultTermPtr : nullptr;
			
			if (!ReturnTerm || !IndexTerm || !DefaultTerm)
			{
				Context.MessageLog.Error(*LOCTEXT("Error_InvalidSelect", "ICE: invalid select node @@").ToString(), Node);
				return;
			}
		}

		FBlueprintCompiledStatement* SelectStatement = new FBlueprintCompiledStatement();
		SelectStatement->Type = EKismetCompiledStatementType::KCST_SwitchValue;
		Context.AllGeneratedStatements.Add(SelectStatement);
		ReturnTerm->InlineGeneratedParameter = SelectStatement;
		SelectStatement->RHS.Add(IndexTerm);

		TArray<UEdGraphPin*> OptionPins;
		SelectNode->GetOptionPins(OptionPins);

		for (int32 OptionIdx = 0; OptionIdx < OptionPins.Num(); ++OptionIdx)
		{
			{
				FBPTerminal* LiteralTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
				LiteralTerm->Type = IndexTerm->Type;
				LiteralTerm->bIsLiteral = true;
				const UEnum* NodeEnum = SelectNode->GetEnum();
				LiteralTerm->Name = NodeEnum ? OptionPins[OptionIdx]->PinName.ToString() : FString::Printf(TEXT("%d"), OptionIdx);

				if (!CompilerContext.GetSchema()->DefaultValueSimpleValidation(LiteralTerm->Type, *LiteralTerm->Name, LiteralTerm->Name, nullptr, FText()))
				{
					Context.MessageLog.Error(*FText::Format(LOCTEXT("Error_InvalidOptionValueFmt", "Invalid option value '{0}' in @@"), FText::FromString(LiteralTerm->Name)).ToString(), Node);
					return;
				}
				SelectStatement->RHS.Add(LiteralTerm);
			}

			{
				UEdGraphPin* NetPin = OptionPins[OptionIdx] ? FEdGraphUtilities::GetNetFromPin(OptionPins[OptionIdx]) : nullptr;
				FBPTerminal** ValueTermPtr = NetPin ? Context.NetMap.Find(NetPin) : nullptr;
				FBPTerminal* ValueTerm = ValueTermPtr ? *ValueTermPtr : nullptr;
				if (!ensure(ValueTerm))
				{
					Context.MessageLog.Error(*LOCTEXT("Error_NoTermFound", "No term registered for pin @@").ToString(), NetPin);
					return;
				}

				{
					using namespace UE::KismetCompiler;

					FBPTerminal* ImplicitCastEntry =
						CastingUtils::InsertImplicitCastStatement(Context, OptionPins[OptionIdx], ValueTerm);

					if (ImplicitCastEntry)
					{
						ValueTerm = ImplicitCastEntry;
					}
				}

				SelectStatement->RHS.Add(ValueTerm);
			}
		}

		SelectStatement->RHS.Add(DefaultTerm);
	}
};

UK2Node_Select::UK2Node_Select(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	NumOptionPins = 2;

	IndexPinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
	IndexPinType.PinSubCategory = UEdGraphSchema_K2::PSC_Index;
	IndexPinType.PinSubCategoryObject = nullptr;

	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveNone;
}

void UK2Node_Select::AllocateDefaultPins()
{
	// To refresh, just in case it changed
	SetEnum(Enum, true);

	// No need to reconstruct the node after force setting the enum, we are at the start of reconstruction already
	bReconstructNode = false;

	if (Enum)
	{
		NumOptionPins = EnumEntries.Num();
	}

	// Create the option pins
	for (int32 Idx = 0; Idx < NumOptionPins; Idx++)
	{
		UEdGraphPin* NewPin = nullptr;

		if (Enum)
		{
			const FName PinName = EnumEntries[Idx];
			UEdGraphPin* TempPin = FindPin(PinName);
			if (!TempPin)
			{
				NewPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, PinName);
			}
		}
		else
		{
			const FName PinName = *FString::Printf(TEXT("Option %d"), Idx);
			NewPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, PinName);
		}

		if (NewPin)
		{
			NewPin->bDisplayAsMutableRef = true;
			if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
			{
				const FCoreTexts& CoreTexts = FCoreTexts::Get();
				NewPin->PinFriendlyName = (Idx == 0 ? CoreTexts.False : CoreTexts.True);
			}
			else if (Idx < EnumEntryFriendlyNames.Num())
			{
				NewPin->PinFriendlyName = EnumEntryFriendlyNames[Idx];
			}
		}
	}

	// Create the index wildcard pin
	CreatePin(EGPD_Input, IndexPinType.PinCategory, IndexPinType.PinSubCategory, IndexPinType.PinSubCategoryObject.Get(), TEXT("Index"));

	// Create the return value
	UEdGraphPin* ReturnPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, UEdGraphSchema_K2::PN_ReturnValue);
	ReturnPin->bDisplayAsMutableRef = true;

	Super::AllocateDefaultPins();
}

void UK2Node_Select::AutowireNewNode(UEdGraphPin* FromPin)
{
	if (FromPin)
	{
		// Attempt to autowire to the index pin as users generally drag off of something intending to use
		// it as an index in a select statement rather than an arbitrary entry:
		const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
		UEdGraphPin* IndexPin = GetIndexPin();
		ECanCreateConnectionResponse ConnectResponse = K2Schema->CanCreateConnection(FromPin, IndexPin).Response;
		if (ConnectResponse == ECanCreateConnectionResponse::CONNECT_RESPONSE_MAKE)
		{
			if (K2Schema->TryCreateConnection(FromPin, IndexPin))
			{
				FromPin->GetOwningNode()->NodeConnectionListChanged();
				this->NodeConnectionListChanged();
				return;
			}
		}
	}

	// No connection made, just use default autowire logic:
	Super::AutowireNewNode(FromPin);
}

FText UK2Node_Select::GetTooltipText() const
{
	return LOCTEXT("SelectNodeTooltip", "Return the option at Index, (first option is indexed at 0)");
}

FText UK2Node_Select::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Select", "Select");
}

UK2Node::ERedirectType UK2Node_Select::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	if (bReconstructForPinTypeChange)
	{
		// If we're reconstructing for the purposes of changing the index pin type then we want to
		// keep our connections based on the index of the option pin
		if (NewPin != GetIndexPin() && NewPin != GetReturnValuePin())
		{
			if (NewPinIndex == OldPinIndex)
			{
				return UK2Node::ERedirectType_Name;
			}
		}
	}

	// Check to see if the new pin name matches the old pin name.
	if (Enum && (NewPinIndex < NumOptionPins) && (NewPin->PinName != OldPin->PinName))
	{
		// The names don't match, so check for an enum redirect from the old pin name.
		const int32 EnumIndex = Enum->GetIndexByName(OldPin->PinName);
		if(EnumIndex != INDEX_NONE)
		{
			// Found a redirect. Attempt to match it to the new pin name.
			// Can't use Enum->GetNameByIndex here because it doesn't do namespace mangling
			const FString NewPinName = Enum->GetNameStringByIndex(EnumIndex);
			if (NewPinName == NewPin->PinName.ToString())
			{
				// The redirect is a match, so we can reconstruct this pin using the old pin's state.
				return UK2Node::ERedirectType_Name;
			}
		}
	}

	// Fall back to base class functionality for all other cases.
	return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

void UK2Node_Select::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	// See if this node was saved in the old version with a boolean as the condition
	UEdGraphPin* OldConditionPin = nullptr;
	UEdGraphPin* OldIndexPin = nullptr;
	UEdGraphPin* OldReturnPin = nullptr;
	for (UEdGraphPin* OldPin : OldPins)
	{
		if (OldPin->PinName == NAME_bPickOption0)
		{
			OldConditionPin = OldPin;
		}
		else if (OldPin->PinName == NAME_Index)
		{
			OldIndexPin = OldPin;
		}
		else if (OldPin->PinName == UEdGraphSchema_K2::PN_ReturnValue)
		{
			OldReturnPin = OldPin;
		}
	}

	UEdGraphPin* ReturnPin = GetReturnValuePin();
	check(ReturnPin);

	if (OldReturnPin && (ReturnPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard))
	{
		// Always copy type from node prior, if pins have changed those will error at compilation time
		ReturnPin->PinType = OldReturnPin->PinType;
	}

	UEdGraphPin* IndexPin = GetIndexPin();
	check(IndexPin);

	// If we are fixing up an old bool node (swap the options and copy the condition links)
	if (OldConditionPin)
	{
		// Set the index pin type
		IndexPinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		IndexPinType.PinSubCategory = NAME_None;
		IndexPinType.PinSubCategoryObject = nullptr;

		// Set the pin type and Copy the pin
		IndexPin->PinType = IndexPinType;
		Schema->CopyPinLinks(*OldConditionPin, *IndexPin);
		// If we copy links, we need to send a notification
		if (IndexPin->LinkedTo.Num() > 0)
		{
			PinConnectionListChanged(IndexPin);
		}

		UEdGraphPin* OptionPin0 = FindPin(NAME_Option_0);
		UEdGraphPin* OptionPin1 = FindPin(NAME_Option_1);

		for (UEdGraphPin* OldPin : OldPins)
		{
			if (OldPin->PinName == OptionPin0->PinName)
			{
				Schema->MovePinLinks(*OldPin, *OptionPin1);
			}
			else if (OldPin->PinName == OptionPin1->PinName)
			{
				Schema->MovePinLinks(*OldPin, *OptionPin0);
			}
		}
	}

	// If the index pin has links or a default value but is a wildcard, this is an old int pin so convert it
	if (OldIndexPin &&
		IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard &&
		(OldIndexPin->LinkedTo.Num() > 0 || !OldIndexPin->DefaultValue.IsEmpty()))
	{
		IndexPinType.PinCategory = UEdGraphSchema_K2::PC_Int;
		IndexPinType.PinSubCategory = NAME_None;
		IndexPinType.PinSubCategoryObject = nullptr;
		IndexPin->PinType = IndexPinType;
	}

	// Set up default values for index and option pins now that the information is available
	Schema->SetPinAutogeneratedDefaultValueBasedOnType(IndexPin);

	const bool bFillTypeFromReturn = ReturnPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard;
	TArray<UEdGraphPin*> OptionPins;
	GetOptionPins(OptionPins);
	for (UEdGraphPin* Pin : OptionPins)
	{
		const bool bTypeShouldBeFilled = Pin && (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard);
		if (bTypeShouldBeFilled && bFillTypeFromReturn)
		{
			Pin->PinType = ReturnPin->PinType;			
		}
		Schema->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
	}
}

void UK2Node_Select::PostReconstructNode()
{
	// After ReconstructNode we must be sure that no additional reconstruction is required
	bReconstructNode = false;
	bReconstructForPinTypeChange = false;

	UEdGraphPin* ReturnPin = GetReturnValuePin();
	const bool bFillTypeFromConnected = ReturnPin && (ReturnPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard);

	if (bFillTypeFromConnected)
	{
		FEdGraphPinType PinType = ReturnPin->PinType;

		if (ReturnPin->LinkedTo.Num() > 0)
		{
			PinType = ReturnPin->LinkedTo[0]->PinType;
		}
		else
		{
			TArray<UEdGraphPin*> OptionPins;
			GetOptionPins(OptionPins);
			for (UEdGraphPin* Pin : OptionPins)
			{
				if (Pin && Pin->LinkedTo.Num() > 0)
				{
					PinType = Pin->LinkedTo[0]->PinType;
					break;
				}
			}
		}

		ReturnPin->PinType = PinType;
		OnPinTypeChanged(ReturnPin);
	}

	Super::PostReconstructNode();
}

/** Determine if any pins are connected, if so make all the other pins the same type, if not, make sure pins are switched back to wildcards */
void UK2Node_Select::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	// If this is the Enum pin we need to set the enum and reconstruct the node
	if (Pin == GetIndexPin())
	{
		// If the index pin was just linked to another pin
		if (Pin->LinkedTo.Num() > 0 && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			UEdGraphPin* LinkPin = Pin->LinkedTo[0];

			if (Pin->PinType != LinkPin->PinType)
			{
				Pin->PinType = LinkPin->PinType;

				OnPinTypeChanged(Pin);
			}
		}
	}
	else
	{
		// Grab references to all option pins and the return pin
		TArray<UEdGraphPin*> OptionPins;
		GetOptionPins(OptionPins);
		UEdGraphPin* ReturnPin = FindPin(UEdGraphSchema_K2::PN_ReturnValue);

		// See if this pin is one of the wildcard pins
		const bool bIsWildcardPin = ((Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard) && ((Pin == ReturnPin) || (OptionPins.Find(Pin) != INDEX_NONE)));

		TFunction<bool(UEdGraphPin*)> PinInUse = [&PinInUse](UEdGraphPin* PinToConsider)
		{
			bool bPinInUse = ((PinToConsider->LinkedTo.Num() > 0) || (PinToConsider->ParentPin != nullptr) || !PinToConsider->DoesDefaultValueMatchAutogenerated());
			if (!bPinInUse)
			{
				for (UEdGraphPin* SubPin : PinToConsider->SubPins)
				{
					bPinInUse = PinInUse(SubPin);
					if (bPinInUse)
					{
						break;
					}
				}
			}
			return bPinInUse;
		};

		bool bPinsInUse = PinInUse(ReturnPin);
		
		if (!bPinsInUse)
		{
			for (UEdGraphPin* OptionPin : OptionPins)
			{
				bPinsInUse = PinInUse(OptionPin);
				if (bPinsInUse)
				{
					break;
				}
			}
		}

		bool bPinTypeChanged = false;

		if (bPinsInUse)
		{
			// If the pin was one of the wildcards we have to handle it specially
			if (bIsWildcardPin)
			{
				// If the pin is linked, make sure the other wildcard pins match
				if (Pin->LinkedTo.Num() > 0)
				{
					UEdGraphPin* LinkPin = Pin->LinkedTo[0];

					if (Pin->PinType != LinkPin->PinType)
					{
						Pin->PinType = LinkPin->PinType;
						bPinTypeChanged = true;
					}
				}
			}
		}
		else
		{
			bPinTypeChanged = true;
			Pin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			Pin->PinType.PinSubCategory = NAME_None;
			Pin->PinType.PinSubCategoryObject = nullptr;
		}

		if (bPinTypeChanged)
		{
			OnPinTypeChanged(Pin);
		}
	}
}

UEdGraphPin* UK2Node_Select::GetReturnValuePin() const
{
	UEdGraphPin* Pin = FindPin(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin);
	return Pin;
}

UEdGraphPin* UK2Node_Select::GetIndexPin() const
{
	UEdGraphPin* Pin = GetIndexPinUnchecked();
	check(Pin != NULL);
	return Pin;
}

UEdGraphPin* UK2Node_Select::GetIndexPinUnchecked() const
{
	return FindPin(TEXT("Index"));
}

void UK2Node_Select::GetOptionPins(TArray<UEdGraphPin*>& OptionPins) const
{
	OptionPins.Reset();

	// If the select node is currently dealing with an enum
	if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
		IndexPinType.PinSubCategory.IsNone() &&
		IndexPinType.PinSubCategoryObject != nullptr &&
		IndexPinType.PinSubCategoryObject->IsA(UEnum::StaticClass()))
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (EnumEntries.Contains(Pin->PinName))
			{
				OptionPins.Add(Pin);
			}
		}
	}
	else
	{
		const TCHAR* OptionStr = TEXT("Option");
		for (UEdGraphPin* Pin : Pins)
		{
			if (Pin->PinName.ToString().StartsWith(OptionStr))
			{
				OptionPins.Add(Pin);
			}
		}
	}
}

void UK2Node_Select::GetConditionalFunction(FName& FunctionName, UClass** FunctionClass)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, EqualEqual_BoolBool);
	}
	else if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Byte)
	{
		FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, EqualEqual_ByteByte);
	}
	else if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Int)
	{
		FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, EqualEqual_IntInt);
	}

	*FunctionClass = UKismetMathLibrary::StaticClass();
}

void UK2Node_Select::GetPrintStringFunction(FName& FunctionName, UClass** FunctionClass)
{
	FunctionName = GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, PrintWarning);
	*FunctionClass = UKismetSystemLibrary::StaticClass();
}

void UK2Node_Select::AddInputPin()
{
	Modify();

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Increment the pin count
	NumOptionPins++;
	// We guarantee at least 2 options by default and since we just increased the count
	// to more than 2, we need to make sure we're now dealing with an index for selection
	// instead of the default boolean check
	if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		IndexPinType.PinCategory = UEdGraphSchema_K2::PC_Int;
		GetIndexPin()->BreakAllPinLinks();
	}
	// We will let the AllocateDefaultPins call handle the actual addition via ReconstructNode
	ReconstructNode();
}

void UK2Node_Select::RemoveOptionPinToNode()
{
	// Increment the pin count
	NumOptionPins--;
	// We will let the AllocateDefaultPins call handle the actual subtraction via ReconstructNode
	ReconstructNode();
}

void UK2Node_Select::SetEnum(UEnum* InEnum, bool bForceRegenerate)
{
	UEnum* PrevEnum = Enum;
	Enum = InEnum;

	OrphanedPinSaveMode = (Enum ? ESaveOrphanPinMode::SaveAll : ESaveOrphanPinMode::SaveNone);

	if (bForceRegenerate || (PrevEnum != Enum))
	{
		// regenerate enum name list
		EnumEntries.Reset();
		EnumEntryFriendlyNames.Reset();

		if (Enum)
		{
			for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
			{
				bool const bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), EnumIndex ) || Enum->HasMetaData(TEXT("Spacer"), EnumIndex );
				if (!bShouldBeHidden)
				{
					FString EnumValueName = Enum->GetNameStringByIndex(EnumIndex);
					FText EnumFriendlyName = Enum->GetDisplayNameTextByIndex(EnumIndex);
					EnumEntries.Add(FName(*EnumValueName));
					EnumEntryFriendlyNames.Add(EnumFriendlyName);
				}
			}
		}

		bReconstructNode = true;
	}
}

void UK2Node_Select::NodeConnectionListChanged()
{
	Super::NodeConnectionListChanged();

	if (bReconstructNode)
	{
		ReconstructNode();

		UBlueprint* Blueprint = GetBlueprint();
		if(!Blueprint->bBeingCompiled)
		{
			FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
		}
	}
}

bool UK2Node_Select::CanAddPin() const
{
	if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
		IndexPinType.PinSubCategoryObject.IsValid() &&
		IndexPinType.PinSubCategoryObject.Get()->IsA(UEnum::StaticClass()))
	{
		return false;
	}
	else if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		return false;
	}

	return true;
}

bool UK2Node_Select::CanRemoveOptionPinToNode() const
{
	if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Byte &&
		(NULL != Cast<UEnum>(IndexPinType.PinSubCategoryObject.Get())))
	{
		return false;
	}
	else if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
	{
		return false;
	}

	return true;
}

void UK2Node_Select::ChangePinType(UEdGraphPin* Pin)
{
	OnPinTypeChanged(Pin);

	if (bReconstructNode)
	{
		bReconstructForPinTypeChange = true;
		ReconstructNode();
	}

	UBlueprint* Blueprint = GetBlueprint();
	if (!Blueprint->bBeingCompiled)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(Blueprint);
	}
}

bool UK2Node_Select::CanChangePinType(UEdGraphPin* Pin) const
{
	// If this is the index pin, only allow type switching if nothing is linked to the pin
	if (Pin == GetIndexPin())
	{
		if (Pin->LinkedTo.Num() > 0)
		{
			return false;
		}
	}
	// Else it's one of the wildcard pins that share their type, so make sure none of them have a link
	else
	{
		if (GetReturnValuePin()->LinkedTo.Num() > 0)
		{
			return false;
		}
		else
		{
			TArray<UEdGraphPin*> OptionPins;
			GetOptionPins(OptionPins);
			for (UEdGraphPin* OptionPin : OptionPins)
			{
				if (OptionPin && OptionPin->LinkedTo.Num() > 0)
				{
					return false;
				}
			}
		}
	}
	return true;
}

void UK2Node_Select::PinTypeChanged(UEdGraphPin* Pin)
{
	bReconstructForPinTypeChange = true;
	OnPinTypeChanged(Pin);
}

void UK2Node_Select::OnPinTypeChanged(UEdGraphPin* Pin)
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	if (Pin == GetIndexPin())
	{
		if (IndexPinType != Pin->PinType)
		{
			IndexPinType = Pin->PinType;

			// Since it is an interactive action we want the pins to go away regardless of the new type
			for (UEdGraphPin* PinToDiscard : Pins)
			{
				PinToDiscard->SetSavePinIfOrphaned(false);
			}

			if (IndexPinType.PinSubCategoryObject.IsValid())
			{
				SetEnum(Cast<UEnum>(IndexPinType.PinSubCategoryObject.Get()));
			}
			else if (Enum)
			{
				SetEnum(nullptr);
			}

			// Remove all but two options if we switched to a bool index
			if (IndexPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
			{
				NumOptionPins = 2;
			}

			if (!Schema->IsPinDefaultValid(Pin, Pin->DefaultValue, Pin->DefaultObject, Pin->DefaultTextValue).IsEmpty())
			{
				Schema->ResetPinToAutogeneratedDefaultValue(Pin);
			}

			bReconstructNode = true;
		}
	}
	else
	{
		// Set the return value
		UEdGraphPin* ReturnPin = GetReturnValuePin();
	
		// Recombine the sub pins back into the ReturnPin
		if (ReturnPin->SubPins.Num() > 0)
		{
			Schema->RecombinePin(ReturnPin->SubPins[0]);
		}
		ReturnPin->PinType = Pin->PinType;

		// Recombine all option pins back into their root
		TArray<UEdGraphPin*> OptionPins;
 		GetOptionPins(OptionPins);
		for (UEdGraphPin* OptionPin : OptionPins)
		{
			// Recombine the sub pins back into the OptionPin
			if (OptionPin->ParentPin == nullptr && OptionPin->SubPins.Num() > 0)
			{
				Schema->RecombinePin(OptionPin->SubPins[0]);
			}
		}

		// Get the options again and set them
		GetOptionPins(OptionPins);
		for (UEdGraphPin* OptionPin : OptionPins)
		{
			if (OptionPin->PinType != Pin->PinType ||
				OptionPin == Pin)
			{
				OptionPin->PinType = Pin->PinType;
			}

			if (!Schema->IsPinDefaultValid(OptionPin, OptionPin->DefaultValue, OptionPin->DefaultObject, OptionPin->DefaultTextValue).IsEmpty())
			{
				Schema->ResetPinToAutogeneratedDefaultValue(OptionPin);
			}
		}

		bReconstructNode = true;
	}
}

void UK2Node_Select::PostPasteNode()
{
	Super::PostPasteNode();

	if (UEdGraphPin* IndexPin = GetIndexPinUnchecked())
	{
		// This information will be cleared and we want to restore it
		FString OldDefaultValue = IndexPin->DefaultValue;

		// Corrects data in the index pin that is not valid after pasting
		OnPinTypeChanged(IndexPin);

		// Restore the default value of the index pin
		IndexPin->DefaultValue = MoveTemp(OldDefaultValue);
	}
}

FSlateIcon UK2Node_Select::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Select_16x");
	return Icon;
}

bool UK2Node_Select::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	if (OtherPin && (OtherPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec))
	{
		OutReason = LOCTEXT("ExecConnectionDisallowed", "Cannot connect with Exec pin.").ToString();
		return true;
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

FNodeHandlingFunctor* UK2Node_Select::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return static_cast<FNodeHandlingFunctor*>(new FKCHandler_Select(CompilerContext));
}

void UK2Node_Select::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_Select::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Utilities);
}

void UK2Node_Select::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();
	for (UEdGraphPin* Pin : Pins)
	{
		const bool bValidAutoRefPin = Pin && !Schema->IsMetaPin(*Pin) && (Pin->Direction == EGPD_Input) && (!Pin->LinkedTo.Num() || (GetIndexPin() == Pin));
		if (!bValidAutoRefPin)
		{
			continue;
		}

		// copy defaults as default values can be reset when the pin is connected
		const FString DefaultValue = Pin->DefaultValue;
		TObjectPtr<UObject> DefaultObject = Pin->DefaultObject;
		const FText DefaultTextValue = Pin->DefaultTextValue;
		bool bMatchesDefaults = Pin->DoesDefaultValueMatchAutogenerated();

		UEdGraphPin* ValuePin = UK2Node_CallFunction::InnerHandleAutoCreateRef(this, Pin, CompilerContext, SourceGraph, true);
		if (ValuePin)
		{
			if (bMatchesDefaults)
			{
				// Use the latest code to set default value
				Schema->SetPinAutogeneratedDefaultValueBasedOnType(ValuePin);
			}
			else
			{
				ValuePin->DefaultValue = DefaultValue;
				ValuePin->DefaultObject = DefaultObject;
				ValuePin->DefaultTextValue = DefaultTextValue;
			}
		}
	}
}

void UK2Node_Select::ReloadEnum(class UEnum* InEnum)
{
	SetEnum(InEnum, true);
}

#undef LOCTEXT_NAMESPACE
