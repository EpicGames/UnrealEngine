// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_Self.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Self)

class FKismetCompilerContext;

#define LOCTEXT_NAMESPACE "K2Node_Self"

class FKCHandler_Self : public FNodeHandlingFunctor
{
public:
	FKCHandler_Self(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Self* SelfNode = CastChecked<UK2Node_Self>(Node);

		UEdGraphPin* VarPin = SelfNode->FindPin(UEdGraphSchema_K2::PN_Self);
		check( VarPin );

		FBPTerminal* Term = new FBPTerminal();
		Context.Literals.Add(Term);
		Term->CopyFromPin(VarPin, VarPin->PinName);
		Term->bIsLiteral = true;
		Context.NetMap.Add(VarPin, Term);		
	}
};

UK2Node_Self::UK2Node_Self(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_Self::AllocateDefaultPins()
{
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, UEdGraphSchema_K2::PSC_Self, nullptr, UEdGraphSchema_K2::PN_Self);

	Super::AllocateDefaultPins();
}

FText UK2Node_Self::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "GetSelfReference", "Gets a reference to this instance of the blueprint");
}

FText UK2Node_Self::GetKeywords() const
{
	return LOCTEXT("SelfKeywords", "This");
}

FText UK2Node_Self::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle = NSLOCTEXT("K2Node", "SelfReferenceName", "Self-Reference");
	if (TitleType == ENodeTitleType::MenuTitle)
	{
		NodeTitle = LOCTEXT("ListTitle", "Get a reference to self");
	}
	return NodeTitle;	
}

FNodeHandlingFunctor* UK2Node_Self::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_Self(CompilerContext);
}

void UK2Node_Self::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	if(Schema->IsStaticFunctionGraph(GetGraph()))
	{
		MessageLog.Warning(*NSLOCTEXT("K2Node", "InvalidSelfNode", "Self node @@ cannot be used in a static function.").ToString(), this);
	}
}

void UK2Node_Self::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_Self::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Variables);
}

#undef LOCTEXT_NAMESPACE
