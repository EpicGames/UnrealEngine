// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SetFieldsInStruct.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintEditorSettings.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_Knot.h"
#include "K2Node_StructOperation.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "MakeStructHandler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SetFieldsInStruct)

class FBlueprintActionDatabaseRegistrar;
struct FLinearColor;

#define LOCTEXT_NAMESPACE "K2Node_MakeStruct"

struct SetFieldsInStructHelper
{
	static const TCHAR* StructRefPinName()
	{
		return TEXT("StructRef");
	}

	static const TCHAR* StructOutPinName()
	{
		return TEXT("StructOut");
	}
};

class FKCHandler_SetFieldsInStruct : public FKCHandler_MakeStruct
{
public:
	FKCHandler_SetFieldsInStruct(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_MakeStruct(InCompilerContext)
	{
		UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
		bAutoGenerateGotoForPure = false;
	}

	virtual UEdGraphPin* FindStructPinChecked(UEdGraphNode* InNode) const override
	{
		check(InNode != nullptr && InNode->IsA<UK2Node_SetFieldsInStruct>());
		UEdGraphPin* FoundPin = InNode->FindPinChecked(SetFieldsInStructHelper::StructRefPinName());
		check(EGPD_Input == FoundPin->Direction);
		return FoundPin;
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override
	{
		UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();

		if (Net->Direction == EGPD_Output)
		{
			if (Net->ReferencePassThroughConnection)
			{
				UEdGraphPin* InputPinNet = FEdGraphUtilities::GetNetFromPin(Net->ReferencePassThroughConnection);
				FBPTerminal** InputPinTerm = Context.NetMap.Find(InputPinNet);
				if (InputPinTerm && !(*InputPinTerm)->bPassedByReference)
				{
					// We need a net for the output pin which we have thus far prevented from being registered
					FKCHandler_MakeStruct::RegisterNet(Context, Net);
				}
			}
		}
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override
	{
		UK2Node_SetFieldsInStruct* Node = CastChecked<UK2Node_SetFieldsInStruct>(InNode);
		if (nullptr == Node->StructType)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_UnknownStructure_Error", "Unknown structure to break for @@").ToString(), Node);
			return;
		}
	
		if (!UEdGraphSchema_K2::IsAllowableBlueprintVariableType(Node->StructType, Node->IsIntermediateNode()))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_Error", "The structure @@ is not declared as BlueprintType - consider USTRUCT(BlueprintType) in C++ or resaving the asset").ToString(), Node);
			return;
		}

		FKCHandler_MakeStruct::RegisterNetsImpl(Context, Node);

		if (UEdGraphPin* ReturnPin = Node->FindPin(SetFieldsInStructHelper::StructOutPinName()))
		{
			UEdGraphPin* ReturnStructNet = FEdGraphUtilities::GetNetFromPin(ReturnPin);

			UEdGraphPin* InputPin = Node->FindPinChecked(SetFieldsInStructHelper::StructRefPinName());
			UEdGraphPin* InputPinNet = FEdGraphUtilities::GetNetFromPin(InputPin);
			FBPTerminal** InputTermRef = Context.NetMap.Find(InputPinNet);
		
			if (InputTermRef == nullptr)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_NoTerm_Error", "Failed to generate a term for the @@ pin; was it a struct reference that was left unset?").ToString(), InputPin);
			}
			else
			{
				FBPTerminal* InputTerm = *InputTermRef;
				if (InputTerm->bPassedByReference) //InputPinNet->PinType.bIsReference)
				{
					// Forward the net to the output pin because it's being passed by-ref and this pin is a by-ref pin
					Context.NetMap.Add(ReturnStructNet, InputTerm);
				}
			}
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("SetFieldsInStruct_NoReturnPin_Error", "Failed to find a return pin for node @@. This is likely due to an unresolved dependency.").ToString(), Node);
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		FKCHandler_MakeStruct::Compile(Context, Node);

		UBlueprintEditorSettings* Settings = GetMutableDefault<UBlueprintEditorSettings>();
		{
			UEdGraphPin* InputPin = Node->FindPinChecked(SetFieldsInStructHelper::StructRefPinName());
			UEdGraphPin* InputPinNet = FEdGraphUtilities::GetNetFromPin(InputPin);
			FBPTerminal** InputTerm = Context.NetMap.Find(InputPinNet);

			// If the InputTerm was not a by-ref, then we need to place the modified structure into the local output term with an AssignStatement
			if (InputTerm && !(*InputTerm)->bPassedByReference)
			{
				UEdGraphPin* ReturnPin = Node->FindPin(SetFieldsInStructHelper::StructOutPinName());
				UEdGraphPin* ReturnStructNet = FEdGraphUtilities::GetNetFromPin(ReturnPin);
				FBPTerminal** ReturnTerm = Context.NetMap.Find(ReturnStructNet);

				FBlueprintCompiledStatement& AssignStatement = Context.AppendStatementForNode(Node);
				AssignStatement.Type = KCST_Assignment;
				// The return term is a reference no matter the way we received it.
				(*ReturnTerm)->bPassedByReference = true;
				AssignStatement.LHS = *ReturnTerm;
				AssignStatement.RHS.Add(*InputTerm);
			}
		}

		GenerateSimpleThenGoto(Context, *Node);
	}
};

UK2Node_SetFieldsInStruct::UK2Node_SetFieldsInStruct(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRecursionGuard(false)
{
}

void UK2Node_SetFieldsInStruct::AllocateDefaultPins()
{
	if (StructType)
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

		UEdGraphNode::FCreatePinParams PinParams;
		PinParams.bIsReference = true;

		UEdGraphPin* InPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, StructType, SetFieldsInStructHelper::StructRefPinName(), PinParams);
		UEdGraphPin* OutPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, StructType, SetFieldsInStructHelper::StructOutPinName(), PinParams);

		// Input pin will forward the ref to the output, if the input value is not a reference connection, a copy is made and modified instead and provided as a reference until the function is called again.
		InPin->AssignByRefPassThroughConnection(OutPin);

		OutPin->PinToolTip = LOCTEXT("SetFieldsInStruct_OutPinTooltip", "Reference to the input struct").ToString();
		{
			FStructOnScope StructOnScope(StructType);
			FSetFieldsInStructPinManager OptionalPinManager(StructOnScope.GetStructMemory(), GetBlueprint());
			OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructType);
			OptionalPinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Input, this);
		}
	}
}

FText UK2Node_SetFieldsInStruct::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (StructType == nullptr)
	{
		return LOCTEXT("SetFieldsInNullStructNodeTitle", "Set members in <unknown struct>");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("StructName"), StructType->GetDisplayNameText());
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("SetFieldsInStructNodeTitle", "Set members in {StructName}"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_SetFieldsInStruct::GetTooltipText() const
{
	if (StructType == nullptr)
	{
		return LOCTEXT("SetFieldsInStruct_NullTooltip", "Adds a node that modifies an '<unknown struct>'");
	}
	else if (CachedTooltip.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(
			LOCTEXT("SetFieldsInStruct_Tooltip", "Adds a node that modifies a '{0}'"),
			StructType->GetDisplayNameText()
		), this);
	}
	return CachedTooltip;
}

FSlateIcon UK2Node_SetFieldsInStruct::GetIconAndTint(FLinearColor& OutColor) const
{
	return UK2Node_Variable::GetIconAndTint(OutColor);
}

void UK2Node_SetFieldsInStruct::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	UEdGraphPin* FoundPin = FindPin(SetFieldsInStructHelper::StructRefPinName());
	if (!FoundPin || (FoundPin->LinkedTo.Num() <= 0))
	{
		FText ErrorMessage = LOCTEXT("SetStructFields_NoStructRefError", "The @@ pin must be connected to the struct that you wish to set.");
		MessageLog.Error(*ErrorMessage.ToString(), FoundPin);
		return;
	}

	// Attempt to determine if we're linked to a getter node for a BlueprintReadOnly property.

	for (UEdGraphPin* SourceStructOutputPin : FoundPin->LinkedTo)
	{
		BackTracePinPath(SourceStructOutputPin, [&MessageLog](UEdGraphPin* LinkedStructSourcePin) {
			if (UK2Node_VariableGet* GetterNode = Cast<UK2Node_VariableGet>(LinkedStructSourcePin->GetOwningNode()))
			{
				if (FProperty* BoundProperty = GetterNode->GetPropertyForVariable())
				{
					const bool bIsFunctionInput =
						!BoundProperty->HasAnyPropertyFlags(CPF_ReturnParm) &&
						(!BoundProperty->HasAnyPropertyFlags(CPF_OutParm) || BoundProperty->HasAnyPropertyFlags(CPF_ReferenceParm))
					;

					const bool bIsReadOnly =
						BoundProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly)
					;

					// Function input parameters generally get tagged as readonly by the compiler.
					// If they're actually input/ouput reference parameters, then we can make an exception here.
					if (bIsReadOnly && !bIsFunctionInput)
					{
						// TODO, This should REALLY be an error, but too much code may have been written not following this standard.
						FText ErrorMessage = LOCTEXT("SetStructFields_StructIsConst", "The @@ is a Read Only property and can not be modified directly.");
						MessageLog.Warning(*ErrorMessage.ToString(), LinkedStructSourcePin);
					}
				}
			}
		});
	}
}

void UK2Node_SetFieldsInStruct::BackTracePinPath(UEdGraphPin* OutputPin, TFunctionRef<void(UEdGraphPin*)> Predicate) const
{
	if (bRecursionGuard)
	{
		return;
	}
	
	TGuardValue<bool> RecursionGuard(bRecursionGuard, true);

	UEdGraphNode* OwningNode = OutputPin->GetOwningNode();
	if (UK2Node_Knot* KnotNode = Cast<UK2Node_Knot>(OwningNode))
	{
		UEdGraphPin* KnotInputPin = KnotNode->GetInputPin();
		for (UEdGraphPin* KnotInput : KnotInputPin->LinkedTo)
		{
			BackTracePinPath(KnotInput, Predicate);
		}
	}
	else if(OwningNode)
	{
		Predicate(OutputPin);
	}
}

FNodeHandlingFunctor* UK2Node_SetFieldsInStruct::CreateNodeHandler(class FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_SetFieldsInStruct(CompilerContext);
}

bool UK2Node_SetFieldsInStruct::ShowCustomPinActions(const UEdGraphPin* Pin, bool bIgnorePinsNum)
{
	const int32 MinimalPinsNum = 5;
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	const auto Node = Pin ? Cast<const UK2Node_SetFieldsInStruct>(Pin->GetOwningNodeUnchecked()) : NULL;
	return Node
		&& ((Node->Pins.Num() > MinimalPinsNum) || bIgnorePinsNum)
		&& (EGPD_Input == Pin->Direction)
		&& (Pin->PinName != SetFieldsInStructHelper::StructRefPinName())
		&& !Schema->IsMetaPin(*Pin);
}

void UK2Node_SetFieldsInStruct::RemoveFieldPins(UEdGraphPin* Pin, EPinsToRemove Selection)
{
	if (ShowCustomPinActions(Pin, false) && (Pin->GetOwningNodeUnchecked() == this))
	{
		// Pretend that the action was done on the hidden parent pin if the pin is split
		while (Pin->ParentPin != nullptr)
		{
			Pin = Pin->ParentPin;
		}

		const bool bHideSelected = (Selection == EPinsToRemove::GivenPin);
		const bool bHideNotSelected = (Selection == EPinsToRemove::AllOtherPins);
		bool bWasChanged = false;
		for (FOptionalPinFromProperty& OptionalProperty : ShowPinForProperties)
		{
			const bool bSelected = (Pin->PinName == OptionalProperty.PropertyName);
			const bool bHide = (bSelected && bHideSelected) || (!bSelected && bHideNotSelected);
			if (OptionalProperty.bShowPin && bHide)
			{
				bWasChanged = true;
				OptionalProperty.bShowPin = false;
				Pin->SetSavePinIfOrphaned(false);
			}
		}

		if (bWasChanged)
		{
			ReconstructNode();
		}
	}
}

bool UK2Node_SetFieldsInStruct::AllPinsAreShown() const
{
	UEdGraphPin* InputPin = FindPinChecked(SetFieldsInStructHelper::StructRefPinName(), EGPD_Input);

	// If the input struct pin is currently split, don't allow option to restore members
	if (InputPin != nullptr && InputPin->SubPins.Num() > 0)
	{
		return true;
	}

	for (const FOptionalPinFromProperty& OptionalProperty : ShowPinForProperties)
	{
		if (!OptionalProperty.bShowPin)
		{
			return false;
		}
	}

	return true;
}

void UK2Node_SetFieldsInStruct::RestoreAllPins()
{
	bool bWasChanged = false;
	for (FOptionalPinFromProperty& OptionalProperty : ShowPinForProperties)
	{
		if (!OptionalProperty.bShowPin)
		{
			bWasChanged = true;
			OptionalProperty.bShowPin = true;
		}
	}

	if (bWasChanged)
	{
		ReconstructNode();
	}
}

void UK2Node_SetFieldsInStruct::FSetFieldsInStructPinManager::GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const
{
	FMakeStructPinManager::GetRecordDefaults(TestProperty, Record);

	Record.bShowPin = false;
}

bool UK2Node_SetFieldsInStruct::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if (MyPin->bNotConnectable)
	{
		OutReason = LOCTEXT("SetFieldsInStructConnectionDisallowed", "This pin must enable the override to set a value!").ToString();
		return true;
	}

	return Super::IsConnectionDisallowed(MyPin, OtherPin, OutReason);
}

bool UK2Node_SetFieldsInStruct::CanSplitPin(const UEdGraphPin* Pin) const
{
	if (Super::CanSplitPin(Pin))
	{
		UEdGraphPin* InputPin = FindPinChecked(SetFieldsInStructHelper::StructRefPinName(), EGPD_Input);
		if (Pin == InputPin)
		{
			return false;
		}

		return true;
	}
	else
	{
		return false;
	}
}


void UK2Node_SetFieldsInStruct::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Use Make's filter function but auto promote for output pins
	Super::SetupMenuActions(ActionRegistrar, FMakeStructSpawnerAllowedDelegate::CreateStatic(&UK2Node_MakeStruct::CanBeMade), EGPD_Output);
}

#undef LOCTEXT_NAMESPACE
