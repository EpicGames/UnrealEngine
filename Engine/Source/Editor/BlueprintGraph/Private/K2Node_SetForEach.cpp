// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SetForEach.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_AssignmentStatement.h"
#include "K2Node_CallFunction.h"
#include "K2Node_ExecutionSequence.h"
#include "K2Node_IfThenElse.h"
#include "K2Node_TemporaryVariable.h"
#include "KismetCompiler.h"
#include "Kismet/BlueprintSetLibrary.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/WildcardNodeUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SetForEach)

#define LOCTEXT_NAMESPACE "K2Node_SetForEach"

const FName UK2Node_SetForEach::SetPinName(TEXT("SetPin"));
const FName UK2Node_SetForEach::BreakPinName(TEXT("BreakPin"));
const FName UK2Node_SetForEach::ValuePinName(TEXT("ValuePin"));
const FName UK2Node_SetForEach::CompletedPinName(TEXT("CompletedPin"));

UK2Node_SetForEach::UK2Node_SetForEach()
{
	ValueName = LOCTEXT("ValuePin_FriendlyName", "Set Value").ToString();
}

UEdGraphPin* UK2Node_SetForEach::GetSetPin() const
{
	return FindPinChecked(SetPinName);
}

UEdGraphPin* UK2Node_SetForEach::GetBreakPin() const
{
	return FindPinChecked(BreakPinName);
}

UEdGraphPin* UK2Node_SetForEach::GetForEachPin() const
{
	return FindPinChecked(UEdGraphSchema_K2::PN_Then);
}

UEdGraphPin* UK2Node_SetForEach::GetValuePin() const
{
	return FindPinChecked(ValuePinName);
}

UEdGraphPin* UK2Node_SetForEach::GetCompletedPin() const
{
	return FindPinChecked(CompletedPinName);
}

void UK2Node_SetForEach::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// Actions are registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed).
	// Here, we use the node's class to keep from needlessly instantiating a UBlueprintNodeSpawner.
	// Additionally, if the node type disappears, then the action should go with it.

	UClass* ActionKey = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(ActionKey);
		check(NodeSpawner != nullptr);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_SetForEach::GetMenuCategory() const
{
	return LOCTEXT("NodeMenu", "Utilities|Set");
}

void UK2Node_SetForEach::PostReconstructNode()
{
	Super::PostReconstructNode();

	RefreshWildcardPins();
}

void UK2Node_SetForEach::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.ContainerType = EPinContainerType::Set;
	PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_Wildcard;

	UEdGraphPin* SetPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, SetPinName, PinParams);
	SetPin->PinType.bIsConst = true;
	SetPin->PinType.bIsReference = true;
	SetPin->PinFriendlyName = LOCTEXT("SetPin_FriendlyName", "Set");

	UEdGraphPin* BreakPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, BreakPinName);
	BreakPin->PinFriendlyName = LOCTEXT("BreakPin_FriendlyName", "Break");
	BreakPin->bAdvancedView = true;

	UEdGraphPin* ForEachPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
	ForEachPin->PinFriendlyName = LOCTEXT("ForEachPin_FriendlyName", "Loop Body");

	UEdGraphPin* ValuePin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, ValuePinName);
	ValuePin->PinFriendlyName = FText::FromString(ValueName);

	UEdGraphPin* CompletedPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, CompletedPinName);
	CompletedPin->PinFriendlyName = LOCTEXT("CompletedPin_FriendlyName", "Completed");
	CompletedPin->PinToolTip = LOCTEXT("CompletedPin_Tooltip", "Execution once all set elements have been visited").ToString();

	if (AdvancedPinDisplay == ENodeAdvancedPins::NoPins)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
}

void UK2Node_SetForEach::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (CheckForErrors(CompilerContext))
	{
		// Remove all the links to this node as they are no longer needed
		BreakAllNodeLinks();
		return;
	}

	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	///////////////////////////////////////////////////////////////////////////////////
	// Cache off versions of all our important pins

	UEdGraphPin* ForEach_Exec = GetExecPin();
	UEdGraphPin* ForEach_Set = GetSetPin();
	UEdGraphPin* ForEach_Break = GetBreakPin();
	UEdGraphPin* ForEach_ForEach = GetForEachPin();
	UEdGraphPin* ForEach_Value = GetValuePin();
	UEdGraphPin* ForEach_Completed = GetCompletedPin();

	///////////////////////////////////////////////////////////////////////////////////
	// Create a loop counter variable

	UK2Node_TemporaryVariable* CreateTemporaryVariable = CompilerContext.SpawnIntermediateNode<UK2Node_TemporaryVariable>(this, SourceGraph);
	CreateTemporaryVariable->VariableType.PinCategory = UEdGraphSchema_K2::PC_Int;
	CreateTemporaryVariable->AllocateDefaultPins();

	UEdGraphPin* Temp_Variable = CreateTemporaryVariable->GetVariablePin();

	///////////////////////////////////////////////////////////////////////////////////
	// Initialize the temporary to 0

	UK2Node_AssignmentStatement* InitTemporaryVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	InitTemporaryVariable->AllocateDefaultPins();

	UEdGraphPin* Init_Exec = InitTemporaryVariable->GetExecPin();
	UEdGraphPin* Init_Variable = InitTemporaryVariable->GetVariablePin();
	UEdGraphPin* Init_Value = InitTemporaryVariable->GetValuePin();
	UEdGraphPin* Init_Then = InitTemporaryVariable->GetThenPin();

	CompilerContext.MovePinLinksToIntermediate(*ForEach_Exec, *Init_Exec);
	K2Schema->TryCreateConnection(Init_Variable, Temp_Variable);
	Init_Value->DefaultValue = TEXT("0");

	///////////////////////////////////////////////////////////////////////////////////
	// Branch on comparing the loop index with the size of the set

	UK2Node_IfThenElse* BranchOnIndex = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
	BranchOnIndex->AllocateDefaultPins();

	UEdGraphPin* Branch_Exec = BranchOnIndex->GetExecPin();
	UEdGraphPin* Branch_Input = BranchOnIndex->GetConditionPin();
	UEdGraphPin* Branch_Then = BranchOnIndex->GetThenPin();
	UEdGraphPin* Branch_Else = BranchOnIndex->GetElsePin();

	Init_Then->MakeLinkTo(Branch_Exec);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_Completed, *Branch_Else);

	UK2Node_CallFunction* CompareLessThan = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CompareLessThan->FunctionReference.SetExternalMember( GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Less_IntInt), UKismetMathLibrary::StaticClass());
	CompareLessThan->AllocateDefaultPins();

	UEdGraphPin* Compare_A = CompareLessThan->FindPinChecked(TEXT("A"));
	UEdGraphPin* Compare_B = CompareLessThan->FindPinChecked(TEXT("B"));
	UEdGraphPin* Compare_Return = CompareLessThan->GetReturnValuePin();

	Branch_Input->MakeLinkTo(Compare_Return);
	Temp_Variable->MakeLinkTo(Compare_A);

	UK2Node_CallFunction* GetSetLength = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetSetLength->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintSetLibrary, Set_Length), UBlueprintSetLibrary::StaticClass());
	GetSetLength->AllocateDefaultPins();

	UEdGraphPin* SetLength_Set = GetSetLength->FindPinChecked(TEXT("TargetSet"));
	UEdGraphPin* SetLength_Return = GetSetLength->GetReturnValuePin();

	// Coerce the wildcard pin types
	SetLength_Set->PinType = ForEach_Set->PinType;

	Compare_B->MakeLinkTo(SetLength_Return);
	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Set, *SetLength_Set);

	///////////////////////////////////////////////////////////////////////////////////
	// Sequence the loop body and incrementing the loop counter

	UK2Node_ExecutionSequence* LoopSequence = CompilerContext.SpawnIntermediateNode<UK2Node_ExecutionSequence>(this, SourceGraph);
	LoopSequence->AllocateDefaultPins();

	UEdGraphPin* Sequence_Exec = LoopSequence->GetExecPin();
	UEdGraphPin* Sequence_One = LoopSequence->GetThenPinGivenIndex(0);
	UEdGraphPin* Sequence_Two = LoopSequence->GetThenPinGivenIndex(1);

	Branch_Then->MakeLinkTo(Sequence_Exec);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_ForEach, *Sequence_One);

	UK2Node_CallFunction* GetSetElement = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetSetElement->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintSetLibrary, Set_GetItemByIndex), UBlueprintSetLibrary::StaticClass());
	GetSetElement->AllocateDefaultPins();

	UEdGraphPin* GetElement_Set = GetSetElement->FindPinChecked(TEXT("TargetSet"));
	UEdGraphPin* GetElement_Index = GetSetElement->FindPinChecked(TEXT("Index"));
	UEdGraphPin* GetElement_Value = GetSetElement->FindPinChecked(TEXT("Item"));

	// Coerce the wildcard pin types
	GetElement_Set->PinType = ForEach_Set->PinType;
	GetElement_Value->PinType = ForEach_Value->PinType;

	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Set, *GetElement_Set);
	GetElement_Index->MakeLinkTo(Temp_Variable);
	CompilerContext.MovePinLinksToIntermediate(*ForEach_Value, *GetElement_Value);

	///////////////////////////////////////////////////////////////////////////////////
	// Increment the loop counter by one

	const auto IncrementVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	IncrementVariable->AllocateDefaultPins();

	UEdGraphPin* Inc_Exec = IncrementVariable->GetExecPin();
	UEdGraphPin* Inc_Variable = IncrementVariable->GetVariablePin();
	UEdGraphPin* Inc_Value = IncrementVariable->GetValuePin();
	UEdGraphPin* Inc_Then = IncrementVariable->GetThenPin();

	Sequence_Two->MakeLinkTo(Inc_Exec);
	Branch_Exec->MakeLinkTo(Inc_Then);
	K2Schema->TryCreateConnection(Temp_Variable, Inc_Variable);

	UK2Node_CallFunction* AddOne = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	AddOne->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Add_IntInt), UKismetMathLibrary::StaticClass());
	AddOne->AllocateDefaultPins();

	UEdGraphPin* Add_A = AddOne->FindPinChecked(TEXT("A"));
	UEdGraphPin* Add_B = AddOne->FindPinChecked(TEXT("B"));
	UEdGraphPin* Add_Return = AddOne->GetReturnValuePin();

	Temp_Variable->MakeLinkTo(Add_A);
	Add_B->DefaultValue = TEXT("1");
	Add_Return->MakeLinkTo(Inc_Value);

	///////////////////////////////////////////////////////////////////////////////////
	// Create a sequence from the break exec that will set the loop counter to the last array index.
	// The loop will then increment the counter and terminate on the next run of SequenceTwo.

	UK2Node_AssignmentStatement* SetVariable = CompilerContext.SpawnIntermediateNode<UK2Node_AssignmentStatement>(this, SourceGraph);
	SetVariable->AllocateDefaultPins();

	UEdGraphPin* Set_Exec = SetVariable->GetExecPin();
	UEdGraphPin* Set_Variable = SetVariable->GetVariablePin();
	UEdGraphPin* Set_Value = SetVariable->GetValuePin();

	CompilerContext.MovePinLinksToIntermediate(*ForEach_Break, *Set_Exec);
	K2Schema->TryCreateConnection(Temp_Variable, Set_Variable);

	UK2Node_CallFunction* GetSetLastIndex = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetSetLastIndex->FunctionReference.SetExternalMember(GET_FUNCTION_NAME_CHECKED(UBlueprintSetLibrary, Set_GetLastIndex), UBlueprintSetLibrary::StaticClass());
	GetSetLastIndex->AllocateDefaultPins();

	UEdGraphPin* GetIndex_Set = GetSetLastIndex->FindPinChecked(TEXT("TargetSet"));
	UEdGraphPin* GetIndex_Return = GetSetLastIndex->GetReturnValuePin();

	// Coerce the wildcard pin types
	GetIndex_Set->PinType = ForEach_Set->PinType;
	CompilerContext.CopyPinLinksToIntermediate(*ForEach_Set, *GetIndex_Set);

	GetIndex_Return->MakeLinkTo(Set_Value);

	BreakAllNodeLinks();
}

FText UK2Node_SetForEach::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "For Each Loop (Set)");
}

FText UK2Node_SetForEach::GetTooltipText() const
{
	return LOCTEXT("NodeToolTip", "Loop over each element of a Set");
}

FSlateIcon UK2Node_SetForEach::GetIconAndTint(FLinearColor& OutColor) const
{
	return FSlateIcon("EditorStyle", "GraphEditor.Macro.ForEach_16x");
}

void UK2Node_SetForEach::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	check(Pin);

	if (Pin->PinName == SetPinName)
	{
		RefreshWildcardPins();
	}
}

void UK2Node_SetForEach::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bool bRefresh = false;

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UK2Node_SetForEach, ValueName))
	{
		GetValuePin()->PinFriendlyName = FText::FromString(ValueName);
		bRefresh = true;
	}

	if (bRefresh)
	{
		// Poke the graph to update the visuals based on the above changes
		GetGraph()->NotifyGraphChanged();
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	}
}

bool UK2Node_SetForEach::CheckForErrors(const FKismetCompilerContext& CompilerContext)
{
	bool bError = false;

	if (GetSetPin()->LinkedTo.Num() == 0)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MissingSet_Error", "For Each (Set) node @@ must have a Set to iterate." ).ToString(), this);
		bError = true;
	}

	return bError;
}

void UK2Node_SetForEach::RefreshWildcardPins()
{
	UEdGraphPin* SetPin = GetSetPin();
	check(SetPin);

	UEdGraphPin* ValuePin = GetValuePin();
	check(ValuePin);

	const bool bIsWildcardPin = FWildcardNodeUtils::IsWildcardPin(SetPin);
	if (bIsWildcardPin && SetPin->LinkedTo.Num() > 0)
	{
		if (const UEdGraphPin* InferrablePin = FWildcardNodeUtils::FindInferrableLinkedPin(SetPin))
		{
			FWildcardNodeUtils::InferType(SetPin, InferrablePin->PinType);

			// In some contexts, the value pin may not be in a wildcard state (eg: paste operation).
			// We'll just leave the pin with its current type and let the compiler catch any issues (if any).
			// This also helps ensure that we don't jostle any pins that are in a split state.
			if (FWildcardNodeUtils::IsWildcardPin(ValuePin))
			{
				FWildcardNodeUtils::InferType(ValuePin, InferrablePin->PinType);
			}
		}
	}

	// If no pins are connected, then we need to reset the dependent pins back to the original wildcard state.
	if (SetPin->LinkedTo.Num() == 0)
	{
		FWildcardNodeUtils::ResetToWildcard(SetPin);
		FWildcardNodeUtils::ResetToWildcard(ValuePin);
	}
}

#undef LOCTEXT_NAMESPACE
