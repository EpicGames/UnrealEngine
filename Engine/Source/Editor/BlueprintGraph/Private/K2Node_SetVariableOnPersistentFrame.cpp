// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_SetVariableOnPersistentFrame.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SetVariableOnPersistentFrame)

#define LOCTEXT_NAMESPACE "K2Node_SetVariableOnPersistentFrame"

class FKCHandler_SetVariableOnPersistentFrame : public FNodeHandlingFunctor
{
public:
	FKCHandler_SetVariableOnPersistentFrame(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		check(Node && CompilerContext.GetSchema());
		for (auto Pin : Node->Pins)
		{
			if (!Pin || CompilerContext.GetSchema()->IsMetaPin(*Pin))
			{
				continue;
			}

			if ((Pin->Direction != EGPD_Input) || (1 != Pin->LinkedTo.Num()) || Context.bIsUbergraph || !Context.NewClass || !Context.NewClass->UberGraphFunction)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SetVariableOnPersistentFrame_IceError", "ICE SetVariableOnPersistentFrame @@").ToString(), Pin);
				return;
			}

			bool bIsSparseProperty;
			FProperty* BoundProperty = FKismetCompilerUtilities::FindPropertyInScope(Context.NewClass->UberGraphFunction, Pin, CompilerContext.MessageLog, CompilerContext.GetSchema(), Context.NewClass, bIsSparseProperty);
			// no setters on sparse properties for now
			check(!bIsSparseProperty);
			if (!BoundProperty || (BoundProperty->GetOwner<UObject>() != Context.NewClass->UberGraphFunction))
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SetVariableOnPersistentFrame_IceErrorNoProperty", "ICE SetVariableOnPersistentFrame - No property found. @@").ToString(), Pin);
				return;
			}

			// Create the term in the list
			FBPTerminal* Term = new FBPTerminal();
			Context.PersistentFrameVariableReferences.Add(Term);
			Term->CopyFromPin(Pin, Pin->PinName);
			Term->AssociatedVarProperty = BoundProperty;
			Context.NetMap.Add(Pin, Term);
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		check(Node && CompilerContext.GetSchema());
		for (auto Pin : Node->Pins)
		{
			if (!Pin || CompilerContext.GetSchema()->IsMetaPin(*Pin))
			{
				continue;
			}

			FBPTerminal** DestTerm = Context.NetMap.Find(Pin);
			FBPTerminal** SourceTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(Pin));
			if (DestTerm && *DestTerm && SourceTerm && *SourceTerm)
			{
				FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
				Statement.Type = KCST_AssignmentOnPersistentFrame;
				Statement.LHS = *DestTerm;
				Statement.RHS.Add(*SourceTerm);
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("SetVariableOnPersistentFrame_NoTerm", "ICE SetVariableOnPersistentFrame - No terminal found. @@").ToString(), Pin);
				return;
			}
		}

		// Generate the output impulse from this node
		GenerateSimpleThenGoto(Context, *Node);
	}
};

void UK2Node_SetVariableOnPersistentFrame::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	Super::AllocateDefaultPins();
}

FNodeHandlingFunctor* UK2Node_SetVariableOnPersistentFrame::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_SetVariableOnPersistentFrame(CompilerContext);
}

#undef LOCTEXT_NAMESPACE
