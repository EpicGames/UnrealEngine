// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SpawnActor.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallArrayFunction.h"
#include "K2Node_CallFunction.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SpawnActor)

struct FK2Node_SpawnActorHelper
{
	static const FName WorldContextPinName;
	static const FName BlueprintPinName;
	static const FName SpawnTransformPinName;
	static const FName NoCollisionFailPinName;
};

const FName FK2Node_SpawnActorHelper::WorldContextPinName(TEXT("WorldContextObject"));
const FName FK2Node_SpawnActorHelper::BlueprintPinName(TEXT("Blueprint"));
const FName FK2Node_SpawnActorHelper::SpawnTransformPinName(TEXT("SpawnTransform"));
const FName FK2Node_SpawnActorHelper::NoCollisionFailPinName(TEXT("SpawnEvenIfColliding"));

#define LOCTEXT_NAMESPACE "K2Node_SpawnActor"

UK2Node_SpawnActor::UK2Node_SpawnActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Attempts to spawn a new Actor with the specified transform");
}

void UK2Node_SpawnActor::AllocateDefaultPins()
{
	UEdGraphSchema_K2 const* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Add execution pins
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	// If required add the world context pin
	if (FBlueprintEditorUtils::ShouldShowWorldContextPin(GetBlueprint()))
	{
		CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UObject::StaticClass(), FK2Node_SpawnActorHelper::WorldContextPinName);
	}

	// Add blueprint pin
	UEdGraphPin* BlueprintPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UBlueprint::StaticClass(), FK2Node_SpawnActorHelper::BlueprintPinName);
	K2Schema->ConstructBasicPinTooltip(*BlueprintPin, LOCTEXT("BlueprintPinDescription", "The blueprint Actor you want to spawn"), BlueprintPin->PinToolTip);

	// Transform pin
	UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
	UEdGraphPin* TransformPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, TransformStruct, FK2Node_SpawnActorHelper::SpawnTransformPinName);
	K2Schema->ConstructBasicPinTooltip(*TransformPin, LOCTEXT("TransformPinDescription", "The transform to spawn the Actor with"), TransformPin->PinToolTip);

	// bNoCollisionFail pin
	UEdGraphPin* NoCollisionFailPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Boolean, FK2Node_SpawnActorHelper::NoCollisionFailPinName);
	K2Schema->ConstructBasicPinTooltip(*NoCollisionFailPin, LOCTEXT("NoCollisionFailPinDescription", "Determines if the Actor should be spawned when the location is blocked by a collision"), NoCollisionFailPin->PinToolTip);

	// Result pin
	UEdGraphPin* ResultPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, AActor::StaticClass(), UEdGraphSchema_K2::PN_ReturnValue);
	K2Schema->ConstructBasicPinTooltip(*ResultPin, LOCTEXT("ResultPinDescription", "The spawned Actor"), ResultPin->PinToolTip);

	Super::AllocateDefaultPins();
}

void UK2Node_SpawnActor::CreatePinsForClass(UClass* InClass)
{
	check(InClass != NULL);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	for (TFieldIterator<FProperty> PropertyIt(InClass, EFieldIteratorFlags::IncludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		UClass* PropertyClass = CastChecked<UClass>(Property->GetOwner<UObject>());
		const bool bIsDelegate = Property->IsA(FMulticastDelegateProperty::StaticClass());
		const bool bIsExposedToSpawn = UEdGraphSchema_K2::IsPropertyExposedOnSpawn(Property);
		const bool bIsSettableExternally = !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance);

		if(	bIsExposedToSpawn &&
			!Property->HasAnyPropertyFlags(CPF_Parm) && 
			bIsSettableExternally &&
			Property->HasAllPropertyFlags(CPF_BlueprintVisible) &&
			!bIsDelegate )
		{
			UEdGraphPin* Pin = CreatePin(EGPD_Input, NAME_None, Property->GetFName());
			const bool bPinGood = (Pin != nullptr) && K2Schema->ConvertPropertyToPinType(Property, /*out*/ Pin->PinType);	

			// Copy tooltip from the property.
			if (Pin != nullptr)
			{
				K2Schema->ConstructBasicPinTooltip(*Pin, Property->GetToolTipText(), Pin->PinToolTip);
			}
		}
	}

	// Change class of output pin
	UEdGraphPin* ResultPin = GetResultPin();
	ResultPin->PinType.PinSubCategoryObject = InClass;
}

UClass* UK2Node_SpawnActor::GetClassToSpawn(const TArray<UEdGraphPin*>* InPinsToSearch /*=NULL*/) const
{
	UClass* UseSpawnClass = NULL;
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* BlueprintPin = GetBlueprintPin(PinsToSearch);
	if(BlueprintPin && BlueprintPin->DefaultObject != NULL && BlueprintPin->LinkedTo.Num() == 0)
	{
		UBlueprint* BP = CastChecked<UBlueprint>(BlueprintPin->DefaultObject);
		UseSpawnClass = BP->GeneratedClass;
	}

	return UseSpawnClass;
}

void UK2Node_SpawnActor::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	AllocateDefaultPins();
	UClass* UseSpawnClass = GetClassToSpawn(&OldPins);
	if( UseSpawnClass != NULL )
	{
		CreatePinsForClass(UseSpawnClass);
	}

	RestoreSplitPins(OldPins);
}

bool UK2Node_SpawnActor::IsSpawnVarPin(UEdGraphPin* Pin) const
{
	if (Pin->ParentPin)
	{
		return IsSpawnVarPin(Pin->ParentPin);
	}

	return(	Pin->PinName != UEdGraphSchema_K2::PN_Execute &&
			Pin->PinName != UEdGraphSchema_K2::PN_Then &&
			Pin->PinName != UEdGraphSchema_K2::PN_ReturnValue &&
			Pin->PinName != FK2Node_SpawnActorHelper::BlueprintPinName &&
			Pin->PinName != FK2Node_SpawnActorHelper::WorldContextPinName &&
			Pin->PinName != FK2Node_SpawnActorHelper::NoCollisionFailPinName &&
			Pin->PinName != FK2Node_SpawnActorHelper::SpawnTransformPinName );
}


void UK2Node_SpawnActor::PinDefaultValueChanged(UEdGraphPin* ChangedPin) 
{
	if (ChangedPin->PinName == FK2Node_SpawnActorHelper::BlueprintPinName)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		// Because the archetype has changed, we break the output link as the output pin type will change
		UEdGraphPin* ResultPin = GetResultPin();
		ResultPin->BreakAllPinLinks();

		// Remove all pins related to archetype variables
		TArray<UEdGraphPin*> OldPins = Pins;
		for (int32 i = 0; i < OldPins.Num(); i++)
		{
			UEdGraphPin* OldPin = OldPins[i];
			if (IsSpawnVarPin(OldPin))
			{
				OldPin->MarkAsGarbage();
				Pins.Remove(OldPin);
			}
		}

		CachedNodeTitle.MarkDirty();

		UClass* UseSpawnClass = GetClassToSpawn();
		if(UseSpawnClass != NULL)
		{
			CreatePinsForClass(UseSpawnClass);
		}

		// Refresh the UI for the graph so the pin changes show up
		UEdGraph* Graph = GetGraph();
		Graph->NotifyNodeChanged(this);

		// Mark dirty
		FBlueprintEditorUtils::MarkBlueprintAsModified(GetBlueprint());
	}
}

FText UK2Node_SpawnActor::GetTooltipText() const
{
	return NodeTooltip;
}

UEdGraphPin* UK2Node_SpawnActor::GetBlueprintPin(const TArray<UEdGraphPin*>* InPinsToSearch /*= NULL*/) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* TestPin : *PinsToSearch)
	{
		if( TestPin && TestPin->PinName == FK2Node_SpawnActorHelper::BlueprintPinName )
		{
			Pin = TestPin;
			break;
		}
	}
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActor::GetSpawnTransformPin()const
{
	UEdGraphPin* Pin = FindPinChecked(FK2Node_SpawnActorHelper::SpawnTransformPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActor::GetNoCollisionFailPin()const
{
	UEdGraphPin* Pin = FindPinChecked(FK2Node_SpawnActorHelper::NoCollisionFailPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActor::GetWorldContextPin() const
{
	UEdGraphPin* Pin = FindPin(FK2Node_SpawnActorHelper::WorldContextPinName);
	check(Pin == NULL || Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActor::GetResultPin() const
{
	UEdGraphPin* Pin = FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	check(Pin->Direction == EGPD_Output);
	return Pin;
}

FLinearColor UK2Node_SpawnActor::GetNodeTitleColor() const
{
	return Super::GetNodeTitleColor();
}


FText UK2Node_SpawnActor::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	UEdGraphPin* BlueprintPin = GetBlueprintPin();
	if (BlueprintPin == NULL)
	{
		return NSLOCTEXT("K2Node", "SpawnActorNone_Title", "SpawnActor NONE");
	}
	else if (BlueprintPin->LinkedTo.Num() > 0)
	{
		// Blueprint will be determined dynamically, so we don't have the name in this case
		return NSLOCTEXT("K2Node", "SpawnActorUnknown_Title", "SpawnActor");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("ActorName"), FText::FromString(BlueprintPin->DefaultObject->GetName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SpawnActor", "SpawnActor {ActorName}"), Args), this);
	}
	return CachedNodeTitle;
}

bool UK2Node_SpawnActor::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const 
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return Super::IsCompatibleWithGraph(TargetGraph) && (!Blueprint || FBlueprintEditorUtils::FindUserConstructionScript(Blueprint) != TargetGraph);
}

FNodeHandlingFunctor* UK2Node_SpawnActor::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_SpawnActor::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	static FName BeginSpawningBlueprintFuncName = GET_FUNCTION_NAME_CHECKED(UGameplayStatics, BeginSpawningActorFromBlueprint);
	static FName BlueprintParamName(TEXT("Blueprint"));
	static FName WorldContextParamName(TEXT("WorldContextObject"));

	static FName FinishSpawningFuncName = GET_FUNCTION_NAME_CHECKED(UGameplayStatics, FinishSpawningActor);
	static FName ActorParamName(TEXT("Actor"));
	static FName TransformParamName(TEXT("SpawnTransform"));
	static FName NoCollisionFailParamName(TEXT("bNoCollisionFail"));

	static FName ObjectParamName(TEXT("Object"));
	static FName ValueParamName(TEXT("Value"));
	static FName PropertyNameParamName(TEXT("PropertyName"));

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	UEdGraphPin* SpawnNodeExec = GetExecPin();
	UEdGraphPin* SpawnNodeTransform = GetSpawnTransformPin();
	UEdGraphPin* SpawnNodeNoCollisionFail = GetNoCollisionFailPin();
	UEdGraphPin* SpawnWorldContextPin = GetWorldContextPin();
	UEdGraphPin* SpawnBlueprintPin = GetBlueprintPin();
	UEdGraphPin* SpawnNodeThen = GetThenPin();
	UEdGraphPin* SpawnNodeResult = GetResultPin();

	UBlueprint* SpawnBlueprint = NULL;
	if(SpawnBlueprintPin != NULL)
	{
		SpawnBlueprint = Cast<UBlueprint>(SpawnBlueprintPin->DefaultObject);
	}

	if(NULL == SpawnBlueprint)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("SpawnActorNodeMissingBlueprint_Error", "Spawn node @@ must have a blueprint specified.").ToString(), this);
		// we break exec links so this is the only error we get, don't want the SpawnActor node being considered and giving 'unexpected node' type warnings
		BreakAllNodeLinks();
		return;
	}

	if(0 == SpawnBlueprintPin->LinkedTo.Num())	
	{
		if(NULL == SpawnBlueprint)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("SpawnActorNodeMissingBlueprint_Error", "Spawn node @@ must have a blueprint specified.").ToString(), this);
			// we break exec links so this is the only error we get, don't want the SpawnActor node being considered and giving 'unexpected node' type warnings
			BreakAllNodeLinks();
			return;
		}

		// check if default blueprint is based on Actor
		const UClass* GeneratedClass = SpawnBlueprint->GeneratedClass;
		bool bInvalidBase = GeneratedClass && !GeneratedClass->IsChildOf(AActor::StaticClass());

		const UClass* SkeletonGeneratedClass = SpawnBlueprint->SkeletonGeneratedClass;
		bInvalidBase |= SkeletonGeneratedClass && !SkeletonGeneratedClass->IsChildOf(AActor::StaticClass());

		if(bInvalidBase)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("SpawnActorNodeInvalidBlueprint_Error", "Spawn node @@ must have a blueprint based on Actor specified.").ToString(), this);
			// we break exec links so this is the only error we get, don't want the SpawnActor node being considered and giving 'unexpected node' type warnings
			BreakAllNodeLinks();
			return;
		}
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'begin spawn' call node
	UK2Node_CallFunction* CallBeginSpawnNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallBeginSpawnNode->FunctionReference.SetExternalMember(BeginSpawningBlueprintFuncName, UGameplayStatics::StaticClass());
	CallBeginSpawnNode->AllocateDefaultPins();

	UEdGraphPin* CallBeginExec = CallBeginSpawnNode->GetExecPin();
	UEdGraphPin* CallBeginWorldContextPin = CallBeginSpawnNode->FindPinChecked(WorldContextParamName);
	UEdGraphPin* CallBeginBlueprintPin = CallBeginSpawnNode->FindPinChecked(BlueprintParamName);
	UEdGraphPin* CallBeginTransform = CallBeginSpawnNode->FindPinChecked(TransformParamName);
	UEdGraphPin* CallBeginNoCollisionFail = CallBeginSpawnNode->FindPinChecked(NoCollisionFailParamName);
	UEdGraphPin* CallBeginResult = CallBeginSpawnNode->GetReturnValuePin();

	// Move 'exec' connection from spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeExec, *CallBeginExec);

	if(SpawnBlueprintPin->LinkedTo.Num() > 0)
	{
		// Copy the 'blueprint' connection from the spawn node to 'begin spawn'
		CompilerContext.MovePinLinksToIntermediate(*SpawnBlueprintPin, *CallBeginBlueprintPin);
	}
	else
	{
		// Copy blueprint literal onto begin spawn call 
		CallBeginBlueprintPin->DefaultObject = SpawnBlueprint;
	}

	// Copy the world context connection from the spawn node to 'begin spawn' if necessary
	if (SpawnWorldContextPin)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnWorldContextPin, *CallBeginWorldContextPin);
	}

	// Copy the 'transform' connection from the spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeTransform, *CallBeginTransform);
		
	// Copy the 'bNoCollisionFail' connection from the spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeNoCollisionFail, *CallBeginNoCollisionFail);

	//////////////////////////////////////////////////////////////////////////
	// create 'finish spawn' call node
	UK2Node_CallFunction* CallFinishSpawnNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	CallFinishSpawnNode->FunctionReference.SetExternalMember(FinishSpawningFuncName, UGameplayStatics::StaticClass());
	CallFinishSpawnNode->AllocateDefaultPins();

	UEdGraphPin* CallFinishExec = CallFinishSpawnNode->GetExecPin();
	UEdGraphPin* CallFinishThen = CallFinishSpawnNode->GetThenPin();
	UEdGraphPin* CallFinishActor = CallFinishSpawnNode->FindPinChecked(ActorParamName);
	UEdGraphPin* CallFinishTransform = CallFinishSpawnNode->FindPinChecked(TransformParamName);
	UEdGraphPin* CallFinishResult = CallFinishSpawnNode->GetReturnValuePin();

	// Move 'then' connection from spawn node to 'finish spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *CallFinishThen);
			
	// Copy transform connection
	CompilerContext.CopyPinLinksToIntermediate(*CallBeginTransform, *CallFinishTransform);
		
	// Connect output actor from 'begin' to 'finish'
	CallBeginResult->MakeLinkTo(CallFinishActor);

	// Move result connection from spawn node to 'finish spawn'
	CallFinishResult->PinType = SpawnNodeResult->PinType; // Copy type so it uses the right actor subclass
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallFinishResult);

	//////////////////////////////////////////////////////////////////////////
	// create 'set var' nodes

	// Get 'result' pin from 'begin spawn', this is the actual actor we want to set properties on
	UK2Node_CallFunction* LastNode = CallBeginSpawnNode;

	// Create 'set var by name' nodes and hook them up
	for(int32 PinIdx=0; PinIdx < Pins.Num(); PinIdx++)
	{
		// Only create 'set param by name' node if this pin is linked to something
		UEdGraphPin* SpawnVarPin = Pins[PinIdx];
		if(SpawnVarPin->LinkedTo.Num() > 0)
		{
			UFunction* SetByNameFunction = Schema->FindSetVariableByNameFunction(SpawnVarPin->PinType);
			if(SetByNameFunction)
			{
				UK2Node_CallFunction* SetVarNode = NULL;
				if(SpawnVarPin->PinType.IsArray())
				{
					SetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallArrayFunction>(this, SourceGraph);
				}
				else
				{
					SetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				}
				SetVarNode->SetFromFunction(SetByNameFunction);
				SetVarNode->AllocateDefaultPins();

				// Connect this node into the exec chain
				UEdGraphPin* LastThen = LastNode->GetThenPin();
				UEdGraphPin* SetVarExec = SetVarNode->GetExecPin();
				LastThen->MakeLinkTo(SetVarExec);

				// Connect the new actor to the 'object' pin
				UEdGraphPin* ObjectPin = SetVarNode->FindPinChecked(ObjectParamName);
				CallBeginResult->MakeLinkTo(ObjectPin);

				// Fill in literal for 'property name' pin - name of pin is property name
				UEdGraphPin* PropertyNamePin = SetVarNode->FindPinChecked(PropertyNameParamName);
				PropertyNamePin->DefaultValue = SpawnVarPin->PinName.ToString();

				// Move connection from the variable pin on the spawn node to the 'value' pin
				UEdGraphPin* ValuePin = SetVarNode->FindPinChecked(ValueParamName);
				CompilerContext.MovePinLinksToIntermediate(*SpawnVarPin, *ValuePin);
				if(SpawnVarPin->PinType.IsArray())
				{
					SetVarNode->PinConnectionListChanged(ValuePin);
				}

				// Update 'last node in sequence' var
				LastNode = SetVarNode;
			}
		}
	}

	// Make exec connection between 'then' on last node and 'finish'
	UEdGraphPin* LastThen = LastNode->GetThenPin();
	LastThen->MakeLinkTo(CallFinishExec);

	// Break any links to the expanded node
	BreakAllNodeLinks();
}

bool UK2Node_SpawnActor::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UClass* SourceClass = GetClassToSpawn();
	const UBlueprint* SourceBlueprint = GetBlueprint();
	const bool bResult = (SourceClass != NULL) && (SourceClass->ClassGeneratedBy.Get() != SourceBlueprint);
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(SourceClass);
	}
	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_SpawnActor::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	UClass* ClassToSpawn = GetClassToSpawn();
	const FString ClassToSpawnStr = ClassToSpawn ? ClassToSpawn->GetName() : TEXT( "InvalidClass" );
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "SpawnActor" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "ActorClass" ), ClassToSpawnStr ));
}

bool UK2Node_SpawnActor::IsDeprecated() const
{
	return false;
}

FEdGraphNodeDeprecationResponse UK2Node_SpawnActor::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeTypeIsDeprecated)
	{
		Response.MessageType = EEdGraphNodeDeprecationMessageType::None;
		Response.MessageText = LOCTEXT("SpawnActorNodeOnlyDefaultBlueprint_Deprecatio", "Spawn Actor @@ is DEPRECATED and should be replaced by SpawnActorFromClass");
	}

	return Response;
}

FSlateIcon UK2Node_SpawnActor::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.SpawnActor_16x");
	return Icon;
}

#undef LOCTEXT_NAMESPACE
