// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SpawnActorFromClass.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/EngineTypes.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "K2Node_CallFunction.h"
#include "K2Node_EnumLiteral.h"
#include "K2Node_Select.h"
#include "Kismet/GameplayStatics.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Math/Transform.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UE5MainStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SpawnActorFromClass)

struct FLinearColor;

struct FK2Node_SpawnActorFromClassHelper
{
	static const FName SpawnTransformPinName;
	static const FName SpawnEvenIfCollidingPinName;
	static const FName NoCollisionFailPinName;
	static const FName CollisionHandlingOverridePinName;
	static const FName TransformScaleMethodPinName;
	static const FName OwnerPinName;
};

const FName FK2Node_SpawnActorFromClassHelper::SpawnTransformPinName(TEXT("SpawnTransform"));
const FName FK2Node_SpawnActorFromClassHelper::SpawnEvenIfCollidingPinName(TEXT("SpawnEvenIfColliding"));		// deprecated pin, name kept for backwards compat
const FName FK2Node_SpawnActorFromClassHelper::NoCollisionFailPinName(TEXT("bNoCollisionFail"));		// deprecated pin, name kept for backwards compat
const FName FK2Node_SpawnActorFromClassHelper::CollisionHandlingOverridePinName(TEXT("CollisionHandlingOverride"));
const FName FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName(TEXT("TransformScaleMethod"));
const FName FK2Node_SpawnActorFromClassHelper::OwnerPinName(TEXT("Owner"));

#define LOCTEXT_NAMESPACE "K2Node_SpawnActorFromClass"

UK2Node_SpawnActorFromClass::UK2Node_SpawnActorFromClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	NodeTooltip = LOCTEXT("NodeTooltip", "Attempts to spawn a new Actor with the specified transform");
}

UClass* UK2Node_SpawnActorFromClass::GetClassPinBaseClass() const
{
	return AActor::StaticClass();
}

void UK2Node_SpawnActorFromClass::AllocateDefaultPins()
{
	Super::AllocateDefaultPins();

	// Transform pin
	UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
	UEdGraphPin* TransformPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Struct, TransformStruct, FK2Node_SpawnActorFromClassHelper::SpawnTransformPinName);

	// Collision handling method pin
	UEnum* const CollisionMethodEnum = StaticEnum<ESpawnActorCollisionHandlingMethod>();
	UEdGraphPin* const CollisionHandlingOverridePin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, CollisionMethodEnum, FK2Node_SpawnActorFromClassHelper::CollisionHandlingOverridePinName);
	CollisionHandlingOverridePin->DefaultValue = CollisionMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorCollisionHandlingMethod::Undefined));
	
	// Pin to set transform scaling behavior (ie whether to multiply scale with root component or to just ignore the root component default scale)
	UEnum* const ScaleMethodEnum = StaticEnum<ESpawnActorScaleMethod>();
	UEdGraphPin* const ScaleMethodPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, ScaleMethodEnum, FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName);
	ScaleMethodPin->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::SelectDefaultAtRuntime));
	ScaleMethodPin->bAdvancedView = true;
	
	UEdGraphPin* OwnerPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, AActor::StaticClass(), FK2Node_SpawnActorFromClassHelper::OwnerPinName);
	OwnerPin->bAdvancedView = true;
	if (ENodeAdvancedPins::NoPins == AdvancedPinDisplay)
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}
}

void UK2Node_SpawnActorFromClass::MaybeUpdateCollisionPin(TArray<UEdGraphPin*>& OldPins)
{
	// see if there's a bNoCollisionFail pin
	for (UEdGraphPin* Pin : OldPins)
	{
		if (Pin->PinName == FK2Node_SpawnActorFromClassHelper::NoCollisionFailPinName || Pin->PinName == FK2Node_SpawnActorFromClassHelper::SpawnEvenIfCollidingPinName)
		{
			bool bHadOldCollisionPin = true;
			if (Pin->LinkedTo.Num() == 0)
			{
				// no links, use the default value of the pin
				bool const bOldCollisionPinValue = (Pin->DefaultValue == FString(TEXT("true")));

				UEdGraphPin* const CollisionHandlingOverridePin = GetCollisionHandlingOverridePin();
				if (CollisionHandlingOverridePin)
				{
					UEnum const* const MethodEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.ESpawnActorCollisionHandlingMethod"), EFindObjectFlags::ExactClass);
					CollisionHandlingOverridePin->DefaultValue =
						bOldCollisionPinValue
						? MethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorCollisionHandlingMethod::AlwaysSpawn))
						: MethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding));
				}
			}
			else
			{
				// something was linked. we will just move the links to the new pin
				// #note: this will be an invalid linkage and the BP compiler will complain, and that's intentional
				// so that users will be able to see and fix issues
				UEdGraphPin* const CollisionHandlingOverridePin = GetCollisionHandlingOverridePin();
				check(CollisionHandlingOverridePin);

				UEnum* const MethodEnum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/Engine.ESpawnActorCollisionHandlingMethod"), EFindObjectFlags::ExactClass);
				
				FGraphNodeCreator<UK2Node_EnumLiteral> AlwaysSpawnLiteralCreator(*GetGraph());
				UK2Node_EnumLiteral* const AlwaysSpawnLiteralNode = AlwaysSpawnLiteralCreator.CreateNode();
				AlwaysSpawnLiteralNode->Enum = MethodEnum;
				AlwaysSpawnLiteralNode->NodePosX = NodePosX;
				AlwaysSpawnLiteralNode->NodePosY = NodePosY;
				AlwaysSpawnLiteralCreator.Finalize();
			
				FGraphNodeCreator<UK2Node_EnumLiteral> AdjustIfNecessaryLiteralCreator(*GetGraph());
				UK2Node_EnumLiteral* const AdjustIfNecessaryLiteralNode = AdjustIfNecessaryLiteralCreator.CreateNode();
				AdjustIfNecessaryLiteralNode->Enum = MethodEnum;
				AdjustIfNecessaryLiteralNode->NodePosX = NodePosX;
				AdjustIfNecessaryLiteralNode->NodePosY = NodePosY;
				AdjustIfNecessaryLiteralCreator.Finalize();

				FGraphNodeCreator<UK2Node_Select> SelectCreator(*GetGraph());
				UK2Node_Select* const SelectNode = SelectCreator.CreateNode();
				SelectNode->NodePosX = NodePosX;
				SelectNode->NodePosY = NodePosY;
				SelectCreator.Finalize();

				// find pins we want to set and link up
				auto FindEnumInputPin = [](UK2Node_EnumLiteral const* Node)
				{
					for (UEdGraphPin* NodePin : Node->Pins)
					{
						if (NodePin->PinName == Node->GetEnumInputPinName())
						{
							return NodePin;
						}
					}
					return (UEdGraphPin*)nullptr;
				};

				UEdGraphPin* const AlwaysSpawnLiteralNodeInputPin = FindEnumInputPin(AlwaysSpawnLiteralNode);
				UEdGraphPin* const AdjustIfNecessaryLiteralInputPin = FindEnumInputPin(AdjustIfNecessaryLiteralNode);

				TArray<UEdGraphPin*> SelectOptionPins;
				SelectNode->GetOptionPins(SelectOptionPins);
				UEdGraphPin* const SelectIndexPin = SelectNode->GetIndexPin();

				auto FindResultPin = [](UK2Node const* Node)
				{
					for (UEdGraphPin* NodePin : Node->Pins)
					{
						if (EEdGraphPinDirection::EGPD_Output == NodePin->Direction)
						{
							return NodePin;
						}
					}
					return (UEdGraphPin*)nullptr;
				};
				UEdGraphPin* const AlwaysSpawnLiteralNodeResultPin = FindResultPin(AlwaysSpawnLiteralNode);
				check(AlwaysSpawnLiteralNodeResultPin);
				UEdGraphPin* const AdjustIfNecessaryLiteralResultPin = FindResultPin(AdjustIfNecessaryLiteralNode);
				check(AdjustIfNecessaryLiteralResultPin);

				UEdGraphPin* const OldBoolPin = Pin->LinkedTo[0];
				check(OldBoolPin);

				//
				// now set data and links that we want to set
				//

				AlwaysSpawnLiteralNodeInputPin->DefaultValue = MethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorCollisionHandlingMethod::AlwaysSpawn));
				AdjustIfNecessaryLiteralInputPin->DefaultValue = MethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButDontSpawnIfColliding));

				OldBoolPin->BreakLinkTo(Pin);
				OldBoolPin->MakeLinkTo(SelectIndexPin);

				AlwaysSpawnLiteralNodeResultPin->MakeLinkTo(SelectOptionPins[0]);
				AdjustIfNecessaryLiteralResultPin->MakeLinkTo(SelectOptionPins[1]);
				
				UEdGraphPin* const SelectOutputPin = SelectNode->GetReturnValuePin();
				check(SelectOutputPin);
				SelectOutputPin->MakeLinkTo(CollisionHandlingOverridePin);

				// tell select node to update its wildcard status
				SelectNode->NotifyPinConnectionListChanged(SelectIndexPin);
				SelectNode->NotifyPinConnectionListChanged(SelectOptionPins[0]);
				SelectNode->NotifyPinConnectionListChanged(SelectOptionPins[1]);
				SelectNode->NotifyPinConnectionListChanged(SelectOutputPin);

			}
		}
	}
}


void UK2Node_SpawnActorFromClass::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) 
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	MaybeUpdateCollisionPin(OldPins);
}

bool UK2Node_SpawnActorFromClass::IsSpawnVarPin(UEdGraphPin* Pin) const
{
	UEdGraphPin* ParentPin = Pin->ParentPin;
	while (ParentPin)
	{
		if (ParentPin->PinName == FK2Node_SpawnActorFromClassHelper::SpawnTransformPinName)
		{
			return false;
		}
		ParentPin = ParentPin->ParentPin;
	}

	return(	Super::IsSpawnVarPin(Pin) &&
			Pin->PinName != FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName &&
			Pin->PinName != FK2Node_SpawnActorFromClassHelper::CollisionHandlingOverridePinName &&
			Pin->PinName != FK2Node_SpawnActorFromClassHelper::SpawnTransformPinName && 
			Pin->PinName != FK2Node_SpawnActorFromClassHelper::OwnerPinName );
}

void UK2Node_SpawnActorFromClass::GetPinHoverText(const UEdGraphPin& Pin, FString& HoverTextOut) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	if (UEdGraphPin* TransformPin = GetSpawnTransformPin())
	{
		K2Schema->ConstructBasicPinTooltip(*TransformPin, LOCTEXT("TransformPinDescription", "The transform to spawn the Actor with"), TransformPin->PinToolTip);
	}
	if (UEdGraphPin* CollisionHandlingOverridePin = GetCollisionHandlingOverridePin())
	{
		K2Schema->ConstructBasicPinTooltip(*CollisionHandlingOverridePin, LOCTEXT("CollisionHandlingOverridePinDescription", "Specifies how to handle collisions at the spawn point. If undefined, uses actor class settings."), CollisionHandlingOverridePin->PinToolTip);
	}
	if (UEdGraphPin* OwnerPin = GetOwnerPin())
	{
		K2Schema->ConstructBasicPinTooltip(*OwnerPin, LOCTEXT("OwnerPinDescription", "Can be left empty; primarily used for replication (bNetUseOwnerRelevancy and bOnlyRelevantToOwner), or visibility (PrimitiveComponent's bOwnerNoSee/bOnlyOwnerSee)"), OwnerPin->PinToolTip);
	}

	return Super::GetPinHoverText(Pin, HoverTextOut);
}

FSlateIcon UK2Node_SpawnActorFromClass::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.SpawnActor_16x");
	return Icon;
}

UEdGraphPin* UK2Node_SpawnActorFromClass::GetSpawnTransformPin() const
{
	UEdGraphPin* Pin = FindPinChecked(FK2Node_SpawnActorFromClassHelper::SpawnTransformPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActorFromClass::GetCollisionHandlingOverridePin() const
{
	UEdGraphPin* const Pin = FindPinChecked(FK2Node_SpawnActorFromClassHelper::CollisionHandlingOverridePinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActorFromClass::GetScaleMethodPin() const
{
	UEdGraphPin* const Pin = FindPinChecked(FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName);
	check(Pin->Direction == EGPD_Input);
	return Pin;
}

UEdGraphPin* UK2Node_SpawnActorFromClass::TryGetScaleMethodPin() const
{
	return FindPin(FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName, EGPD_Input);
}

UEdGraphPin* UK2Node_SpawnActorFromClass::GetOwnerPin() const
{
	UEdGraphPin* Pin = FindPin(FK2Node_SpawnActorFromClassHelper::OwnerPinName);
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

FText UK2Node_SpawnActorFromClass::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle = NSLOCTEXT("K2Node", "SpawnActor_BaseTitle", "Spawn Actor from Class");
	if (TitleType != ENodeTitleType::MenuTitle)
	{
		if (UEdGraphPin* ClassPin = GetClassPin())
		{
			if (ClassPin->LinkedTo.Num() > 0)
			{
				// Blueprint will be determined dynamically, so we don't have the name in this case
				NodeTitle = NSLOCTEXT("K2Node", "SpawnActor_Title_Unknown", "SpawnActor");
			}
			else if (ClassPin->DefaultObject == nullptr)
			{
				NodeTitle = NSLOCTEXT("K2Node", "SpawnActor_Title_NONE", "SpawnActor NONE");
			}
			else
			{
				if (CachedNodeTitle.IsOutOfDate(this))
				{
					FText ClassName;
					if (UClass* PickedClass = Cast<UClass>(ClassPin->DefaultObject))
					{
						ClassName = PickedClass->GetDisplayNameText();
					}

					FFormatNamedArguments Args;
					Args.Add(TEXT("ClassName"), ClassName);

					// FText::Format() is slow, so we cache this to save on performance
					CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SpawnActor_Title_Class", "SpawnActor {ClassName}"), Args), this);
				}
				NodeTitle = CachedNodeTitle;
			} 
		}
		else
		{
			NodeTitle = NSLOCTEXT("K2Node", "SpawnActor_Title_NONE", "SpawnActor NONE");
		}
	}
	return NodeTitle;
}

bool UK2Node_SpawnActorFromClass::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const 
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	const UClass* Class = Blueprint ? Blueprint->GeneratedClass.Get() : nullptr;
	return Super::IsCompatibleWithGraph(TargetGraph) &&
		(	!Blueprint || 
			(FBlueprintEditorUtils::FindUserConstructionScript(Blueprint) != TargetGraph && Class && Class->GetDefaultObject()->ImplementsGetWorld()));
}

void UK2Node_SpawnActorFromClass::PostPlacedNewNode()
{
	UEdGraphPin* const ScaleMethodPin = GetScaleMethodPin();
	const UEnum* const ScaleMethodEnum = StaticEnum<ESpawnActorScaleMethod>();
	ScaleMethodPin->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::MultiplyWithRoot));
}

void UK2Node_SpawnActorFromClass::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
}

void UK2Node_SpawnActorFromClass::PostLoad()
{
	FixupScaleMethodPin();
	Super::PostLoad();
}

void UK2Node_SpawnActorFromClass::FixupScaleMethodPin()
{
	// if this node is being diffed, don't fix up anything. Keep the legacy pins
	const UPackage* Package = GetPackage();
	if (Package && Package->HasAnyPackageFlags(PKG_ForDiffing))
	{
		return;
	}

	// For the sake of cook determinism, leave the scale method as ESpawnActorScaleMethod::SelectDefaultAtRuntime when cooking.
	// This shouldn't impact runtime behavior.
	if (IsRunningCookCommandlet())
	{
		return;
	}

	if (GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::SpawnActorFromClassTransformScaleMethod)
	{
        if (UEdGraphPin* const ScaleMethodPin = TryGetScaleMethodPin())
        {
			const UEdGraphPin* const ClassPin = FindPin(TEXT( "Class" ));
	        const UEnum* const ScaleMethodEnum = StaticEnum<ESpawnActorScaleMethod>();
            if (const UClass* Class = Cast<UClass>(ClassPin->DefaultObject))
            {
            	if (const AActor* ActorCDO = Cast<AActor>(Class->GetDefaultObject(false)))
                {
                    if (const USceneComponent* Root = ActorCDO->GetRootComponent()) // native root component
                    {
                        ScaleMethodPin->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::MultiplyWithRoot));
                    }
                    else
                    {
                        ScaleMethodPin->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::OverrideRootScale));
                    }
                }
            }
            else
            {
            	// if the class can't be determined during compile time, defer the default value to runtime
            	ScaleMethodPin->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::SelectDefaultAtRuntime));
            }
        }
        else
        {
	        UE_LOGF(LogBlueprint, Warning, "Blueprint Node '%ls' is missing '%ls' pin. Proceeding as if it's default is SelectDefaultAtRuntime",
	        	*GetNodeTitle(ENodeTitleType::FullTitle).ToString(),
	        	*FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName.ToString()
	        )
        }
	}
}

void UK2Node_SpawnActorFromClass::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	UClass* ClassToSpawn = GetClassToSpawn();
	const FString ClassToSpawnStr = ClassToSpawn ? ClassToSpawn->GetName() : TEXT( "InvalidClass" );
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "SpawnActorFromClass" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "ActorClass" ), ClassToSpawnStr ));
}

FNodeHandlingFunctor* UK2Node_SpawnActorFromClass::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandlingFunctor(CompilerContext);
}

void UK2Node_SpawnActorFromClass::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	// make sure that if scale method needs to be given a default value, it gets it
	FixupScaleMethodPin();

	static const FName BeginSpawningBlueprintFuncName = GET_FUNCTION_NAME_CHECKED(UGameplayStatics, BeginDeferredActorSpawnFromClass);
	static const FName ActorClassParamName(TEXT("ActorClass"));
	static const FName WorldContextParamName(TEXT("WorldContextObject"));

	static const FName FinishSpawningFuncName = GET_FUNCTION_NAME_CHECKED(UGameplayStatics, FinishSpawningActor);
	static const FName ActorParamName(TEXT("Actor"));
	static const FName TransformParamName(TEXT("SpawnTransform"));
	static const FName CollisionHandlingOverrideParamName(TEXT("CollisionHandlingOverride"));
	static const FName OwnerParamName(TEXT("Owner"));

	static const FName ObjectParamName(TEXT("Object"));
	static const FName ValueParamName(TEXT("Value"));
	static const FName PropertyNameParamName(TEXT("PropertyName"));

	UK2Node_SpawnActorFromClass* SpawnNode = this;
	UEdGraphPin* SpawnNodeExec = SpawnNode->GetExecPin();
	UEdGraphPin* SpawnNodeTransform = SpawnNode->GetSpawnTransformPin();
	UEdGraphPin* SpawnNodeCollisionHandlingOverride = GetCollisionHandlingOverridePin();
	UEdGraphPin* SpawnNodeScaleMethod = TryGetScaleMethodPin();
	UEdGraphPin* SpawnWorldContextPin = SpawnNode->GetWorldContextPin();
	UEdGraphPin* SpawnClassPin = SpawnNode->GetClassPin();
	UEdGraphPin* SpawnNodeOwnerPin = SpawnNode->GetOwnerPin();
	UEdGraphPin* SpawnNodeThen = SpawnNode->GetThenPin();
	UEdGraphPin* SpawnNodeResult = SpawnNode->GetResultPin();

	// Cache the class to spawn. Note, this is the compile time class that the pin was set to or the variable type it was connected to. Runtime it could be a child.
	UClass* ClassToSpawn = GetClassToSpawn();

	UClass* SpawnClass = (SpawnClassPin != NULL) ? Cast<UClass>(SpawnClassPin->DefaultObject) : NULL;
	if ( !SpawnClassPin || ((0 == SpawnClassPin->LinkedTo.Num()) && (NULL == SpawnClass)))
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("SpawnActorNodeMissingClass_Error", "Spawn node @@ must have a @@ specified.").ToString(), SpawnNode, SpawnClassPin);
		// we break exec links so this is the only error we get, don't want the SpawnActor node being considered and giving 'unexpected node' type warnings
		SpawnNode->BreakAllNodeLinks();
		return;
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'begin spawn' call node
	UK2Node_CallFunction* CallBeginSpawnNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(SpawnNode, SourceGraph);
	CallBeginSpawnNode->FunctionReference.SetExternalMember(BeginSpawningBlueprintFuncName, UGameplayStatics::StaticClass());
	CallBeginSpawnNode->AllocateDefaultPins();

	UEdGraphPin* CallBeginExec = CallBeginSpawnNode->GetExecPin();
	UEdGraphPin* CallBeginWorldContextPin = CallBeginSpawnNode->FindPinChecked(WorldContextParamName);
	UEdGraphPin* CallBeginActorClassPin = CallBeginSpawnNode->FindPinChecked(ActorClassParamName);
	UEdGraphPin* CallBeginTransform = CallBeginSpawnNode->FindPinChecked(TransformParamName);
	UEdGraphPin* CallBeginScaleMethod = CallBeginSpawnNode->FindPinChecked(FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName);
	UEdGraphPin* CallBeginCollisionHandlingOverride = CallBeginSpawnNode->FindPinChecked(CollisionHandlingOverrideParamName);

	UEdGraphPin* CallBeginOwnerPin = CallBeginSpawnNode->FindPinChecked(FK2Node_SpawnActorFromClassHelper::OwnerPinName);
	UEdGraphPin* CallBeginResult = CallBeginSpawnNode->GetReturnValuePin();	

	// Move 'exec' connection from spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeExec, *CallBeginExec);

	if(SpawnClassPin->LinkedTo.Num() > 0)
	{
		// Copy the 'blueprint' connection from the spawn node to 'begin spawn'
		CompilerContext.MovePinLinksToIntermediate(*SpawnClassPin, *CallBeginActorClassPin);
	}
	else
	{
		// Copy blueprint literal onto begin spawn call 
		CallBeginActorClassPin->DefaultObject = SpawnClass;
	}

	// Copy the world context connection from the spawn node to 'begin spawn' if necessary
	if (SpawnWorldContextPin)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnWorldContextPin, *CallBeginWorldContextPin);
	}

	if (SpawnNodeOwnerPin != nullptr)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnNodeOwnerPin, *CallBeginOwnerPin);
	}

	// Copy the 'transform' connection from the spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeTransform, *CallBeginTransform);

	// Copy the 'CollisionHandlingOverride' connection from the spawn node to 'begin spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeCollisionHandlingOverride, *CallBeginCollisionHandlingOverride);

	// Copy the 'ScaleMethod' connection from the spawn node to 'begin spawn'
	if (SpawnNodeScaleMethod)
	{
		CompilerContext.MovePinLinksToIntermediate(*SpawnNodeScaleMethod, *CallBeginScaleMethod);
	}
	else
	{
		// if for some reason we couldn't find scale method pin on the spawn node, defer it's value to runtime
	    const UEnum* const ScaleMethodEnum = StaticEnum<ESpawnActorScaleMethod>();
		CallBeginScaleMethod->DefaultValue = ScaleMethodEnum->GetNameStringByValue(static_cast<int>(ESpawnActorScaleMethod::SelectDefaultAtRuntime));
	}

	//////////////////////////////////////////////////////////////////////////
	// create 'finish spawn' call node
	UK2Node_CallFunction* CallFinishSpawnNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(SpawnNode, SourceGraph);
	CallFinishSpawnNode->FunctionReference.SetExternalMember(FinishSpawningFuncName, UGameplayStatics::StaticClass());
	CallFinishSpawnNode->AllocateDefaultPins();

	UEdGraphPin* CallFinishExec = CallFinishSpawnNode->GetExecPin();
	UEdGraphPin* CallFinishThen = CallFinishSpawnNode->GetThenPin();
	UEdGraphPin* CallFinishActor = CallFinishSpawnNode->FindPinChecked(ActorParamName);
	UEdGraphPin* CallFinishTransform = CallFinishSpawnNode->FindPinChecked(TransformParamName);
	UEdGraphPin* CallFinishScaleMethod = CallFinishSpawnNode->FindPinChecked(FK2Node_SpawnActorFromClassHelper::TransformScaleMethodPinName);
	UEdGraphPin* CallFinishResult = CallFinishSpawnNode->GetReturnValuePin();

	// Move 'then' connection from spawn node to 'finish spawn'
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeThen, *CallFinishThen);

	// Copy transform connection
	CompilerContext.CopyPinLinksToIntermediate(*CallBeginTransform, *CallFinishTransform);
 
	// Copy 'ScaleMethod' connection
	CompilerContext.CopyPinLinksToIntermediate(*CallBeginScaleMethod, *CallFinishScaleMethod);

	// Connect output actor from 'begin' to 'finish'
	CallBeginResult->MakeLinkTo(CallFinishActor);

	// Move result connection from spawn node to 'finish spawn'
	CallFinishResult->PinType = SpawnNodeResult->PinType; // Copy type so it uses the right actor subclass
	CompilerContext.MovePinLinksToIntermediate(*SpawnNodeResult, *CallFinishResult);

	//////////////////////////////////////////////////////////////////////////
	// create 'set var' nodes

	// Get 'result' pin from 'begin spawn', this is the actual actor we want to set properties on
	UEdGraphPin* LastThen = FKismetCompilerUtilities::GenerateAssignmentNodes(CompilerContext, SourceGraph, CallBeginSpawnNode, SpawnNode, CallBeginResult, ClassToSpawn, CallBeginActorClassPin);

	// Make exec connection between 'then' on last node and 'finish'
	LastThen->MakeLinkTo(CallFinishExec);

	// Break any links to the expanded node
	SpawnNode->BreakAllNodeLinks();
}

#undef LOCTEXT_NAMESPACE
