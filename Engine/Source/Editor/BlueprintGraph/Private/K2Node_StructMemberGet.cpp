// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StructMemberGet.h"

#include "EdGraph/EdGraphSchema.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "K2Node.h"
#include "Misc/AssertionMacros.h"
#include "StructMemberNodeHandlers.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StructMemberGet)

class FKismetCompilerContext;

//////////////////////////////////////////////////////////////////////////
// UK2Node_StructMemberGet

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_StructMemberGet::UK2Node_StructMemberGet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_StructMemberGet::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);
	}
}

void UK2Node_StructMemberGet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		GetSchema()->ReconstructNode(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_StructMemberGet::AllocateDefaultPins()
{
	//@TODO: Create a context pin

	// Display any currently visible optional pins
	{
		FStructOperationOptionalPinManager OptionalPinManager;
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructType);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Output, this);
	}
}

void UK2Node_StructMemberGet::AllocatePinsForSingleMemberGet(FName MemberName)
{
	//@TODO: Create a context pin

	// Updater for subclasses that allow hiding pins
	struct FSingleVariablePinManager : public FOptionalPinManager
	{
		FName MatchName;

		FSingleVariablePinManager(FName InMatchName)
			: MatchName(InMatchName)
		{
		}

		// FOptionalPinsUpdater interface
		virtual void GetRecordDefaults(FProperty* TestProperty, FOptionalPinFromProperty& Record) const override
		{
			Record.bCanToggleVisibility = false;
			Record.bShowPin = TestProperty->GetFName() == MatchName;
		}
		// End of FOptionalPinsUpdater interface
	};


	// Display any currently visible optional pins
	{
		FSingleVariablePinManager PinManager(MemberName);
		PinManager.RebuildPropertyList(ShowPinForProperties, StructType);
		PinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Output, this);
	}
}

FText UK2Node_StructMemberGet::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromString(GetVarNameString()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(LOCTEXT("K2Node_StructMemberGet_Tooltip", "Get member variables of {VariableName}"), Args), this);
	}
	return CachedTooltip;
}

FText UK2Node_StructMemberGet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromString(GetVarNameString()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("GetMembersInVariable", "Get members in {VariableName}"), Args), this);
	}
	return CachedNodeTitle;
}

FNodeHandlingFunctor* UK2Node_StructMemberGet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_StructMemberVariableGet(CompilerContext);
}

#undef LOCTEXT_NAMESPACE
