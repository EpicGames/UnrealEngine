// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StructMemberSet.h"

#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/MemberReference.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "StructMemberNodeHandlers.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "PropertyCustomizationHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StructMemberSet)

class FKismetCompilerContext;
class UEdGraphPin;

//////////////////////////////////////////////////////////////////////////
// UK2Node_StructMemberSet

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_StructMemberSet::UK2Node_StructMemberSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_StructMemberSet::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::CacheShownPins(ShowPinForProperties, OldShownPins);
	}
}

void UK2Node_StructMemberSet::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FOptionalPinFromProperty, bShowPin))
	{
		FOptionalPinManager::EvaluateOldShownPins(ShowPinForProperties, OldShownPins, this);
		GetSchema()->ReconstructNode(*this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UK2Node_StructMemberSet::AllocateExecPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
}

void UK2Node_StructMemberSet::AllocateDefaultPins()
{
	// Add the execution sequencing pin
	AllocateExecPins();

	// Display any currently visible optional pins
	{
		FStructOperationOptionalPinManager OptionalPinManager;
		OptionalPinManager.RebuildPropertyList(ShowPinForProperties, StructType);
		OptionalPinManager.CreateVisiblePins(ShowPinForProperties, StructType, EGPD_Input, this);
	}
}

FText UK2Node_StructMemberSet::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromName(VariableReference.GetMemberName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(LOCTEXT("K2Node_StructMemberSet_Tooltip", "Set member variables of {VariableName}"), Args), this);
	}
	return CachedTooltip;
}

FText UK2Node_StructMemberSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableName"), FText::FromName(VariableReference.GetMemberName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("SetMembersInVariable", "Set members in {VariableName}"), Args), this);
	}
	return CachedNodeTitle;
}

UK2Node::ERedirectType UK2Node_StructMemberSet::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex)  const
{
	return UK2Node::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
}

FNodeHandlingFunctor* UK2Node_StructMemberSet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_StructMemberVariableSet(CompilerContext);
}

FBoolProperty* UK2Node_StructMemberSet::GetOverrideConditionForProperty(const FProperty* InProperty) const
{
	bool bNegate = false;
	if (FBoolProperty* OverrideProperty = PropertyCustomizationHelpers::GetEditConditionProperty(InProperty, bNegate))
	{
		// Determine if the edit condition is included in the optional input pin set - if so, then it is not considered to be an override condition.
		const bool bIsOverrideExposedForInput = ShowPinForProperties.ContainsByPredicate([OverrideProperty](const FOptionalPinFromProperty& PropertyEntry)
		{
			return PropertyEntry.PropertyName == OverrideProperty->GetFName();
		});

		if (!bIsOverrideExposedForInput)
		{
			return OverrideProperty;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
