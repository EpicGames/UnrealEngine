// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_StructOperation.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintFieldNodeSpawner.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "StructUtils/UserDefinedStruct.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UserDefinedStructure/UserDefinedStructEditorData.h"

//////////////////////////////////////////////////////////////////////////
// UK2Node_StructOperation

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_StructOperation)

UK2Node_StructOperation::UK2Node_StructOperation(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_StructOperation::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	// Skip UK2Node_Variable's validation because it doesn't need a property (see CL# 1756451)
	UK2Node::ValidateNodeDuringCompilation(MessageLog);
}

bool UK2Node_StructOperation::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	const bool bResult = nullptr != StructType;
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(StructType);
	}

	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

void UK2Node_StructOperation::FStructOperationOptionalPinManager::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex, FProperty* Property) const
{
	FOptionalPinManager::CustomizePinData(Pin, SourcePropertyName, ArrayIndex, Property);

	if (Pin && Property)
	{
		const UUserDefinedStruct* UDStructure = Cast<const UUserDefinedStruct>(Property->GetOwnerStruct());
		if (UDStructure && UDStructure->EditorData)
		{
			const FStructVariableDescription* VarDesc = FStructureEditorUtils::GetVarDesc(UDStructure).FindByPredicate(
				FStructureEditorUtils::FFindByNameHelper<FStructVariableDescription>(Property->GetFName()));
			if (VarDesc)
			{
				Pin->PersistentGuid = VarDesc->VarGuid;
			}
		}
	}
}

bool UK2Node_StructOperation::DoRenamedPinsMatch(const UEdGraphPin* NewPin, const UEdGraphPin* OldPin, bool bStructInVariablesOut) const
{
	if (NewPin && OldPin && (OldPin->Direction == NewPin->Direction))
	{
		const EEdGraphPinDirection StructDirection = bStructInVariablesOut ? EGPD_Input : EGPD_Output;
		const EEdGraphPinDirection VariablesDirection = bStructInVariablesOut ? EGPD_Output : EGPD_Input;
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>(); 
		const bool bCompatible = K2Schema && K2Schema->ArePinTypesCompatible(NewPin->PinType, OldPin->PinType);

		if (!bCompatible)
		{
			return false;
		}
		else if (StructDirection == OldPin->Direction)
		{
			// Struct name was changed, which is fine
			return true;
		}
		else if (VariablesDirection == OldPin->Direction)
		{
			// Name of a member variable was changed, check guids and redirects
			if ((NewPin->PersistentGuid == OldPin->PersistentGuid) && OldPin->PersistentGuid.IsValid())
			{
				return true;
			}

			if (DoesRenamedVariableMatch(OldPin->PinName, NewPin->PinName, StructType))
			{
				return true;
			}
		}
	}
	return false;
}

void UK2Node_StructOperation::SetupMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar, const FMakeStructSpawnerAllowedDelegate& AllowedDelegate, EEdGraphPinDirection PinDirectionToPromote) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeStruct(UEdGraphNode* NewNode, FFieldVariant /*StructField*/, TWeakObjectPtr<UScriptStruct> NonConstStructPtr)
		{
			UK2Node_StructOperation* StructNode = CastChecked<UK2Node_StructOperation>(NewNode);
			StructNode->StructType = NonConstStructPtr.Get();
		}

		static void OverrideCategory(FBlueprintActionContext const& Context, IBlueprintNodeBinder::FBindingSet const& /*Bindings*/, FBlueprintActionUiSpec* UiSpecOut, TWeakObjectPtr<UScriptStruct> StructPtr, EEdGraphPinDirection PinDirectionToPromote)
		{
			for (UEdGraphPin* Pin : Context.Pins)
			{
				UScriptStruct* PinStruct = Cast<UScriptStruct>(Pin->PinType.PinSubCategoryObject.Get());
				if ((PinStruct != nullptr) && (StructPtr.Get() == PinStruct) && (Pin->Direction == PinDirectionToPromote))
				{
					UiSpecOut->Category = NSLOCTEXT("BlueprintFunctionNodeSpawner", "EmptyFunctionCategory", "|");
					break;
				}
			}
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterStructActions(FBlueprintActionDatabaseRegistrar::FMakeStructSpawnerDelegate::CreateLambda([NodeClass, AllowedDelegate, PinDirectionToPromote](const UScriptStruct* Struct)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = nullptr;

		if (AllowedDelegate.Execute(Struct, false))
		{
			NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UScriptStruct*>(Struct));
			check(NodeSpawner != nullptr);
			TWeakObjectPtr<UScriptStruct> NonConstStructPtr = MakeWeakObjectPtr(const_cast<UScriptStruct*>(Struct));
			NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeStruct, NonConstStructPtr);
			NodeSpawner->DynamicUiSignatureGetter = UBlueprintFieldNodeSpawner::FUiSpecOverrideDelegate::CreateStatic(GetMenuActions_Utils::OverrideCategory, NonConstStructPtr, PinDirectionToPromote);

		}
		return NodeSpawner;
	}));
}

FString UK2Node_StructOperation::GetPinMetaData(FName InPinName, FName InKey)
{
	for (TFieldIterator<FProperty> It(StructType); It; ++It)
	{
		const FProperty* Property = *It;
		if(Property && Property->GetFName() == InPinName)
		{
			return Property->GetMetaData(InKey);
		}
	}
	return Super::GetPinMetaData(InPinName, InKey);
}

FString UK2Node_StructOperation::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	return UEdGraphNode::GetFindReferenceSearchString_Impl(InFlags);
}

bool UK2Node_StructOperation::IsActionFilteredOut(const FBlueprintActionFilter& Filter)
{
	bool bIsFiltered = false;
	if (StructType)
	{
		if (StructType->GetBoolMetaData(FBlueprintMetadata::MD_BlueprintInternalUseOnly))
		{
			bIsFiltered = true;
		}
		else if (StructType->GetBoolMetaDataHierarchical(FBlueprintMetadata::MD_BlueprintInternalUseOnlyHierarchical))
		{
			bIsFiltered = true;
		}
		else if (!StructType->GetBoolMetaData(FBlueprintMetadata::MD_AllowableBlueprintVariableType))
		{
			bIsFiltered = true;
			for (UEdGraphPin* ContextPin : Filter.Context.Pins)
			{
				if (ContextPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && ContextPin->PinType.PinSubCategoryObject == StructType)
				{
					bIsFiltered = false;
					break;
				}
			}
		}
	}
	return bIsFiltered;
}
