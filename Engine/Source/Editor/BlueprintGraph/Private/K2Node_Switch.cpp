// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_Switch.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphNodeUtils.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Switch)

#define LOCTEXT_NAMESPACE "K2Node_Switch"

namespace 
{
	static FName DefaultPinName(TEXT("Default"));
	static FName SelectionPinName(TEXT("Selection"));
}

//////////////////////////////////////////////////////////////////////////
// FKCHandler_Switch

class FKCHandler_Switch : public FNodeHandlingFunctor
{
protected:
	TMap<UEdGraphNode*, FBPTerminal*> BoolTermMap;

public:
	FKCHandler_Switch(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Switch* SwitchNode = Cast<UK2Node_Switch>(Node);

		FNodeHandlingFunctor::RegisterNets(Context, Node);

		// Create a term to determine if the compare was successful or not
		//@TODO: Ideally we just create one ever, not one per switch
		FBPTerminal* BoolTerm = Context.CreateLocalTerminal();
		BoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		BoolTerm->Source = Node;
		BoolTerm->Name = Context.NetNameMap->MakeValidName(Node, TEXT("CmpSuccess"));
		BoolTermMap.Add(Node, BoolTerm);
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Switch* SwitchNode = CastChecked<UK2Node_Switch>(Node);

		FEdGraphPinType ExpectedExecPinType;
		ExpectedExecPinType.PinCategory = UEdGraphSchema_K2::PC_Exec;

		// Make sure that the input pin is connected and valid for this block
		UEdGraphPin* ExecTriggeringPin = Context.FindRequiredPinByName(SwitchNode, UEdGraphSchema_K2::PN_Execute, EGPD_Input);
		if ((ExecTriggeringPin == NULL) || !Context.ValidatePinType(ExecTriggeringPin, ExpectedExecPinType))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoValidExecutionPinForSwitch_Error", "@@ must have a valid execution pin @@").ToString(), SwitchNode, ExecTriggeringPin);
			return;
		}

		// Make sure that the selection pin is connected and valid for this block
		UEdGraphPin* SelectionPin = SwitchNode->GetSelectionPin();
		if ((SelectionPin == NULL) || !Context.ValidatePinType(SelectionPin, SwitchNode->GetPinType()))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("NoValidSelectionPinForSwitch_Error", "@@ must have a valid execution pin @@").ToString(), SwitchNode, SelectionPin);
			return;
		}

		// Find the boolean intermediate result term, so we can track whether the compare was successful
		FBPTerminal* BoolTerm = BoolTermMap.FindRef(SwitchNode);

		// Generate the output impulse from this node
		UEdGraphPin* SwitchSelectionNet = FEdGraphUtilities::GetNetFromPin(SelectionPin);
		FBPTerminal* SwitchSelectionTerm = Context.NetMap.FindRef(SwitchSelectionNet);

		if ((BoolTerm != NULL) && (SwitchSelectionTerm != NULL))
		{
			UEdGraphPin* FuncPin = SwitchNode->GetFunctionPin();
			FBPTerminal* FuncContext = Context.NetMap.FindRef(FuncPin);
			UEdGraphPin* DefaultPin = SwitchNode->GetDefaultPin();
			
			// We don't need to generate if checks if there are no connections to it if there is no default pin or if the default pin is not linked 
			// If there is a default pin that is linked then it would fall through to that default if we do not generate the cases
			const bool bCanSkipUnlinkedCase = (DefaultPin == nullptr || DefaultPin->LinkedTo.Num() == 0);

			// Pull out function to use
			UClass* FuncClass = Cast<UClass>(FuncPin->PinType.PinSubCategoryObject.Get());
			UFunction* FunctionPtr = FindUField<UFunction>(FuncClass, FuncPin->PinName);
			check(FunctionPtr);

			// Run thru all the output pins except for the default label
			for (auto PinIt = SwitchNode->Pins.CreateIterator(); PinIt; ++PinIt)
			{
				UEdGraphPin* Pin = *PinIt;

				if ((Pin->Direction == EGPD_Output) && (Pin != DefaultPin) && (!bCanSkipUnlinkedCase || Pin->LinkedTo.Num() > 0))
				{
					// Create a term for the switch case value
					FBPTerminal* CaseValueTerm = new FBPTerminal();
					Context.Literals.Add(CaseValueTerm);
					CaseValueTerm->Name = SwitchNode->GetExportTextForPin(Pin);
					CaseValueTerm->Type = SwitchNode->GetInnerCaseType();
					CaseValueTerm->SourcePin = Pin;
					CaseValueTerm->bIsLiteral = true;

					// Call the comparison function associated with this switch node
					FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(SwitchNode);
					Statement.Type = KCST_CallFunction;
					Statement.FunctionToCall = FunctionPtr;
					Statement.FunctionContext = FuncContext;
					Statement.bIsParentContext = false;

					Statement.LHS = BoolTerm;
					Statement.RHS.Add(SwitchSelectionTerm);
					Statement.RHS.Add(CaseValueTerm);

					// Jump to output if strings are actually equal
					FBlueprintCompiledStatement& IfFailTest_SucceedAtBeingEqualGoto = Context.AppendStatementForNode(SwitchNode);
					IfFailTest_SucceedAtBeingEqualGoto.Type = KCST_GotoIfNot;
					IfFailTest_SucceedAtBeingEqualGoto.LHS = BoolTerm;

					Context.GotoFixupRequestMap.Add(&IfFailTest_SucceedAtBeingEqualGoto, Pin);
				}
			}

			// Finally output default pin
			GenerateSimpleThenGoto(Context, *SwitchNode, DefaultPin);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermPassed_Error", "Failed to resolve term passed into @@").ToString(), SelectionPin);
		}
	}

private:
	FEdGraphPinType ExpectedSelectionPinType;
};

UK2Node_Switch::UK2Node_Switch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bHasDefaultPin = true;
	bHasDefaultPinValueChanged = false;
}

void UK2Node_Switch::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == TEXT("bHasDefaultPin"))
	{
		// Signal to the reconstruction logic that the default pin value has changed
		bHasDefaultPinValueChanged = true;
		
		if (!bHasDefaultPin)
		{
			UEdGraphPin* DefaultPin = GetDefaultPin();
			if (DefaultPin)
			{
				const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
				K2Schema->BreakPinLinks(*DefaultPin, true);
			}
		}

		ReconstructNode();

		// Clear the default pin value change flag
		bHasDefaultPinValueChanged = false;

	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FName UK2Node_Switch::GetSelectionPinName()
{
	return SelectionPinName;
}

void UK2Node_Switch::AllocateDefaultPins()
{
	// Add default pin
	if (bHasDefaultPin)
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, DefaultPinName);
	}

	// Add exec input pin
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

	// Create selection pin based on type
	CreateSelectionPin();

	// Create a new function pin
	CreateFunctionPin();

	// Create any case pins if required
	CreateCasePins();
}

UK2Node::ERedirectType UK2Node_Switch::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	// If the default pin setting has changed, return a match for the "execute" input pin (which will have swapped slots), so that we don't have to break any links to it
	if(bHasDefaultPinValueChanged && ((OldPinIndex == 0) || (NewPinIndex == 0)))
	{
		if((bHasDefaultPin && OldPinIndex == 0 && NewPinIndex == 1)
			|| (!bHasDefaultPin && OldPinIndex == 1 && NewPinIndex == 0))
		{
			return ERedirectType_Name;
		}
	}
	else if (NewPin->PinName == OldPin->PinName)
	{
		// Compare the names, case-sensitively
		return ERedirectType_Name;
	}
	return ERedirectType_None;
}

FLinearColor UK2Node_Switch::GetNodeTitleColor() const
{
	// Use yellow for now
	return FLinearColor(255.0f, 255.0f, 0.0f);
}

FSlateIcon UK2Node_Switch::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Switch_16x");
	return Icon;
}

void UK2Node_Switch::AddPinToSwitchNode()
{
	const FName NewPinName = GetUniquePinName();
	if (!NewPinName.IsNone())
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, NewPinName);
	}
}

void UK2Node_Switch::RemovePinFromSwitchNode(UEdGraphPin* TargetPin) 
{
	// If removing the default pin, we'll need to reconstruct the node, so send a property changed event to handle that
	if(bHasDefaultPin && TargetPin == GetDefaultPin())
	{
		FProperty* HasDefaultPinProperty = FindFProperty<FProperty>(GetClass(), "bHasDefaultPin");
		if(HasDefaultPinProperty)
		{
			PreEditChange(HasDefaultPinProperty);

			bHasDefaultPin = false;

			FPropertyChangedEvent HasDefaultPinPropertyChangedEvent(HasDefaultPinProperty);
			PostEditChangeProperty(HasDefaultPinPropertyChangedEvent);
		}
	}
	else
	{
		RemovePin(TargetPin);

		TargetPin->MarkAsGarbage();
		Pins.Remove(TargetPin);
	}
}

bool UK2Node_Switch::CanRemoveExecutionPin(UEdGraphPin* TargetPin) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	// Don't allow removing last pin
	int32 NumExecPins = 0;
	for (int32 i = 0; i < Pins.Num(); ++i)
	{
		UEdGraphPin* PotentialPin = Pins[i];
		if (K2Schema->IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output))
		{
			NumExecPins++;
		}
	}

	return NumExecPins > 1;
}

// Returns the exec output pin name for a given 0-based index
FName UK2Node_Switch::GetPinNameGivenIndex(int32 Index) const
{
	return *FString::Printf(TEXT("%d"), Index);
}

void UK2Node_Switch::CreateFunctionPin()
{
	// Set properties on the function pin
	UEdGraphPin* FunctionPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, FunctionClass, FunctionName);
	FunctionPin->bDefaultValueIsReadOnly = true;
	FunctionPin->bNotConnectable = true;
	FunctionPin->bHidden = true;

	UFunction* Function = FindUField<UFunction>(FunctionClass, FunctionName);
	const bool bIsStaticFunc = Function ? Function->HasAllFunctionFlags(FUNC_Static) : false;
	if (bIsStaticFunc)
	{
		// Wire up the self to the CDO of the class if it's not us
		if (UBlueprint* BP = GetBlueprint())
		{
			UClass* FunctionOwnerClass = Function->GetOuterUClass();
			if (!BP->SkeletonGeneratedClass || !BP->SkeletonGeneratedClass->IsChildOf(FunctionOwnerClass))
			{
				FunctionPin->DefaultObject = FunctionOwnerClass->GetDefaultObject();
			}
		}
	}
}

UEdGraphPin* UK2Node_Switch::GetFunctionPin() const
{
	//@TODO: Should probably use a specific index, though FindPin starts at 0, so this won't *currently* conflict with user created pins
	return FindPin(FunctionName);
}

UEdGraphPin* UK2Node_Switch::GetSelectionPin() const
{
	//@TODO: Should probably use a specific index, though FindPin starts at 0, so this won't *currently* conflict with user created pins
	return FindPin(SelectionPinName);
}

UEdGraphPin* UK2Node_Switch::GetDefaultPin() const
{
	return (bHasDefaultPin)
		? Pins[0]
		: NULL;
}

FNodeHandlingFunctor* UK2Node_Switch::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_Switch(CompilerContext);
}

FText UK2Node_Switch::GetMenuCategory() const
{
	static FNodeTextCache CachedCategory;
	if (CachedCategory.IsOutOfDate(this))
	{
		// FText::Format() is slow, so we cache this to save on performance
		CachedCategory.SetCachedText(FEditorCategoryUtils::BuildCategoryString(FCommonEditorCategory::FlowControl, LOCTEXT("ActionMenuCategory", "Switch")), this);
	}
	return CachedCategory;
}

FString UK2Node_Switch::GetExportTextForPin(const UEdGraphPin* Pin) const
{
	return Pin->PinName.ToString();
}

FEdGraphPinType UK2Node_Switch::GetInnerCaseType() const
{
	UEdGraphPin* SelectionPin = GetSelectionPin();
	if (ensure(SelectionPin))
	{
		return SelectionPin->PinType;
	}
	return FEdGraphPinType();
}
#undef LOCTEXT_NAMESPACE
