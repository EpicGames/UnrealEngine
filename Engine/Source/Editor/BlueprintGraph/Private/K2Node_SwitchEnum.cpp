// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SwitchEnum.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintFieldNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraphSchema_K2.h"
#include "FindInBlueprintManager.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SwitchEnum)

class UBlueprintNodeSpawner;

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_SwitchEnum::UK2Node_SwitchEnum(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bHasDefaultPin = false;
	OrphanedPinSaveMode = ESaveOrphanPinMode::SaveAll;

	FunctionName = TEXT("NotEqual_ByteByte");
	FunctionClass = UKismetMathLibrary::StaticClass();
}

void UK2Node_SwitchEnum::SetEnum(UEnum* InEnum)
{
	Enum = InEnum;

	// regenerate enum name list
	EnumEntries.Empty();
	EnumFriendlyNames.Empty();

	if (Enum)
	{
		PreloadObject(Enum);

		// When on async loading thread, postload happens later on GT unless it's possible
		// to do it right now safely.
		if (IsInGameThread() || Enum->IsPostLoadThreadSafe())
		{
			Enum->ConditionalPostLoad();
		}

		for (int32 EnumIndex = 0; EnumIndex < Enum->NumEnums() - 1; ++EnumIndex)
		{
			bool const bShouldBeHidden = Enum->HasMetaData(TEXT("Hidden"), EnumIndex ) || Enum->HasMetaData(TEXT("Spacer"), EnumIndex );
			if (!bShouldBeHidden)
			{
				FString const EnumValueName = Enum->GetNameStringByIndex(EnumIndex);
				EnumEntries.Add( FName(*EnumValueName) );

				FText EnumFriendlyName = Enum->GetDisplayNameTextByIndex(EnumIndex);
				EnumFriendlyNames.Add( EnumFriendlyName );
			}
		}
	}
}

void UK2Node_SwitchEnum::PreloadRequiredAssets()
{
	PreloadObject(Enum);
	Super::PreloadRequiredAssets();
}

FText UK2Node_SwitchEnum::GetNodeTitle(ENodeTitleType::Type TitleType) const 
{
	if (Enum == nullptr)
	{
		return LOCTEXT("SwitchEnum_BadEnumTitle", "Switch on (bad enum)");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("EnumName"), FText::FromString(Enum->GetName()));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "Switch_Enum", "Switch on {EnumName}"), Args), this);
	}
	return CachedNodeTitle;
}

FText UK2Node_SwitchEnum::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SwitchEnum_ToolTip", "Selects an output that matches the input value");
}

void UK2Node_SwitchEnum::AddPinSearchMetaDataInfo(const UEdGraphPin* Pin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(Pin, OutTaggedMetaData);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if (Enum != nullptr && K2Schema->IsExecPin(*Pin) && Pin->Direction == EGPD_Output && Enum->IsNative() && EnumEntries.Contains(Pin->GetFName()))
	{
		// Allow native enum switch pins to be searchable by C++ enum name
		OutTaggedMetaData.Add(FSearchTagDataPair(FFindInBlueprintSearchTags::FiB_NativeName, FText::FromString(Pin->GetName())));
	}
}

void UK2Node_SwitchEnum::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	if (!Enum)
	{
		MessageLog.Error(*LOCTEXT("MissingSwitchEnum", "@@ must have a valid enum - check log for 'Failed import' add a redirector or recreate the enum").ToString(), this);
	}
	return Super::ValidateNodeDuringCompilation(MessageLog);
}

bool UK2Node_SwitchEnum::IsConnectionDisallowed(const UEdGraphPin* MyPin, const UEdGraphPin* OtherPin, FString& OutReason) const
{
	const UEnum* SubCategoryObject = Cast<UEnum>( OtherPin->PinType.PinSubCategoryObject.Get() );
	if( SubCategoryObject )
	{
		if( SubCategoryObject != Enum )
		{
			return true;
		}
	}
	return false;
}

void UK2Node_SwitchEnum::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	struct GetMenuActions_Utils
	{
		static void SetNodeEnum(UEdGraphNode* NewNode, FFieldVariant /*EnumField*/, TWeakObjectPtr<UEnum> NonConstEnumPtr)
		{
			UK2Node_SwitchEnum* EnumNode = CastChecked<UK2Node_SwitchEnum>(NewNode);
			EnumNode->Enum = NonConstEnumPtr.Get();
		}
	};

	UClass* NodeClass = GetClass();
	ActionRegistrar.RegisterEnumActions( FBlueprintActionDatabaseRegistrar::FMakeEnumSpawnerDelegate::CreateLambda([NodeClass](const UEnum* InEnum)->UBlueprintNodeSpawner*
	{
		UBlueprintFieldNodeSpawner* NodeSpawner = UBlueprintFieldNodeSpawner::Create(NodeClass, const_cast<UEnum*>(InEnum));
		check(NodeSpawner != nullptr);
		TWeakObjectPtr<UEnum> NonConstEnumPtr = MakeWeakObjectPtr(const_cast<UEnum*>(InEnum));
		NodeSpawner->SetNodeFieldDelegate = UBlueprintFieldNodeSpawner::FSetNodeFieldDelegate::CreateStatic(GetMenuActions_Utils::SetNodeEnum, NonConstEnumPtr);

		return NodeSpawner;
	}) );
}

void UK2Node_SwitchEnum::CreateSelectionPin()
{
	UEdGraphPin* Pin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Byte, Enum, TEXT("Selection"));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

FEdGraphPinType UK2Node_SwitchEnum::GetPinType() const
{ 
	FEdGraphPinType EnumPinType;
	EnumPinType.PinCategory = UEdGraphSchema_K2::PC_Byte;
	EnumPinType.PinSubCategoryObject = Enum;
	return EnumPinType; 
}

void UK2Node_SwitchEnum::CreateCasePins()
{
	if (Enum)
	{
		SetEnum(Enum);
	}

	const bool bShouldUseAdvancedView = (EnumEntries.Num() > 5);
	if(bShouldUseAdvancedView && (ENodeAdvancedPins::NoPins == AdvancedPinDisplay))
	{
		AdvancedPinDisplay = ENodeAdvancedPins::Hidden;
	}

	for (int32 Index = 0; Index < EnumEntries.Num(); ++Index)
	{
		UEdGraphPin* NewPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, EnumEntries[Index]);
		if (EnumFriendlyNames.IsValidIndex(Index))
		{
			NewPin->PinFriendlyName = EnumFriendlyNames[Index];
		}
		
		if(bShouldUseAdvancedView && (Index > 2))
		{
			NewPin->bAdvancedView = true;
		}
	}
}

UK2Node::ERedirectType UK2Node_SwitchEnum::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const
{
	UK2Node::ERedirectType ReturnValue = Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	if (ReturnValue == UK2Node::ERedirectType_None && Enum && OldPinIndex > 2 && NewPinIndex > 2)
	{
		const int64 OldValue = Enum->GetValueByName(OldPin->PinName);
		const int64 NewValue = Enum->GetValueByName(NewPin->PinName);
		// This handles redirects properly
		if (OldValue == NewValue && OldValue != INDEX_NONE)
		{
			ReturnValue = UK2Node::ERedirectType_Name;
		}
	}
	return ReturnValue;
}

void UK2Node_SwitchEnum::AddPinToSwitchNode()
{
	// first try to restore unconnected pin, since connected one is always visible
	for(auto PinIt = Pins.CreateIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* Pin = *PinIt;
		if(Pin && (0 == Pin->LinkedTo.Num()) && Pin->bAdvancedView)
		{
			Pin->Modify();
			Pin->bAdvancedView = false;
			return;
		}
	}

	for(auto PinIt = Pins.CreateIterator(); PinIt; ++PinIt)
	{
		UEdGraphPin* Pin = *PinIt;
		if(Pin && Pin->bAdvancedView)
		{
			Pin->Modify();
			Pin->bAdvancedView = false;
			return;
		}
	}
}

void UK2Node_SwitchEnum::RemovePinFromSwitchNode(UEdGraphPin* Pin)
{
	if(Pin)
	{
		if(!Pin->bAdvancedView)
		{
			Pin->Modify();
			Pin->bAdvancedView = true;
		}
		Pin->BreakAllPinLinks();
	}
}

void UK2Node_SwitchEnum::ReloadEnum(class UEnum* InEnum)
{
	SetEnum(InEnum);
}

#undef LOCTEXT_NAMESPACE
