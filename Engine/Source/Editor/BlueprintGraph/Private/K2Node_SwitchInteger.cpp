// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SwitchInteger.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetMathLibrary.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SwitchInteger)

UK2Node_SwitchInteger::UK2Node_SwitchInteger(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	StartIndex = 0;

	FunctionName = TEXT("NotEqual_IntInt");
	FunctionClass = UKismetMathLibrary::StaticClass();
}

FText UK2Node_SwitchInteger::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "Switch_Interger", "Switch on Int");
}

FText UK2Node_SwitchInteger::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SwitchInt_ToolTip", "Selects an output that matches the input value");
}

void UK2Node_SwitchInteger::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_SwitchInteger::CreateSelectionPin()
{
	UEdGraphPin* Pin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Int, TEXT("Selection"));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

FEdGraphPinType UK2Node_SwitchInteger::GetPinType() const 
{ 
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_Int;
	return PinType;
}

FName UK2Node_SwitchInteger::GetPinNameGivenIndex(int32 Index) const
{
	return *FString::Printf(TEXT("%d"), Index);
}

FName UK2Node_SwitchInteger::GetUniquePinName()
{
	FName NewPinName;
	int32 i = StartIndex;
	while (true)
	{
		NewPinName = GetPinNameGivenIndex(i++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}
	return NewPinName;
}

void UK2Node_SwitchInteger::CreateCasePins()
{
	// Do nothing. By default nothing to do
	// and during realloc we will add new pins
}

void UK2Node_SwitchInteger::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	int32 ExecOutPinCount = StartIndex;

	UEdGraphPin* DefaultPin = GetDefaultPin();

	for (int32 i = bHasDefaultPin ? 1 : 0; i < OldPins.Num(); ++i)
	{
		UEdGraphPin* TestPin = OldPins[i];
		if (UEdGraphSchema_K2::IsExecPin(*TestPin) && (TestPin->Direction == EGPD_Output))
		{
			// Skip the default pin to avoid creating an extra output pin in the case where the default pin has been toggled off
			if (TestPin->PinName != TEXT("Default"))
			{
				const FName NewPinName = GetPinNameGivenIndex(ExecOutPinCount);
				ExecOutPinCount++;

				// Make sure the old pin and new pin names match
				TestPin->PinName = NewPinName;

				// Create the new output pin to match
				CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, NewPinName);
			}
		}
	}
}

bool UK2Node_SwitchInteger::CanRemoveExecutionPin(UEdGraphPin* TargetPin) const
{
	if (!Super::CanRemoveExecutionPin(TargetPin))
	{
		return false;
	}

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	int32 PinIndex = (StartIndex >= 0) ? StartIndex : 0;
	int32 FoundIndex = INDEX_NONE;

	UEdGraphPin* DefaultPin = GetDefaultPin();

	if (TargetPin == DefaultPin)
	{
		return true;
	}

	for (int32 i = 0; i < Pins.Num(); ++i)
	{
		UEdGraphPin* PotentialPin = Pins[i];
		if (K2Schema->IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output) && (PotentialPin != DefaultPin))
		{
			if (PotentialPin == TargetPin)
			{
				FoundIndex = PinIndex;
			}

			++PinIndex;
		}
	}

	// Only allow removing last pin
	if (FoundIndex != PinIndex - 1)
	{
		return false;
	}
	return true;
}

void UK2Node_SwitchInteger::RemovePin(UEdGraphPin* TargetPin)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	int32 PinIndex = (StartIndex >= 0) ? StartIndex : 0;

	UEdGraphPin* DefaultPin = GetDefaultPin();

	for (int32 i = 0; i < Pins.Num(); ++i)
	{
		UEdGraphPin* PotentialPin = Pins[i];
		if (K2Schema->IsExecPin(*PotentialPin) && (PotentialPin->Direction == EGPD_Output) && (PotentialPin != DefaultPin) && (PotentialPin != TargetPin))
		{
			PotentialPin->PinName = GetPinNameGivenIndex(PinIndex);
			++PinIndex;
		}
	}
}

void UK2Node_SwitchInteger::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == TEXT("StartIndex"))
	{
		ReconstructNode();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
