// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SwitchName.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetMathLibrary.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SwitchName)

UK2Node_SwitchName::UK2Node_SwitchName(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	FunctionName = TEXT("NotEqual_NameName");
	FunctionClass = UKismetMathLibrary::StaticClass();
}

void UK2Node_SwitchName::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bIsDirty = false;
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == TEXT("PinNames"))
	{
		bIsDirty = true;
	}

	if (bIsDirty)
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

FText UK2Node_SwitchName::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "Switch_Name", "Switch on Name");
}

FText UK2Node_SwitchName::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SwitchName_ToolTip", "Selects an output that matches the input value");
}

void UK2Node_SwitchName::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_SwitchName::CreateSelectionPin()
{
	UEdGraphPin* Pin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Name, TEXT("Selection"));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

FEdGraphPinType UK2Node_SwitchName::GetPinType() const 
{ 
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_Name;
	return PinType;
}

FName UK2Node_SwitchName::GetPinNameGivenIndex(int32 Index) const
{
	check(Index);
	return PinNames[Index];
}

void UK2Node_SwitchName::CreateCasePins()
{
	for (const FName& PinName : PinNames)
	{
		CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);
	}
}

FName UK2Node_SwitchName::GetUniquePinName()
{
	FName NewPinName;
	int32 Index = 0;
	while (true)
	{
		NewPinName =  *FString::Printf(TEXT("Case_%d"), Index++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}
	return NewPinName;
}

void UK2Node_SwitchName::AddPinToSwitchNode()
{
	const FName PinName = GetUniquePinName();
	PinNames.Add(PinName);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);
}

void UK2Node_SwitchName::RemovePin(UEdGraphPin* TargetPin)
{
	checkSlow(TargetPin);

	// Clean-up pin name array
	PinNames.Remove(TargetPin->PinName);
}
