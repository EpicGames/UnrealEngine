// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_SwitchString.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraphSchema_K2.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet/KismetStringLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_SwitchString)

UK2Node_SwitchString::UK2Node_SwitchString(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	bIsCaseSensitive = false;
	SetupCaseSensitivityFunction();
	FunctionClass = UKismetStringLibrary::StaticClass();
}

void UK2Node_SwitchString::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bIsDirty = false;
	FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if (PropertyName == TEXT("PinNames"))
	{
		bIsDirty = true;
	}
	else if (PropertyName == TEXT("bIsCaseSensitive"))
	{
		SetupCaseSensitivityFunction();

		FunctionClass = UKismetStringLibrary::StaticClass();
		bIsDirty = true;
	}
	 
	if (bIsDirty)
	{
		ReconstructNode();
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
	GetGraph()->NotifyNodeChanged(this);
}

void UK2Node_SwitchString::SetupCaseSensitivityFunction()
{
	FunctionName = (bIsCaseSensitive == true)
		?  TEXT("NotEqual_StrStr")
		: TEXT("NotEqual_StriStri");
}

FText UK2Node_SwitchString::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NSLOCTEXT("K2Node", "Switch_String", "Switch on String");
}

FText UK2Node_SwitchString::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SwitchString_ToolTip", "Selects an output that matches the input value");
}

void UK2Node_SwitchString::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

void UK2Node_SwitchString::CreateSelectionPin()
{
	UEdGraphPin* Pin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_String, TEXT("Selection"));
	GetDefault<UEdGraphSchema_K2>()->SetPinAutogeneratedDefaultValueBasedOnType(Pin);
}

FEdGraphPinType UK2Node_SwitchString::GetPinType() const 
{ 
	FEdGraphPinType PinType;
	PinType.PinCategory = UEdGraphSchema_K2::PC_String;
	return PinType;
}

FName UK2Node_SwitchString::GetPinNameGivenIndex(int32 Index) const
{
	check(Index);
	return PinNames[Index];
}

void UK2Node_SwitchString::CreateCasePins()
{
	for( const FName& PinName : PinNames)
	{
		UEdGraphPin* Pin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);
		Pin->bAllowFriendlyName = false;
	}
}

FName UK2Node_SwitchString::GetUniquePinName()
{
	FName NewPinName;
	int32 Index = 0;
	while (true)
	{
		NewPinName = *FString::Printf(TEXT("Case_%d"), Index++);
		if (!FindPin(NewPinName))
		{
			break;
		}
	}
	return NewPinName;
}

void UK2Node_SwitchString::AddPinToSwitchNode()
{
	const FName PinName = GetUniquePinName();
	PinNames.Add(PinName);

	UEdGraphPin* Pin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, PinName);
	Pin->bAllowFriendlyName = false;
}

void UK2Node_SwitchString::RemovePin(UEdGraphPin* TargetPin)
{
	checkSlow(TargetPin);

	// Clean-up pin name array
	PinNames.Remove(TargetPin->PinName);
}

void UK2Node_SwitchString::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
	
	TArray<FString> UniquePinNames;
	UniquePinNames.Reserve(PinNames.Num());
	ESearchCase::Type CaseSensitivity = bIsCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase;

	for (FName Pin : PinNames)
	{
		FString PinStr = Pin.ToString();

		for (const FString& UniquePin : UniquePinNames)
		{
			if (PinStr.Equals(UniquePin, CaseSensitivity))
			{
				MessageLog.Error(*NSLOCTEXT("K2Node", "SwitchString_DuplicateCases", "@@ contains duplicate cases.").ToString(), this);
				return;
			}
		}
		UniquePinNames.Emplace(MoveTemp(PinStr));
	}
}
