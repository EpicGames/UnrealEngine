// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_TemporaryVariable.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "EditorCategoryUtils.h"
#include "Internationalization/Internationalization.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_TemporaryVariable)

class FKismetCompilerContext;

class FKCHandler_TemporaryVariable : public FNodeHandlingFunctor
{
public:
	FKCHandler_TemporaryVariable(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override
	{
		// This net is an anonymous temporary variable
		FBPTerminal* Term = Context.CreateLocalTerminal(Context.IsEventGraph() ? ETerminalSpecification::TS_ForcedShared : ETerminalSpecification::TS_Unspecified);

		FString NetName = Context.NetNameMap->MakeValidName(Net);

		Term->CopyFromPin(Net, NetName);

		UK2Node_TemporaryVariable* TempVarNode = CastChecked<UK2Node_TemporaryVariable>(Net->GetOwningNode());
		Term->bIsSavePersistent = TempVarNode->bIsPersistent;

		Context.NetMap.Add(Net, Term);
	}
};

UK2Node_TemporaryVariable::UK2Node_TemporaryVariable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsPersistent(false)
{
}

void UK2Node_TemporaryVariable::AllocateDefaultPins()
{
	UEdGraphPin* VariablePin = CreatePin(EGPD_Output, NAME_None, TEXT("Variable"));
	VariablePin->PinType = VariableType;

	Super::AllocateDefaultPins();
}

FText UK2Node_TemporaryVariable::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableType"), UEdGraphSchema_K2::TypeToText(VariableType));

		FText TitleFormat;
		if (bIsPersistent)
		{
			TitleFormat = NSLOCTEXT("K2Node", "PersistentLocalVariable_Tooltip", "Local temporary {VariableType} variable, which is serialized in save games");
		}
		else
		{
			TitleFormat = NSLOCTEXT("K2Node", "LocalTemporaryVariable_Tooltip", "Local temporary {VariableType} variable");
		}

		// FText::Format() is slow, so we cache this to save on performance
		CachedTooltip.SetCachedText(FText::Format(TitleFormat, Args), this);
	}
	return CachedTooltip;
}

FText UK2Node_TemporaryVariable::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VariableType"), UEdGraphSchema_K2::TypeToText(VariableType));

		FText TitleFormat = !bIsPersistent ? NSLOCTEXT("K2Node", "LocalTemporaryVariable_Title", "Local {VariableType}") : NSLOCTEXT("K2Node", "PersistentLocalVariable", "Persistent Local {VariableType}");
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(TitleFormat, Args), this);
	}
	
	return CachedNodeTitle;
}

bool UK2Node_TemporaryVariable::IsNodePure() const
{
	return true;
}

void UK2Node_TemporaryVariable::PostReconstructNode()
{
	UEdGraphPin* VariablePin = GetVariablePin();
	PinConnectionListChanged(VariablePin);

	return Super::PostReconstructNode();
}

void UK2Node_TemporaryVariable::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	UEdGraphPin* VariablePin = GetVariablePin();
	if (VariableType.PinCategory == UEdGraphSchema_K2::PC_Wildcard && Pin && Pin == VariablePin && Pin->LinkedTo.Num() > 0)
	{
		VariablePin->PinType = Pin->LinkedTo[0]->PinType;
	}
	else
	{
		VariablePin->PinType = VariableType;
	}
	return Super::NotifyPinConnectionListChanged(Pin);
}

FString UK2Node_TemporaryVariable::GetDescriptiveCompiledName() const
{
	FString Result = FString(TEXT("Temp_")) + VariableType.PinCategory.ToString();
		
	if (!NodeComment.IsEmpty())
	{
		Result += TEXT("_");
		Result += NodeComment;
	}

	// If this node is persistent, we need to add the NodeGuid, which should be propagated from the macro that created this, in order to ensure persistence 
	if (bIsPersistent)
	{
		Result += TEXT("_");
		Result += NodeGuid.ToString();
	}

	return Result;
}

bool UK2Node_TemporaryVariable::IsCompatibleWithGraph(UEdGraph const* TargetGraph) const
{
	bool bIsCompatible = Super::IsCompatibleWithGraph(TargetGraph);
	if (bIsCompatible)
	{
		bIsCompatible = false;

		EGraphType const GraphType = TargetGraph->GetSchema()->GetGraphType(TargetGraph);
		if (GraphType == GT_Macro)
		{
			bIsCompatible = !bIsPersistent;
		}
	}

	return bIsCompatible;
}

bool UK2Node_TemporaryVariable::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// These nodes can be pasted anywhere that UK2Node's are compatible with the graph
	// Avoiding the call to IsCompatibleWithGraph because these nodes should normally only
	// be placed in Macros, but it's nice to be able to paste Macro functionality anywhere.
	return Super::IsCompatibleWithGraph(TargetGraph);
}

// get variable pin
UEdGraphPin* UK2Node_TemporaryVariable::GetVariablePin()
{
	return FindPin(TEXT("Variable"));
}

FNodeHandlingFunctor* UK2Node_TemporaryVariable::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_TemporaryVariable(CompilerContext);
}

void UK2Node_TemporaryVariable::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (!ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		return;
	}

	auto MakeTempVarNodeSpawner = [](FEdGraphPinType const& VarType, bool bVarIsPersistent)
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(UK2Node_TemporaryVariable::StaticClass());
		check(NodeSpawner != nullptr);

		auto PostSpawnLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode, FEdGraphPinType InVarType, bool bInIsPersistent)
		{
			UK2Node_TemporaryVariable* TempVarNode = CastChecked<UK2Node_TemporaryVariable>(NewNode);
			TempVarNode->VariableType  = InVarType;
			TempVarNode->bIsPersistent = bInIsPersistent;
		};

		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(PostSpawnLambda, VarType, bVarIsPersistent);
		return NodeSpawner;
	};

	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Int, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Int, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Int64, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Int64, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Double, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_String, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_String, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Text, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Text, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Wildcard, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Wildcard, NAME_None, nullptr, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));

	UScriptStruct* VectorStruct  = TBaseStructure<FVector>::Get();
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Vector"), VectorStruct, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Vector"), VectorStruct, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	
	UScriptStruct* RotatorStruct = TBaseStructure<FRotator>::Get();
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Rotator"), RotatorStruct, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Rotator"), RotatorStruct, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	
	UScriptStruct* TransformStruct = TBaseStructure<FTransform>::Get();
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Transform"), TransformStruct, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("Transform"), TransformStruct, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));

	UScriptStruct* BlendSampleStruct = FindObjectChecked<UScriptStruct>(nullptr, TEXT("/Script/Engine.BlendSampleData"));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("BlendSampleData"), BlendSampleStruct, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Struct, TEXT("BlendSampleData"), BlendSampleStruct, EPinContainerType::Array, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/false));

	// add persistent bool and int types (for macro graphs)
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Int, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/true));
	ActionRegistrar.AddBlueprintAction(ActionKey, MakeTempVarNodeSpawner(FEdGraphPinType(UEdGraphSchema_K2::PC_Boolean, NAME_None, nullptr, EPinContainerType::None, /*bIsReference =*/ false, /*InValueTerminalType=*/FEdGraphTerminalType()), /*bIsPersistent =*/true));
}

FText UK2Node_TemporaryVariable::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Macro);
}

FBlueprintNodeSignature UK2Node_TemporaryVariable::GetSignature() const
{
	FBlueprintNodeSignature NodeSignature = Super::GetSignature();

	FString TypeString;
	if (bIsPersistent)
	{
		TypeString = TEXT("Persistent ");
	}
	TypeString += UEdGraphSchema_K2::TypeToText(VariableType).ToString();

	static const FName VarTypeSignatureKey(TEXT("VarType"));
	NodeSignature.AddNamedValue(VarTypeSignatureKey, TypeString);

	return NodeSignature;
}
