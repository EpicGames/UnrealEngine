// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_Timeline.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Components/TimelineComponent.h"
#include "Containers/EnumAsByte.h"
#include "Curves/CurveFloat.h"
#include "Curves/CurveLinearColor.h"
#include "Curves/CurveVector.h"
#include "Delegates/Delegate.h"
#include "DiffResults.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Engine/TimelineTemplate.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Composite.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Timeline)

#define LOCTEXT_NAMESPACE "K2Node_Timeline"

/////////////////////////////////////////////////////
// FKCHandler_Timeline
class FKCHandler_Timeline : public FNodeHandlingFunctor
{
public:
	FKCHandler_Timeline(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Timeline* TimelineNode = CastChecked<UK2Node_Timeline>(Node);
		UEdGraphPin* NewTimePin = TimelineNode->GetNewTimePin();
		UEdGraphPin* SetNewTimePin = TimelineNode->GetSetNewTimePin();

		if (NewTimePin && SetNewTimePin)
		{
			// If SetNewTimePin is linked, then FKismetCompilerContext::ExpandTimelineNodes will create a function
			// that handles the casting of the "NewTime" parameter for UTimelineComponent::SetNewTime.
			// If SetNewTimePin is *not* linked, then we don't use "NewTime" at all, and can throw away the cast entry.

			const bool bSetNewTimePinConnected = (SetNewTimePin->LinkedTo.Num() > 0);
			if (!bSetNewTimePinConnected)
			{
				using namespace UE::KismetCompiler;
				CastingUtils::RemoveRegisteredImplicitCast(Context, NewTimePin);
			}
		}
		else
		{
			Context.MessageLog.Error(*LOCTEXT("Error_InvalidNewTimePin", "ICE: missing one or more 'New Time'-related pins on node @@").ToString(), Node);
		}
	}
};

/////////////////////////////////////////////////////
// UK2Node_Timeline

UK2Node_Timeline::UK2Node_Timeline(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoPlay = false;
	bLoop = false;
	bReplicated = false;
	bIgnoreTimeDilation = false;
}

static FName PlayPinName(TEXT("Play"));
static FName PlayFromStartPinName(TEXT("PlayFromStart"));
static FName StopPinName(TEXT("Stop"));
static FName UpdatePinName(TEXT("Update"));
static FName ReversePinName(TEXT("Reverse"));
static FName ReverseFromEndPinName(TEXT("ReverseFromEnd"));
static FName FinishedPinName(TEXT("Finished"));
static FName NewTimePinName(TEXT("NewTime"));
static FName SetNewTimePinName(TEXT("SetNewTime"));
static FName DirectionPinName(TEXT("Direction"));

namespace 
{
	UEdGraphPin* GetPin(const UK2Node_Timeline* Timeline, const FName PinName, EEdGraphPinDirection DesiredDirection) 
	{
		UEdGraphPin* Pin = Timeline->FindPin(PinName);
		check(Pin);
		check(Pin->Direction == DesiredDirection);
		return Pin;
	}
}

void UK2Node_Timeline::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	bCanRenameNode = 1;

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, PlayPinName);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, PlayFromStartPinName);
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, StopPinName);
	CreatePin(EGPD_Input,  UEdGraphSchema_K2::PC_Exec, ReversePinName);
	CreatePin(EGPD_Input,  UEdGraphSchema_K2::PC_Exec, ReverseFromEndPinName);

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UpdatePinName);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, FinishedPinName);

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, SetNewTimePinName);

	UEdGraphPin* NewPositionPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Float, NewTimePinName);
	K2Schema->SetPinAutogeneratedDefaultValue(NewPositionPin, TEXT("0.0"));

	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Byte, FTimeline::GetTimelineDirectionEnum(), DirectionPinName);

	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);

	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	if(Timeline)
	{
		// Ensure the timeline template is fully loaded or the node representation will be wrong.
		PreloadObject(Timeline);

		for (int32 OutputDisplayIndex = 0; OutputDisplayIndex < Timeline->GetNumDisplayTracks(); ++OutputDisplayIndex)
		{
			FTTTrackId TrackId = Timeline->GetDisplayTrackId(OutputDisplayIndex);
			if (TrackId.TrackType == FTTTrackBase::TT_Event)
			{
				const FTTEventTrack& EventTrack = Timeline->EventTracks[TrackId.TrackIndex];
				CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, EventTrack.GetTrackName());
			}
			else if (TrackId.TrackType == FTTTrackBase::TT_FloatInterp)
			{
				const FTTFloatTrack& FloatTrack = Timeline->FloatTracks[TrackId.TrackIndex];
				CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Float, FloatTrack.GetTrackName());
			}
			else if (TrackId.TrackType == FTTTrackBase::TT_VectorInterp)
			{
				UScriptStruct* VectorStruct = TBaseStructure<FVector>::Get();
				const FTTVectorTrack& VectorTrack = Timeline->VectorTracks[TrackId.TrackIndex];
				CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, VectorStruct, VectorTrack.GetTrackName());
			}
			else if (TrackId.TrackType == FTTTrackBase::TT_LinearColorInterp)
			{
				UScriptStruct* LinearColorStruct = TBaseStructure<FLinearColor>::Get();
				const FTTLinearColorTrack& LinearColorTrack = Timeline->LinearColorTracks[TrackId.TrackIndex];
				CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Struct, LinearColorStruct, LinearColorTrack.GetTrackName());
			}
		}

		// cache play status
		bAutoPlay = Timeline->bAutoPlay;
		bLoop = Timeline->bLoop;
		bReplicated = Timeline->bReplicated;
		bIgnoreTimeDilation = Timeline->bIgnoreTimeDilation;
	}

	Super::AllocateDefaultPins();
}

void UK2Node_Timeline::PreloadRequiredAssets()
{
	UBlueprint* Blueprint = GetBlueprint();
	if(ensure(Blueprint))
	{
		UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
		if(Timeline)
		{
			// Ensure the timeline template is fully loaded or the node representation will be wrong.
			PreloadObject(Timeline);
		}
	}
	
	Super::PreloadRequiredAssets();
}

void UK2Node_Timeline::DestroyNode()
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);
	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	if(Timeline)
	{
		FBlueprintEditorUtils::RemoveTimeline(Blueprint, Timeline, true);

		// Move template object out of the way so that we can potentially create a timeline with the same name either through a paste or a new timeline action
		Timeline->Rename(NULL, GetTransientPackage(), REN_None);
	}

	Super::DestroyNode();
}

void UK2Node_Timeline::PostPasteNode()
{
	Super::PostPasteNode();

	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);

	UTimelineTemplate* OldTimeline = NULL;

	//find the template with same UUID
	for(TObjectIterator<UTimelineTemplate> It;It;++It)
	{
		UTimelineTemplate* Template = *It;
		if(Template->TimelineGuid == TimelineGuid)
		{
			OldTimeline = Template;
			break;
		}
	}

	// Make sure TimelineName is unique, and we allocate a new timeline template object for this node
	TimelineName = FBlueprintEditorUtils::FindUniqueTimelineName(Blueprint);

	if(!OldTimeline)
	{
		if (UTimelineTemplate* Template = FBlueprintEditorUtils::AddNewTimeline(Blueprint, TimelineName))
		{
			bAutoPlay = Template->bAutoPlay;
			bLoop = Template->bLoop;
			bReplicated = Template->bReplicated;
			bIgnoreTimeDilation = Template->bIgnoreTimeDilation;
		}
	}
	else
	{
		check(NULL != Blueprint->GeneratedClass);
		Blueprint->Modify();
		const FName TimelineTemplateName = *UTimelineTemplate::TimelineVariableNameToTemplateName(TimelineName);
		UTimelineTemplate* Template = DuplicateObject<UTimelineTemplate>(OldTimeline, Blueprint->GeneratedClass, TimelineTemplateName);
		bAutoPlay = Template->bAutoPlay;
		bLoop = Template->bLoop;
		bReplicated = Template->bReplicated;
		bIgnoreTimeDilation = Template->bIgnoreTimeDilation;
		Template->SetFlags(RF_Transactional);
		Blueprint->Timelines.Add(Template);

		// Fix up timeline tracks to point to the proper location.  When duplicated, they're still parented to their old blueprints because we don't have the appropriate scope.  Note that we never want to fix up external curve asset references
		{
			for( auto TrackIt = Template->FloatTracks.CreateIterator(); TrackIt; ++TrackIt )
			{
				FTTFloatTrack& Track = *TrackIt;
				if (!Track.bIsExternalCurve && Track.CurveFloat && Track.CurveFloat->GetOuter()->IsA(UBlueprint::StaticClass()))
				{
					Track.CurveFloat->Rename(*Template->MakeUniqueCurveName(Track.CurveFloat, Track.CurveFloat->GetOuter()), Blueprint, REN_DontCreateRedirectors);
				}
			}

			for( auto TrackIt = Template->EventTracks.CreateIterator(); TrackIt; ++TrackIt )
			{
				FTTEventTrack& Track = *TrackIt;
				if (!Track.bIsExternalCurve && Track.CurveKeys && Track.CurveKeys->GetOuter()->IsA(UBlueprint::StaticClass()))
				{
					Track.CurveKeys->Rename(*Template->MakeUniqueCurveName(Track.CurveKeys, Track.CurveKeys->GetOuter()), Blueprint, REN_DontCreateRedirectors);
				}
			}

			for( auto TrackIt = Template->VectorTracks.CreateIterator(); TrackIt; ++TrackIt )
			{
				FTTVectorTrack& Track = *TrackIt;
				if (!Track.bIsExternalCurve && Track.CurveVector && Track.CurveVector->GetOuter()->IsA(UBlueprint::StaticClass()))
				{
					Track.CurveVector->Rename(*Template->MakeUniqueCurveName(Track.CurveVector, Track.CurveVector->GetOuter()), Blueprint, REN_DontCreateRedirectors);
				}
			}

			for( auto TrackIt = Template->LinearColorTracks.CreateIterator(); TrackIt; ++TrackIt )
			{
				FTTLinearColorTrack& Track = *TrackIt;
				if (!Track.bIsExternalCurve && Track.CurveLinearColor && Track.CurveLinearColor->GetOuter()->IsA(UBlueprint::StaticClass()))
				{
					Track.CurveLinearColor->Rename(*Template->MakeUniqueCurveName(Track.CurveLinearColor, Track.CurveLinearColor->GetOuter()), Blueprint, REN_DontCreateRedirectors);
				}
			}
		}

		FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, TimelineName);
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
}

bool UK2Node_Timeline::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	if(Super::IsCompatibleWithGraph(TargetGraph))
	{
		UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
		if(Blueprint)
		{
			const UEdGraphSchema_K2* K2Schema = Cast<UEdGraphSchema_K2>(TargetGraph->GetSchema());
			check(K2Schema);

			const bool bSupportsEventGraphs = FBlueprintEditorUtils::DoesSupportEventGraphs(Blueprint);
			const bool bAllowEvents = (K2Schema->GetGraphType(TargetGraph) == GT_Ubergraph) && bSupportsEventGraphs &&
				(Blueprint->BlueprintType != BPTYPE_MacroLibrary);

			if(bAllowEvents)
			{
				return FBlueprintEditorUtils::DoesSupportTimelines(Blueprint);
			}
			else
			{
				bool bCompositeOfUbberGraph = false;

				//If the composite has a ubergraph in its outer, it is allowed to have timelines
				if (bSupportsEventGraphs && K2Schema->IsCompositeGraph(TargetGraph))
				{
					while (TargetGraph)
					{
						if (UK2Node_Composite* Composite = Cast<UK2Node_Composite>(TargetGraph->GetOuter()))
						{
							TargetGraph = Cast<UEdGraph>(Composite->GetOuter());
						}
						else if (K2Schema->GetGraphType(TargetGraph) == GT_Ubergraph)
						{
							bCompositeOfUbberGraph = true;
							break;
						}
						else
						{
							TargetGraph = Cast<UEdGraph>(TargetGraph->GetOuter());
						}
					}
				}
				return bCompositeOfUbberGraph ? FBlueprintEditorUtils::DoesSupportTimelines(Blueprint) : false;
			}
		}
	}

	return false;
}

FLinearColor UK2Node_Timeline::GetNodeTitleColor() const
{
	return FLinearColor(1.0f, 0.51f, 0.0f);
}

FText UK2Node_Timeline::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText Title = FText::FromName(TimelineName);
	
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint != nullptr);
	
	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	// if a node hasn't been spawned for this node yet, then lets title it
	// after what it will do (the name would be invalid anyways)
	if (Timeline == nullptr)
	{
		// if this node hasn't spawned a
		Title = LOCTEXT("NoTimelineTitle", "Add Timeline...");
	}
	return Title;
}

UEdGraphPin* UK2Node_Timeline::GetDirectionPin() const
{
	UEdGraphPin* Pin = FindPin(DirectionPinName);
	if (Pin)
	{
		const bool bIsOutput = (EGPD_Output == Pin->Direction);
		const bool bProperType = (UEdGraphSchema_K2::PC_Byte == Pin->PinType.PinCategory);
		const bool bProperSubCategoryObj = (Pin->PinType.PinSubCategoryObject == FTimeline::GetTimelineDirectionEnum());
		if(bIsOutput && bProperType && bProperSubCategoryObj)
		{
			return Pin;
		}
	}
	return NULL;
}

UEdGraphPin* UK2Node_Timeline::GetPlayPin() const
{
	return GetPin(this, PlayPinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetPlayFromStartPin() const
{
	return GetPin(this, PlayFromStartPinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetStopPin() const
{
	return GetPin(this, StopPinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetReversePin() const
{
	return GetPin(this, ReversePinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetReverseFromEndPin() const
{
	return GetPin(this, ReverseFromEndPinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetUpdatePin() const
{
	return GetPin(this, UpdatePinName, EGPD_Output);
}

UEdGraphPin* UK2Node_Timeline::GetFinishedPin() const
{
	return GetPin(this, FinishedPinName, EGPD_Output);
}

UEdGraphPin* UK2Node_Timeline::GetNewTimePin() const
{
	return GetPin(this, NewTimePinName, EGPD_Input);
}

UEdGraphPin* UK2Node_Timeline::GetSetNewTimePin() const
{
	return GetPin(this, SetNewTimePinName, EGPD_Input);
}

bool UK2Node_Timeline::RenameTimeline(const FString& NewName)
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);

	FName NewTimelineName(*NewName);
	if (FBlueprintEditorUtils::RenameTimeline(Blueprint, TimelineName, NewTimelineName))
	{
		// Clear off any existing error message now the timeline has been renamed
		this->ErrorMsg.Empty();
		this->bHasCompilerMessage = false;

		return true;
	}
	return false;
}

void UK2Node_Timeline::PrepareForCopying() 
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);
	//Set the GUID so we can identify which timeline template the copied node should use
	UTimelineTemplate* Template  = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	check(Template);
	TimelineGuid = Template->TimelineGuid; // hold onto the template's Guid so on paste we can match it up on paste
}

//Determine if all the tracks contained with both arrays are identical
template<class T>
void FindExactTimelineDifference(struct FDiffResults& Results, FDiffSingleResult Result, const TArray<T>& Tracks1, const TArray<T>& Tracks2, FString TrackTypeStr)
{
	if(Tracks1.Num() != Tracks2.Num())
	{
		FText NodeName = Result.Node1->GetNodeTitle(ENodeTitleType::ListView);

		FFormatNamedArguments Args;
		Args.Add(TEXT("TrackType"), FText::FromString(TrackTypeStr));
		Args.Add(TEXT("NodeName"), NodeName);

		Result.Diff = EDiffType::TIMELINE_NUM_TRACKS;
		Result.ToolTip =  FText::Format(LOCTEXT("DIF_TimelineNumTracksToolTip", "The number of {TrackType} tracks in Timeline '{NodeName}' has changed"), Args);
		Result.Category = EDiffType::MODIFICATION;
		Result.DisplayString = FText::Format(LOCTEXT("DIF_TimelineNumTracks", "{TrackType} Track Count '{NodeName}'"), Args);
		Results.Add(Result);
		return;
	}

	for(int32 i = 0;i<Tracks1.Num();++i)
	{
		if(!(Tracks1[i] == Tracks2[i]))
		{
			FName TrackName = Tracks2[i].GetTrackName();
			FText NodeName = Result.Node1->GetNodeTitle(ENodeTitleType::ListView);

			FFormatNamedArguments Args;
			Args.Add(TEXT("TrackName"), FText::FromName(TrackName));
			Args.Add(TEXT("NodeName"), NodeName);

			Result.Diff = EDiffType::TIMELINE_TRACK_MODIFIED;
			Result.ToolTip =  FText::Format(LOCTEXT("DIF_TimelineTrackModifiedToolTip", "Track '{TrackName}' of Timeline '{NodeName}' was Modified"), Args);
			Result.Category = EDiffType::MODIFICATION;
			Result.DisplayString = FText::Format(LOCTEXT("DIF_TimelineTrackModified", "Track Modified '{TrackName}'"), Args);
			Results.Add(Result);
			break;
		}
	}
}

void UK2Node_Timeline::FindDiffs( class UEdGraphNode* OtherNode, struct FDiffResults& Results )  
{
	UK2Node_Timeline* Timeline1 = this;
	UK2Node_Timeline* Timeline2 = Cast<UK2Node_Timeline>(OtherNode);

	UBlueprint* Blueprint1 = Timeline1->GetBlueprint();
	int32 Index1 = FBlueprintEditorUtils::FindTimelineIndex(Blueprint1,Timeline1->TimelineName);

	UBlueprint* Blueprint2 = Timeline2->GetBlueprint();
	int32 Index2 = FBlueprintEditorUtils::FindTimelineIndex(Blueprint2,Timeline2->TimelineName);
	if(Index1 != INDEX_NONE && Index2 != INDEX_NONE)
	{
		UTimelineTemplate* Template1 = Blueprint1->Timelines[Index1];
		UTimelineTemplate* Template2 = Blueprint2->Timelines[Index2];

		FDiffSingleResult Diff;
		Diff.Node1 = Timeline2;
		Diff.Node2 = Timeline1;

		if(Template1->bAutoPlay != Template2->bAutoPlay)
		{
			Diff.Diff = EDiffType::TIMELINE_AUTOPLAY;
			FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeName"), NodeName);

			Diff.ToolTip =  FText::Format(LOCTEXT("DIF_TimelineAutoPlayToolTip", "Timeline '{NodeName}' had its AutoPlay state changed"), Args);
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_TimelineAutoPlay", "Timeline AutoPlay Changed '{NodeName}'"), Args);
			Results.Add(Diff);
		}
		if(Template1->bLoop != Template2->bLoop)
		{
			Diff.Diff = EDiffType::TIMELINE_LOOP;
			FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeName"), NodeName);

			Diff.ToolTip =  FText::Format(LOCTEXT("DIF_TimelineLoopingToolTip", "Timeline '{NodeName}' had its looping state changed"), Args);
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString =  FText::Format(LOCTEXT("DIF_TimelineLooping", "Timeline Loop Changed '{NodeName}'"), Args);
			Results.Add(Diff);
		}
		if(Template1->TimelineLength != Template2->TimelineLength)
		{
			FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeName"), NodeName);
			Args.Add(TEXT("TimelineLength1"), Template1->TimelineLength);
			Args.Add(TEXT("TimelineLength2"), Template2->TimelineLength);

			Diff.Diff = EDiffType::TIMELINE_LENGTH;
			Diff.ToolTip = FText::Format(LOCTEXT("DIF_TimelineLengthToolTip", "Length of Timeline '{NodeName}' has changed. Was {TimelineLength1}, but is now {TimelineLength2}"), Args);
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString =  FText::Format(LOCTEXT("DIF_TimelineLength", "Timeline Length '{NodeName}' [{TimelineLength1} -> {TimelineLength2}]"), Args);
			Results.Add(Diff);
		}
		if (Template1->bIgnoreTimeDilation != Template2->bIgnoreTimeDilation)
		{
			Diff.Diff = EDiffType::TIMELINE_IGNOREDILATION;
			FText NodeName = GetNodeTitle(ENodeTitleType::ListView);

			FFormatNamedArguments Args;
			Args.Add(TEXT("NodeName"), NodeName);

			Diff.ToolTip = FText::Format(LOCTEXT("DIF_TimelineIgnoreDilationToolTip", "Timeline '{NodeName}' had its ignore time dilation state changed"), Args);
			Diff.Category = EDiffType::MODIFICATION;
			Diff.DisplayString = FText::Format(LOCTEXT("DIF_TimelineIgnoreDilation", "Timeline IgnoreTimeDilation Changed '{NodeName}'"), Args);
			Results.Add(Diff);
		}

		//something specific inside has changed
		if(Diff.Diff == EDiffType::NO_DIFFERENCE)
		{
			FindExactTimelineDifference(Results, Diff, Template1->EventTracks, Template2->EventTracks, LOCTEXT("Event", "Event").ToString());
			FindExactTimelineDifference(Results, Diff, Template1->FloatTracks, Template2->FloatTracks, LOCTEXT("Float", "Float").ToString());
			FindExactTimelineDifference(Results, Diff, Template1->VectorTracks, Template2->VectorTracks,  LOCTEXT("Vector", "Vector").ToString() );
		}
		
	}
}

void UK2Node_Timeline::OnRenameNode(const FString& NewName)
{
	RenameTimeline(NewName);
}

TSharedPtr<class INameValidatorInterface> UK2Node_Timeline::MakeNameValidator() const
{
	return MakeShareable(new FKismetNameValidator(GetBlueprint(), TimelineName));
}

FNodeHandlingFunctor* UK2Node_Timeline::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_Timeline(CompilerContext);
}

void UK2Node_Timeline::ExpandForPin(UEdGraphPin* TimelinePin, const FName PropertyName, FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	if (TimelinePin && TimelinePin->LinkedTo.Num() > 0)
	{
		UK2Node_VariableGet* GetVarNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, SourceGraph);
		GetVarNode->VariableReference.SetSelfMember(PropertyName);
		GetVarNode->AllocateDefaultPins();
		UEdGraphPin* ValuePin = GetVarNode->GetValuePin();
		if (NULL != ValuePin)
		{
			CompilerContext.MovePinLinksToIntermediate(*TimelinePin, *ValuePin);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ExpandForPin_Error", "ExpandForPin error, no property found for @@").ToString(), TimelinePin);
		}
	}
}

void UK2Node_Timeline::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);

	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	if(Timeline)
	{
		ExpandForPin(GetDirectionPin(), Timeline->GetDirectionPropertyName(), CompilerContext, SourceGraph);

		for (const FTTFloatTrack& FloatTrack : Timeline->FloatTracks)
		{
			ExpandForPin(FindPin(FloatTrack.GetTrackName()), FloatTrack.GetPropertyName(), CompilerContext, SourceGraph);
		}

		for (const FTTVectorTrack& VectorTrack : Timeline->VectorTracks)
		{
			ExpandForPin(FindPin(VectorTrack.GetTrackName()), VectorTrack.GetPropertyName(), CompilerContext, SourceGraph);
		}

		for (const FTTLinearColorTrack& LinearColorTrack : Timeline->LinearColorTracks)
		{
			ExpandForPin(FindPin(LinearColorTrack.GetTrackName()), LinearColorTrack.GetPropertyName(), CompilerContext, SourceGraph);
		}
	}
}

FText UK2Node_Timeline::GetTooltipText() const
{
	return LOCTEXT("TimelineTooltip", "Timeline node allows values to be keyframed over time.\nDouble click to open timeline editor.");
}

FText UK2Node_Timeline::GetToolTipHeading() const
{
	const FText ExistingHeading = Super::GetToolTipHeading();
	
	if (bReplicated)
	{
		const FText ReplicatedLabel = LOCTEXT("ReplicatedTimelineComponent", "Replicated");

		// If the existing heading is empty then we don't need anything other then "Replicated"
		if (ExistingHeading.IsEmpty())
		{
			return ReplicatedLabel;
		}
		// otherwise, we need an extra newline character ( '\n' ) to get everything properly formatted 
		else
		{
			return FText::Format(LOCTEXT("UK2Node_Timeline_GetToolTipHeading", "{0}\n{1}"), {ReplicatedLabel, ExistingHeading });	
		}
	}

	return ExistingHeading;
}

FName UK2Node_Timeline::GetCornerIcon() const
{
	if (bReplicated)
	{
		return TEXT("Graph.Replication.Replicated");
	}
	return Super::GetCornerIcon();
}

FSlateIcon UK2Node_Timeline::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.Timeline_16x");
	return Icon;
}

UObject* UK2Node_Timeline::GetJumpTargetForDoubleClick() const
{
	UBlueprint* Blueprint = GetBlueprint();
	check(Blueprint);
	UTimelineTemplate* Timeline = Blueprint->FindTimelineTemplateByVariableName(TimelineName);
	return Timeline;
}

FString UK2Node_Timeline::GetDocumentationExcerptName() const
{
	return TEXT("UK2Node_Timeline");
}

void UK2Node_Timeline::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "TimeLine" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), GetName() ));
}

void UK2Node_Timeline::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		auto CustomizeTimelineNodeLambda = [](UEdGraphNode* NewNode, bool bIsTemplateNode)
		{
			UK2Node_Timeline* TimelineNode = CastChecked<UK2Node_Timeline>(NewNode);

			UBlueprint* Blueprint = TimelineNode->GetBlueprint();
			if (Blueprint != nullptr)
			{
				TimelineNode->TimelineName = FBlueprintEditorUtils::FindUniqueTimelineName(Blueprint);
				if (!bIsTemplateNode && FBlueprintEditorUtils::AddNewTimeline(Blueprint, TimelineNode->TimelineName))
				{
					// clear off any existing error message now that the timeline has been added
					TimelineNode->ErrorMsg.Empty();
					TimelineNode->bHasCompilerMessage = false;
				}
			}
		};

		NodeSpawner->CustomizeNodeDelegate = UBlueprintNodeSpawner::FCustomizeNodeDelegate::CreateStatic(CustomizeTimelineNodeLambda);
		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

#undef LOCTEXT_NAMESPACE
