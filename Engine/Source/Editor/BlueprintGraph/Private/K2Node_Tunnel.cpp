// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_Tunnel.h"

#include "Containers/EnumAsByte.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/IConsoleManager.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Composite.h"
#include "K2Node_MacroInstance.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/WildcardNodeUtils.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Tunnel)

class FArchive;
class FCompilerResultsLog;
class UObject;

#define LOCTEXT_NAMESPACE "K2Node"

// @TODO_BH: Remove the CVar for validity checking when we can get all the errors sorted out in a pre-flight
// When we remove this then make sure we have a valid cook happening
namespace PinValidityCheck
{
	/** 
	* CVar controls pin validity warning which will throw when a macro graph is silently failing
	* @see UE-90009
	*/
	static bool bDisplayInvalidPinWarning = true;
	static FAutoConsoleVariableRef CVarDisplayInvalidPinWarning(
		TEXT("bp.PinValidityCheck.bDisplayInvalidPinWarning"), bDisplayInvalidPinWarning,
		TEXT("CVar controls pin validity warning which will throw when a macro graph is silently failing"),
		ECVF_Default);
}

UK2Node_Tunnel::UK2Node_Tunnel(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsEditable = true;
}

void UK2Node_Tunnel::DestroyNode()
{
	if (InputSinkNode != nullptr)
	{
		InputSinkNode->OutputSourceNode = nullptr;
	}

	if (OutputSourceNode != nullptr)
	{
		OutputSourceNode->InputSinkNode = nullptr;
	}

	//@TODO: Should we remove the pins provided by this node from the twinned node(s)?
	Super::DestroyNode();
}

void UK2Node_Tunnel::PostPasteNode()
{
	Super::PostPasteNode();
	//@TODO: The gateway<->gateway node pairing should be unique, so we need to disallow this paste somehow (or just let it be an error later on)!
}

FText UK2Node_Tunnel::GetTooltipText() const
{
	if (bCanHaveInputs && !bCanHaveOutputs)
	{
		return NSLOCTEXT("K2Node", "OutputTunnelTooltip", "Outputs from this graph");
	}
	else if (!bCanHaveInputs && bCanHaveOutputs)
	{
		return NSLOCTEXT("K2Node", "InputTunnelTooltip", "Inputs into this graph");
	}
	else
	{
		return NSLOCTEXT("K2Node", "TunnelConnectionTooltip", "Tunnel Connection");
	}
}

FText UK2Node_Tunnel::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (bCanHaveInputs && !bCanHaveOutputs)
	{
		return NSLOCTEXT("K2Node", "OutputTunnelTitle", "Outputs");
	}
	else if (!bCanHaveInputs && bCanHaveOutputs)
	{
		return NSLOCTEXT("K2Node", "InputTunnelTitle", "Inputs");
	}
	else
	{
		return NSLOCTEXT("K2Node", "TunnelConnectionTitle", "Tunnel Connection");
	}
}

FName UK2Node_Tunnel::CreateUniquePinName(FName InSourcePinName) const
{
	if (GetClass() == UK2Node_Tunnel::StaticClass())
	{
		// When dealing with a tunnel node that is not a sub class (macro/collapsed graph entry and result), attempt to find the paired node and find a valid name between the two
		TWeakObjectPtr<UK2Node_EditablePinBase> TunnelEntry;
		TWeakObjectPtr<UK2Node_EditablePinBase> TunnelResult;
		FBlueprintEditorUtils::GetEntryAndResultNodes(GetGraph(), TunnelEntry, TunnelResult);

		if (TunnelEntry.IsValid() && TunnelResult.IsValid())
		{
			FName PinName(InSourcePinName);

			int32 Index = 1;
			while (TunnelEntry.Get()->FindPin(PinName) != nullptr || TunnelResult.Get()->FindPin(PinName) != nullptr)
			{
				++Index;
				PinName = *FString::Printf(TEXT("%s%d"),*InSourcePinName.ToString(), Index);
			}

			return PinName;
		}
	}

	return Super::CreateUniquePinName(InSourcePinName);
}

bool UK2Node_Tunnel::CanUserDeleteNode() const
{
	// Disallow deletion of tunnels that are inside a tunnel graph, but allow it on top level tunnels that have gotten there on accident
	//@TODO: Remove this code 
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(this))
	{
		const bool bIsExactlyTunnel = (GetClass() == UK2Node_Tunnel::StaticClass());
		const bool bIsTopLevelGraph = (GetGraph()->GetOuter() == Blueprint);
		const bool bIsLibraryProject = (Blueprint->BlueprintType == BPTYPE_MacroLibrary);
		const bool bIsLocalMacro = Blueprint->MacroGraphs.Contains(GetGraph());
		if (bIsExactlyTunnel && bIsTopLevelGraph && !bIsLibraryProject && !bIsLocalMacro)
		{
			return true;
		}
	}

	return false;
}

bool UK2Node_Tunnel::CanDuplicateNode() const
{
	return true;
}

bool UK2Node_Tunnel::IsNodeSafeToIgnore() const
{
	// If there are no connections to anything on this node, it's safe to ignore
	int32 NumConnections = 0;
	for (int32 BoundaryPinIndex = 0; BoundaryPinIndex < Pins.Num(); ++BoundaryPinIndex)
	{
		UEdGraphPin* BoundaryPin = Pins[BoundaryPinIndex];
		NumConnections += BoundaryPin->LinkedTo.Num();
	}

	return NumConnections == 0;
}

bool UK2Node_Tunnel::DrawNodeAsExit() const
{
	return(bCanHaveInputs && !bCanHaveOutputs);
}

bool UK2Node_Tunnel::DrawNodeAsEntry() const
{
	return(!bCanHaveInputs && bCanHaveOutputs);
}

UK2Node_Tunnel* UK2Node_Tunnel::GetInputSink() const
{
	return InputSinkNode;
}

UK2Node_Tunnel* UK2Node_Tunnel::GetOutputSource() const
{
	return OutputSourceNode;
}

bool UK2Node_Tunnel::CanCreateUserDefinedPin(const FEdGraphPinType& InPinType, EEdGraphPinDirection InDesiredDirection, FText& OutErrorMessage)
{
	bool bResult = true;
	// Make sure that if this is an exec node we are allowed one.
	if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Exec && !CanModifyExecutionWires())
	{
		OutErrorMessage = LOCTEXT("MultipleExecPinError", "Cannot support more exec pins!");
		bResult = false;
	}
	else if(InDesiredDirection == EGPD_Input && !bCanHaveInputs)
	{
		OutErrorMessage = LOCTEXT("AddTunnelInputPinError", "Cannot add input pins to entry node!");
		bResult = false;
	}
	else if(InDesiredDirection == EGPD_Output && !bCanHaveOutputs)
	{
		OutErrorMessage = LOCTEXT("AddTunnelOutputPinError", "Cannot add output pins to entry node!");
		bResult = false;
	}
	return bResult;
}

void UK2Node_Tunnel::ClearCachedBlueprintData(UBlueprint* Blueprint)
{
	// Remove data marking graphs as latent, this will be re-cache'd as needed
	MetaData.HasLatentFunctions = INDEX_NONE;
}

void UK2Node_Tunnel::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);
	
	if (PinValidityCheck::bDisplayInvalidPinWarning)
	{
		FBlueprintEditorUtils::ValidatePinConnections(this, MessageLog);
	}
}

void UK2Node_Tunnel::FixupPinStringDataReferences(FArchive* SavingArchive)
{
	Super::FixupPinStringDataReferences(SavingArchive);
	if (SavingArchive)
	{
		UpdateUserDefinedPinDefaultValues();
	}
}

UEdGraphPin* UK2Node_Tunnel::CreatePinFromUserDefinition(const TSharedPtr<FUserPinInfo> NewPinInfo)
{
	// Create the new pin
	EEdGraphPinDirection Direction = bCanHaveInputs ? EGPD_Input : EGPD_Output;

	// Let the user pick the pin direction if legal
	if ( (bCanHaveInputs && NewPinInfo->DesiredPinDirection == EGPD_Input) || (bCanHaveOutputs && NewPinInfo->DesiredPinDirection == EGPD_Output) )
	{
		Direction = NewPinInfo->DesiredPinDirection;
	}

	const UEdGraphSchema_K2* const Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* Result = CreatePin(Direction, NewPinInfo->PinType, NewPinInfo->PinName);
	Schema->SetPinAutogeneratedDefaultValue(Result, NewPinInfo->PinDefaultValue);

	// Make sure it mirrors onto the associated node
	UEdGraphNode* TargetNode = (InputSinkNode ? InputSinkNode : OutputSourceNode);
	if (Cast<UK2Node_Composite>(TargetNode) || Cast<UK2Node_MacroInstance>(TargetNode))
	{
		UEdGraphPin* HasPinAlready = TargetNode->FindPin(Result->PinName);
		if (HasPinAlready == nullptr)
		{
			TargetNode->CreatePin(UEdGraphPin::GetComplementaryDirection(Direction), NewPinInfo->PinType, NewPinInfo->PinName);
		}
	}
	else if (UK2Node_Tunnel* TunnelNode = Cast<UK2Node_Tunnel>((Direction == EGPD_Input) ? InputSinkNode : OutputSourceNode))
	{
		TunnelNode->CreateUserDefinedPin(NewPinInfo->PinName, NewPinInfo->PinType, UEdGraphPin::GetComplementaryDirection(Direction));
	}

	//@TODO: Automatically update loaded macro instances when this node is changed too

	return Result;
}

bool UK2Node_Tunnel::ModifyUserDefinedPinDefaultValue(TSharedPtr<FUserPinInfo> PinInfo, const FString& NewDefaultValue)
{
	if (Super::ModifyUserDefinedPinDefaultValue(PinInfo, NewDefaultValue))
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->HandleParameterDefaultValueChanged(this);

		return true;
	}
	return false;
}

bool UK2Node_Tunnel::ShouldDoSmartWildcardInference()
{
	static const FBoolConfigValueHelper bUseSimpleWildcardInference(TEXT("Blueprints"), TEXT("bUseSimpleWildcardInference"), GEngineIni);
	return !bUseSimpleWildcardInference;
}

bool UK2Node_Tunnel::CanModifyExecutionWires()
{
	return true;
}

ERenamePinResult UK2Node_Tunnel::RenameUserDefinedPinImpl(const FName OldName, const FName NewName, bool bTest)
{
	const ERenamePinResult ThisNodeResult = Super::RenameUserDefinedPinImpl(OldName, NewName, bTest);
	if(ERenamePinResult::ERenamePinResult_NameCollision == ThisNodeResult)
	{
		return ERenamePinResult::ERenamePinResult_NameCollision;
	}

	// And do the same on the twinned pin
	ERenamePinResult TargetNodeResult = ERenamePinResult::ERenamePinResult_Success;
	UEdGraphNode* TargetNode = ((InputSinkNode != nullptr) ? InputSinkNode : OutputSourceNode);
	if (UK2Node_Composite* CompositeNode = Cast<UK2Node_Composite>(TargetNode))
	{
		TargetNodeResult = CompositeNode->RenameUserDefinedPin(OldName, NewName, bTest);
	}
	if(ERenamePinResult::ERenamePinResult_NameCollision == TargetNodeResult)
	{
		return ERenamePinResult::ERenamePinResult_NameCollision;
	}

	return FMath::Min<ERenamePinResult>(ThisNodeResult, TargetNodeResult);
}

UObject* UK2Node_Tunnel::GetJumpTargetForDoubleClick() const
{
	// Try to select the other side of a tunnel node
	UEdGraphNode* TargetNode = GetOutputSource();
	if (TargetNode == nullptr)
	{
		TargetNode = GetInputSink();
	}

	return TargetNode;
}

void UK2Node_Tunnel::CacheWildcardPins()
{
	WildcardPins.Reset();

	const bool bSupportMapWildcards = ShouldDoSmartWildcardInference();
	if(bSupportMapWildcards)
	{
		for (UEdGraphPin* Pin : Pins)
		{
			if (FWildcardNodeUtils::HasAnyWildcards(Pin))
			{
				WildcardPins.Add(Pin);
			}
		}
	}
	else
	{
		for (UEdGraphPin* Pin : Pins)
		{
			// for each of the wildcard pins...
			if (Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
			{
				WildcardPins.Add(Pin);
			}
		}
	}
}

void UK2Node_Tunnel::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	// determine if all wildcard pins are unlinked.
	// if they are, we should revert them all back to wildcard status
	bool bAllWildcardsAreUnlinked = true;
	for (UEdGraphPin* Pin : WildcardPins)
	{
		// find it in the old pins array (where it might not be a wildcard)
		// and see if it's unlinked
		for (UEdGraphPin* OldPin : OldPins)
		{
			if (OldPin->PinName == Pin->PinName)
			{
				TFunction<bool(UEdGraphPin*)> IsPinLinked = [&IsPinLinked](UEdGraphPin* InPin)
				{
					if (InPin->LinkedTo.Num() > 0)
					{
						return true;
					}
					for (UEdGraphPin* SubPin : InPin->SubPins)
					{
						if (IsPinLinked(SubPin))
						{
							return true;
						}
					}
					return false;
				};
				if (IsPinLinked(OldPin))
				{
					bAllWildcardsAreUnlinked = false;
					break;
				}
			}
		}
		if (bAllWildcardsAreUnlinked == false)
		{
			break;
		}
	}

	if (bAllWildcardsAreUnlinked == false)
	{
		const bool bSupportMapWildcards = ShouldDoSmartWildcardInference();
		// Copy pin types from old pins for wildcard pins
		for (UEdGraphPin* const Pin : WildcardPins)
		{
			// Only change the type if it is still a wildcard
			if ((bSupportMapWildcards && FWildcardNodeUtils::HasAnyWildcards(Pin)) ||
				Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
			{
				// find it in the old pins and copy the type
				for (UEdGraphPin const* const OldPin : OldPins)
				{
					if (OldPin->PinName == Pin->PinName)
					{
						Pin->PinType = OldPin->PinType;
						break;
					}
				}
			}
		}
	}
	PostFixupAllWildcardPins(bAllWildcardsAreUnlinked);
}

bool UK2Node_Tunnel::IsCompatibleWithGraph(const UEdGraph* InGraph) const
{
	const bool bIsEntryOrExit = bCanHaveInputs != bCanHaveOutputs;

	// If this is an Entry or Exit, make sure an Entry or exit doesn't already exist in this Graph
	if (bIsEntryOrExit)
	{
		TArray<UK2Node_Tunnel*> Tunnels;
		InGraph->GetNodesOfClass<UK2Node_Tunnel>(Tunnels);

		for (UK2Node_Tunnel* Node : Tunnels)
		{
			if (Node->bCanHaveInputs == bCanHaveInputs && Node->bCanHaveOutputs == bCanHaveOutputs)
			{
				return false;
			}
		}
	}

	return Super::IsCompatibleWithGraph(InGraph);
}

#undef LOCTEXT_NAMESPACE
