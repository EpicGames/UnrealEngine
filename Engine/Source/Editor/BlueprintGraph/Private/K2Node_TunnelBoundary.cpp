// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_TunnelBoundary.h"

#include "EdGraph/EdGraph.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_Composite.h"
#include "K2Node_MacroInstance.h"
#include "K2Node_Tunnel.h"
#include "KismetCompilerMisc.h"
#include "Templates/Casts.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_TunnelBoundary)

struct FKismetFunctionContext;

#define LOCTEXT_NAMESPACE "UK2Node_TunnelBoundary"

class FKCHandler_TunnelBoundary : public FNodeHandlingFunctor
{
public:
	FKCHandler_TunnelBoundary(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		// Tunnel boundaries do nothing except continue the execution sequence.
		GenerateSimpleThenGoto(Context, *Node);
	}

};

UK2Node_TunnelBoundary::UK2Node_TunnelBoundary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

FText UK2Node_TunnelBoundary::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("BaseName"), FText::FromName(BaseName));
	switch (TunnelBoundaryType)
	{
	case ETunnelBoundaryType::EntrySite:
		Args.Add(TEXT("TunnelBoundaryType"), FText::FromString(TEXT("Tunnel Entry Site")));
		break;

	case ETunnelBoundaryType::InputSite:
		Args.Add(TEXT("TunnelBoundaryType"), FText::FromString(TEXT("Tunnel Input Site")));
		break;

	case ETunnelBoundaryType::OutputSite:
		Args.Add(TEXT("TunnelBoundaryType"), FText::FromString(TEXT("Tunnel Output Site")));
		break;
	}

	return FText::Format(LOCTEXT("UK2Node_TunnelBoundary_FullTitle", "{BaseName} - {TunnelBoundaryType}"), Args);
}

FNodeHandlingFunctor* UK2Node_TunnelBoundary::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_TunnelBoundary(CompilerContext);
}

void UK2Node_TunnelBoundary::SetNodeAttributes(const UK2Node_Tunnel* SourceNode)
{
	if (const UK2Node_MacroInstance* MacroInstance = Cast<UK2Node_MacroInstance>(SourceNode))
	{
		// Macro tunnel instance node
		const UEdGraph* TunnelGraph = MacroInstance->GetMacroGraph();
		BaseName = TunnelGraph ? TunnelGraph->GetFName() : NAME_None;
		TunnelBoundaryType = ETunnelBoundaryType::EntrySite;
	}
	else if (const UK2Node_Composite* CompositeInstance = Cast<UK2Node_Composite>(SourceNode))
	{
		// Composite tunnel instance node
		const UEdGraph* TunnelGraph = CompositeInstance->BoundGraph;
		BaseName = TunnelGraph ? TunnelGraph->GetFName() : NAME_None;
		TunnelBoundaryType = ETunnelBoundaryType::EntrySite;
	}
	else if (SourceNode)
	{
		// Input or output tunnel node
		SetNodeAttributes(SourceNode->bCanHaveOutputs ? SourceNode->GetOutputSource() : SourceNode->GetInputSink());
		TunnelBoundaryType = SourceNode->bCanHaveInputs ? ETunnelBoundaryType::OutputSite : ETunnelBoundaryType::InputSite;
	}
	else
	{
		BaseName = NAME_None;
		TunnelBoundaryType = ETunnelBoundaryType::Unknown;
	}
}

#undef LOCTEXT_NAMESPACE
