// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_Variable.h"

#include "BlueprintCompilationManager.h"
#include "BlueprintEditorSettings.h"
#include "Components/ActorComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SimpleConstructionScript.h"
#include "GameFramework/Actor.h"
#include "GameFramework/MovementComponent.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_FunctionEntry.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Kismet2/StructureEditorUtils.h"
#include "KismetCompilerMisc.h"
#include "Logging/MessageLog.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Preferences/UnrealEdOptions.h"
#include "Serialization/Archive.h"
#include "SourceCodeNavigation.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UnrealEdGlobals.h"
#include "Settings/BlueprintEditorProjectSettings.h"
#include "ToolMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_Variable)

#define LOCTEXT_NAMESPACE "K2Node"

UK2Node_Variable::UK2Node_Variable(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_Variable::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Fix old content 
	if(Ar.IsLoading())
	{
		if(Ar.UEVer() < VER_UE4_VARK2NODE_USE_MEMBERREFSTRUCT)
		{
			// Copy info into new struct
			VariableReference.SetDirect(VariableName_DEPRECATED, FGuid(), VariableSourceClass_DEPRECATED, bSelfContext_DEPRECATED);
		}

		if(Ar.UEVer() < VER_UE4_K2NODE_VAR_REFERENCEGUIDS)
		{
			FGuid VarGuid;
			
			// Do not let this code run for local variables
			if (!VariableReference.IsLocalScope())
			{
				const bool bSelf = VariableReference.IsSelfContext();
				UClass* MemberParentClass = VariableReference.GetMemberParentClass(nullptr);
				if (UBlueprint::GetGuidFromClassByFieldName<FProperty>(bSelf? *GetBlueprint()->GeneratedClass : MemberParentClass, VariableReference.GetMemberName(), VarGuid))
				{
					VariableReference.SetDirect(VariableReference.GetMemberName(), VarGuid, bSelf ? nullptr : MemberParentClass, bSelf);
				}
			}
		}
	}
}

void UK2Node_Variable::SetFromProperty(const FProperty* Property, bool bSelfContext, UClass* OwnerClass)
{
	SelfContextInfo = bSelfContext ? ESelfContextInfo::Unspecified : ESelfContextInfo::NotSelfContext;
	VariableReference.SetFromField<FProperty>(Property, bSelfContext, OwnerClass);
}

bool UK2Node_Variable::CreatePinForVariable(EEdGraphPinDirection Direction, FName InPinName/* = NAME_None */)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	FProperty* VariableProperty = GetPropertyForVariable();
	// favor the skeleton property if possible (in case the property type has 
	// been changed, and not yet compiled).
	if (!VariableReference.IsSelfContext())
	{
		UClass* VariableClass = VariableReference.GetMemberParentClass(GetBlueprint()->GeneratedClass);
		if (UBlueprintGeneratedClass* BpClassOwner = Cast<UBlueprintGeneratedClass>(VariableClass))
		{
			// this variable could currently only be a part of some skeleton 
			// class (the blueprint has not be compiled with it yet), so let's 
			// check the skeleton class as well, see if we can pull pin data 
			// from there...
			UBlueprint* VariableBlueprint = CastChecked<UBlueprint>(BpClassOwner->ClassGeneratedBy.Get(), ECastCheckedType::NullAllowed);
			if (VariableBlueprint)
			{
				if (FProperty* SkelProperty = GetPropertyForVariableFromSkeleton())
				{
					VariableProperty = SkelProperty;
				}
			}
		}
	}

	if (VariableProperty != nullptr)
	{
		const FName PinName = InPinName.IsNone() ? GetVarName() : InPinName;
		// Create the pin
		UEdGraphPin* VariablePin = CreatePin(Direction, NAME_None, PinName);
		if (VariableProperty->IsNative())
		{
			VariablePin->PinFriendlyName = VariableProperty->GetDisplayNameText();
		}

		K2Schema->ConvertPropertyToPinType(VariableProperty, /*out*/ VariablePin->PinType);
		K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(VariablePin);
	}
	else
	{
		// Don't handle warning or error logging here, that needs to be done in ValidateNodeDuringCompilation
		return false;
	}

	return true;
}

void UK2Node_Variable::CreatePinForSelf()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	// Create the self pin
	if (!K2Schema->FindSelfPin(*this, EGPD_Input))
	{
		// Do not create a self pin for locally scoped variables or sparse class data
		if (!VariableReference.IsLocalScope())
		{
			bool bSelfTarget = VariableReference.IsSelfContext() && (ESelfContextInfo::NotSelfContext != SelfContextInfo);
			UClass* MemberParentClass = VariableReference.GetMemberParentClass(GetBlueprintClassFromNode());
			UClass* TargetClass = MemberParentClass;
			
			FProperty* VariableProperty = GetPropertyForVariable();

			if (VariableProperty)
			{
				UClass* PropertyClass = Cast<UClass>(VariableProperty->GetOwner<UObject>());

				// Fix up target class if it's not correct, this fixes cases where variables have moved within the hierarchy
				if (PropertyClass && PropertyClass != TargetClass)
				{
					TargetClass = PropertyClass;
				}
			}

			// Self Target pins should always make the class be the owning class of the property,
			// so if the node is from a Macro Blueprint, it will hook up as self in any placed Blueprint
			if (bSelfTarget)
			{
				if (FProperty* Property = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
				{
					UClass* OwnerClass = Property->GetOwnerClass();
					if (OwnerClass)
					{
						TargetClass = OwnerClass->GetAuthoritativeClass();
					}
					else if(TargetClass)
					{
						// check if it's a sparse member, sparse members are accessed via the authoritative
						// class - this matches the convention defined in BlueprintActionDatabaseImpl::AddClassDataObjectActions:
						UClass* AuthClass = TargetClass->GetAuthoritativeClass();
						if (AuthClass->GetSparseClassDataStruct()->IsChildOf(Property->GetOwnerStruct()) )
						{
							TargetClass = AuthClass;
						}
					}
				}
				else if (GetBlueprint()->SkeletonGeneratedClass)
				{
					TargetClass = GetBlueprint()->SkeletonGeneratedClass->GetAuthoritativeClass();
				}
			}
			else if (TargetClass && TargetClass->ClassGeneratedBy)
			{
				TargetClass = TargetClass->GetAuthoritativeClass();
			}

			UEdGraphPin* TargetPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, TargetClass, UEdGraphSchema_K2::PN_Self);
			TargetPin->PinFriendlyName =  LOCTEXT("Target", "Target");

			if (bSelfTarget)
			{
				TargetPin->bHidden = true; // don't show in 'self' context
			}
		}
	}
	else
	{
		//@TODO: Check that the self pin types match!
	}
}

bool UK2Node_Variable::RecreatePinForVariable(EEdGraphPinDirection Direction, TArray<UEdGraphPin*>& OldPins, FName InPinName/* = NAME_None*/)
{
	// probably the node was pasted to a blueprint without the variable
	// we don't want to beak any connection, so the pin will be recreated from old one, but compiler will throw error

	// find old variable pin
	const UEdGraphPin* OldVariablePin = nullptr;
	const FName PinName = InPinName.IsNone() ? GetVarName() : InPinName;
	for (const UEdGraphPin* Pin : OldPins)
	{
		if (Pin && PinName == Pin->PinName)
		{
			OldVariablePin = Pin;
			break;
		}
	}

	if (nullptr != OldVariablePin)
	{
		// create new pin from old one
		UEdGraphPin* VariablePin = CreatePin(Direction, NAME_None, PinName);
		VariablePin->PinType = OldVariablePin->PinType;
		
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		K2Schema->SetPinAutogeneratedDefaultValueBasedOnType(VariablePin);

		Message_Note(*FString::Printf(TEXT("Pin for variable '%s' recreated, but the variable is missing."), *PinName.ToString()));
		return true;
	}
	else
	{
		Message_Warn(*FString::Printf(TEXT("RecreatePinForVariable: '%s' pin not found"), *PinName.ToString()));
		return false;
	}
}

FLinearColor UK2Node_Variable::GetNodeTitleColor() const
{
	FProperty* VariableProperty = GetPropertyForVariable();
	if (VariableProperty)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType VariablePinType;
		K2Schema->ConvertPropertyToPinType(VariableProperty, VariablePinType);

		return K2Schema->GetPinTypeColor(VariablePinType);
	}

	return FLinearColor::White;
}

FString UK2Node_Variable::GetFindReferenceSearchString_Impl(EGetFindReferenceSearchStringFlags InFlags) const
{
	// Legacy behavior for variable nodes was to do an exact search
	if (EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::UseSearchSyntax) || EnumHasAnyFlags(InFlags, EGetFindReferenceSearchStringFlags::Legacy))
	{
		if (VariableReference.IsLocalScope())
		{
			// Generate local variable search query
			return VariableReference.GetReferenceSearchString(nullptr);
		}
		else if (FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
		{
			// Generate member variable search query
			return VariableReference.GetReferenceSearchString(VariableProperty->GetOwnerClass());
		}
	}

	// Simple query: just search for variable name
	return FString::Printf(TEXT("\"%s\""), *VariableReference.GetMemberName().ToString());
}

UK2Node::ERedirectType UK2Node_Variable::DoPinsMatchForReconstruction(const UEdGraphPin* NewPin, int32 NewPinIndex, const UEdGraphPin* OldPin, int32 OldPinIndex) const 
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	if( OldPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Exec )
	{
		return Super::DoPinsMatchForReconstruction(NewPin, NewPinIndex, OldPin, OldPinIndex);
	}

	const bool bPinNamesMatch = (OldPin->PinName == NewPin->PinName);
	const bool bCanMatchSelfs = bPinNamesMatch || ((OldPin->PinName == UEdGraphSchema_K2::PN_Self) && (NewPin->PinName == UEdGraphSchema_K2::PN_Self));
	const bool bTheSameDirection = (NewPin->Direction == OldPin->Direction);

	const bool bNewPinIsObject = (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object);
	const bool bNewPinIsInterface = (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Interface);
	const bool bNewIsClass = (NewPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class);
	const bool bNewPinIsObjectType = 
		bNewPinIsObject ||
		bNewPinIsInterface ||
		bNewIsClass
	;
	
	const FEdGraphPinType& InputType = (OldPin->Direction == EGPD_Output) ? OldPin->PinType : NewPin->PinType;
	const FEdGraphPinType& OutputType = (OldPin->Direction == EGPD_Output) ? NewPin->PinType : OldPin->PinType;
	
	const bool bHasIncompatibleObjectType =
		bNewPinIsObjectType &&
		!K2Schema->ArePinTypesCompatible(OutputType, InputType)
	;

	if (bCanMatchSelfs && bTheSameDirection)
	{
		// the order that the PinTypes are passed to ArePinTypesCompatible() 
		// matters; object pin types are seen as compatible when the output-
		// pin's type is a subclass of the input-pin's type, so we want to keep 
		// that in mind here (should the pins "MatchForReconstruction" if the 
		// variable has been changed to a super class of the original? what 
		// about a subclass?
		// 
		// if these are output nodes, then it is perfectly acceptable that the 
		// variable has been altered to be a sub-class ref (meaning we should 
		// treat the NewPin as an output)... the opposite applies if the pins 
		// are inputs

		if (NewPin->ParentPin)
		{
			// If the OldPin is not split, then these don't match
			if (OldPin->ParentPin == nullptr)
			{
				return ERedirectType_None;
			}

			// Go through and find the original variable pin.
			// If the number of steps out to the original variable pin is not the same then these don't match
			const UEdGraphPin* ParentmostNewPin = NewPin;
			const UEdGraphPin* ParentmostOldPin = OldPin;

			while (ParentmostNewPin->ParentPin)
			{
				if (ParentmostOldPin->ParentPin == nullptr)
				{
					return ERedirectType_None;
				}
				ParentmostNewPin = ParentmostNewPin->ParentPin;
				ParentmostOldPin = ParentmostOldPin->ParentPin;
			}

			if (ParentmostOldPin->ParentPin)
			{
				return ERedirectType_None;
			}

			// Compare whether the names, ignoring the original variable's name in the case of renames, match
			FName NewPinPropertyName = FName(*NewPin->PinName.ToString().RightChop(ParentmostNewPin->PinName.ToString().Len() + 1));
			FName OldPinPropertyName = FName(*OldPin->PinName.ToString().RightChop(ParentmostOldPin->PinName.ToString().Len() + 1));

			if (!DoesRenamedVariableMatch(OldPinPropertyName, NewPinPropertyName, Cast<UStruct>(NewPin->ParentPin->PinType.PinSubCategoryObject.Get())))
			{
				return ERedirectType_None;
			}

			return ERedirectType_Name;
		}
		else if (bHasIncompatibleObjectType)
		{
			// Special Case: If we had a pin match, and the class isn't loaded 
			//               yet because of a cyclic dependency, temporarily 
			//               cast away the const, and fix up.
			if ( bPinNamesMatch &&
				(bNewPinIsObject || bNewPinIsInterface) &&
				(NewPin->PinType.PinSubCategoryObject == nullptr) )
			{
				// @TODO:  Fix this up to be less hacky
				UBlueprintGeneratedClass* TypeClass = Cast<UBlueprintGeneratedClass>(OldPin->PinType.PinSubCategoryObject.Get());
				if (TypeClass && TypeClass->ClassGeneratedBy && TypeClass->ClassGeneratedBy->HasAnyFlags(RF_BeingRegenerated))
				{
					UEdGraphPin* NonConstNewPin = (UEdGraphPin*)NewPin;
					NonConstNewPin->PinType.PinSubCategoryObject = OldPin->PinType.PinSubCategoryObject.Get();
					return ERedirectType_Name;
				}
			}
			// Special Case: if we have object pins that are "compatible" in the
			//               reverse order (meaning one's type is a sub-class of 
			//               the other's), then they could still be acceptable 
			//               if all their connections are still valid (for 
			//               example: if the OldPin was an output only connected 
			//               to super-class pins)
			else if (bNewPinIsObject && K2Schema->ArePinTypesCompatible(InputType, OutputType))
			{
				bool bLinksCompatible = (OldPin->LinkedTo.Num() > 0) && (OldPin->DefaultObject == nullptr);
				for (UEdGraphPin* OldLink : OldPin->LinkedTo)
				{
					const FEdGraphPinType& LinkInputType  = (OldPin->Direction == EGPD_Input) ? NewPin->PinType : OldLink->PinType;
					const FEdGraphPinType& LinkOutputType = (OldPin->Direction == EGPD_Input) ? OldLink->PinType : NewPin->PinType;
				
					if (!K2Schema->ArePinTypesCompatible(LinkOutputType, LinkInputType))
					{
						bLinksCompatible = false;
						break;
					}
				}

				if (bLinksCompatible)
				{
					return ERedirectType_Name;
				}
			}
			else
			{
				const UClass* PSCOClass = Cast<UClass>(OldPin->PinType.PinSubCategoryObject.Get());
				const bool bOldIsBlueprint = PSCOClass && PSCOClass->IsChildOf(UBlueprint::StaticClass());
				// Special Case: If we're migrating from old blueprint references 
				//               to class references, allow pins to be reconnected if coerced
				if (bNewIsClass && bOldIsBlueprint)
				{
					UEdGraphPin* OldPinNonConst = (UEdGraphPin*)OldPin;
					OldPinNonConst->PinName = NewPin->PinName;
					return ERedirectType_Name;
				}
			}
		}
		else
		{
			// By default, we allow the redirect if direction and name match.
			// The type may not match, but we defer that check to ValidateLinkedPinTypes,
			// which attempts to address the incompatibility by inserting a conversion node.
			// This behavior is consistent with how UK2Node operates.
			return ERedirectType_Name;
		}
	}

	return ERedirectType_None;
}

UClass* UK2Node_Variable::GetVariableSourceClass() const
{
	UClass* Result = VariableReference.GetMemberParentClass(GetBlueprintClassFromNode());
	return Result;
}

FProperty* UK2Node_Variable::GetPropertyForVariable_Internal(UClass* OwningClass) const
{
	const FName VarName = GetVarName();

	FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(OwningClass);

	// if the variable has been deprecated, don't use it
	if (VariableProperty != nullptr)
	{
		UEdGraphPin* VariablePin = FindPin(VarName);

		// If the variable has been remapped update the pin
		if (VariablePin && VarName != GetVarName())
		{
			VariablePin->PinName = GetVarName();
		}
	}

	return VariableProperty;
}

FProperty* UK2Node_Variable::GetPropertyForVariable() const
{
	if (!FBlueprintCompilationManager::IsGeneratedClassLayoutReady())
	{
		// first look in the skeleton class:
		if (FProperty* SkeletonProperty = GetPropertyForVariableFromSkeleton())
		{
			return SkeletonProperty;
		}
	}

	return GetPropertyForVariable_Internal(GetBlueprintClassFromNode());
}

FString UK2Node_Variable::GetPinMetaData(FName InPinName, FName InKey)
{
	FString MetaData;

	if (GetVarName() == InPinName)
	{
		if (FProperty* VariableProperty = GetPropertyForVariable())
		{
			if (const FString* FoundMetaData = VariableProperty->FindMetaData(FBlueprintMetadata::MD_AllowAbstractClasses))
			{
				MetaData = *FoundMetaData;
			}
		}
	}

	if (MetaData.IsEmpty())
	{
		MetaData = Super::GetPinMetaData(InPinName, InKey);
	}

	return MetaData;
}

bool UK2Node_Variable::DoesRenamedVariableMatch(FName OldVariableName, FName NewVariableName, UStruct* StructType)
{
	if (NewVariableName == OldVariableName)
	{
		return true;
	}

	FGuid OldPropertyGuid = FStructureEditorUtils::GetGuidFromPropertyName(OldVariableName);
	FGuid NewPropertyGuid = FStructureEditorUtils::GetGuidFromPropertyName(NewVariableName);

	// If guids match this was a user struct rename
	if (OldPropertyGuid.IsValid() && (OldPropertyGuid == NewPropertyGuid))
	{
		return true;
	}

	// Also check native rename if we can find the struct
	if (StructType)
	{
		FName RedirectedPinName = FProperty::FindRedirectedPropertyName(StructType, OldVariableName);

		if (NewVariableName == RedirectedPinName)
		{
			return true;
		}
	}

	return false;
}

FProperty* UK2Node_Variable::GetPropertyForVariableFromSkeleton() const
{
	if (UClass* SkeletonClass = FBlueprintEditorUtils::GetSkeletonClass(VariableReference.GetMemberParentClass(GetBlueprintClassFromNode())))
	{
		return GetPropertyForVariable_Internal(SkeletonClass);
	}

	return nullptr;
}

UEdGraphPin* UK2Node_Variable::GetValuePin() const
{
	UEdGraphPin* Pin = FindPin(GetVarName());
	check(Pin == nullptr || Pin->Direction == EGPD_Output);
	return Pin;
}

void UK2Node_Variable::ValidateNodeDuringCompilation(class FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	FProperty* VariableProperty = GetPropertyForVariable();

	// Local variables do not exist until much later in the compilation than this function can provide
	if (VariableProperty == NULL && !VariableReference.IsLocalScope())
	{
		if (!VariableReference.IsDeprecated())
		{
			FString OwnerName;

			UBlueprint* Blueprint = GetBlueprint();
			if (Blueprint != nullptr)
			{
				OwnerName = Blueprint->GetPathName();
				if (UClass* VarOwnerClass = VariableReference.GetMemberParentClass(Blueprint->GeneratedClass))
				{
					OwnerName = VarOwnerClass->GetPathName();
				}
			}

			if (!FKismetCompilerUtilities::IsMissingMemberPotentiallyLoading(Blueprint, VariableReference.GetMemberParentClass()))
			{
				FString const VarName = VariableReference.GetMemberName().ToString();

				FText const WarningFormat = LOCTEXT("VariableNotFoundFmt", "Could not find a variable named \"{0}\" in '{1}'.\nMake sure '{2}' has been compiled for @@");
				MessageLog.Warning(*FText::Format(WarningFormat, FText::FromString(VarName), FText::FromString(OwnerName), FText::FromString(OwnerName)).ToString(), this);
			}
		}
		else
		{
			MessageLog.Warning(*FText::Format(LOCTEXT("VariableDeprecatedFmt", "Variable '{0}' for @@ was deprecated.  Please update it."), FText::FromString(VariableReference.GetMemberName().ToString())).ToString(), this);
		}
	}

	if (VariableProperty && (VariableProperty->ArrayDim > 1))
	{
		MessageLog.Warning(*LOCTEXT("StaticArray_Warning", "@@ - the native property is a static array, which is not supported by blueprints").ToString(), this);
	}
}

FSlateIcon UK2Node_Variable::GetIconAndTint(FLinearColor& ColorOut) const
{
	const UStruct* VarScope = VariableReference.IsLocalScope() ?
		VariableReference.GetMemberScope(GetBlueprintClassFromNode()) :
		GetVariableSourceClass();

	return GetVariableIconAndColor(VarScope, GetVarName(), ColorOut);
}

FSlateIcon UK2Node_Variable::GetVarIconFromPinType(const FEdGraphPinType& InPinType, FLinearColor& IconColorOut)
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	IconColorOut = K2Schema->GetPinTypeColor(InPinType);

	if (InPinType.IsArray())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.ArrayVariableIcon");
	}
	else if (InPinType.IsSet())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.SetVariableIcon");
	}
	else if (InPinType.IsMap())
	{
		// TODO: Need to properly deal with Key/Value stuff
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.MapVariableKeyIcon");
	}
	else if (InPinType.PinSubCategoryObject.IsValid())
	{
		if(UClass* Class = Cast<UClass>(InPinType.PinSubCategoryObject.Get()))
		{
			return FSlateIconFinder::FindIconForClass( Class );
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.VariableIcon");
}

FText UK2Node_Variable::GetToolTipHeading() const
{
	FText Heading = Super::GetToolTipHeading();

	// attempt to reflect the node's GetCornerIcon() with some tooltip documentation 
	FText IconTag;
	if ( FProperty const* VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()) )
	{
		const UActorComponent* Component = GetActorComponent(VariableProperty);
		const bool IsEditorOnly = VariableProperty->HasAnyPropertyFlags(CPF_EditorOnly) || (Component && Component->bIsEditorOnly);
		const bool IsReplicated = VariableProperty->HasAnyPropertyFlags(CPF_Net) || (Component && Component->GetIsReplicated());
		const bool IsFunctionParameter = VariableProperty->HasAnyPropertyFlags(CPF_Parm);
		const bool IsLocalFunctionVariable = !VariableProperty->HasAnyPropertyFlags(CPF_Parm) && VariableReference.IsLocalScope();
		const UBlueprintEditorSettings* EditorSettings = GetDefault<UBlueprintEditorSettings>();
		if (IsEditorOnly && IsReplicated)
		{
			IconTag = LOCTEXT("ReplicatedEditorOnlyVar", "Editor-Only | Replicated");
		}
		else if (IsReplicated)
		{
			IconTag = LOCTEXT("ReplicatedVar", "Replicated");
		}
		else if (IsEditorOnly)
		{
			IconTag = LOCTEXT("EditorOnlyVar", "Editor-Only");
		}
		else if (IsFunctionParameter && EditorSettings->bShowFunctionParameterIcon)
		{
			IconTag = LOCTEXT("FunctionParameterVar", "Function parameter");
		}
		else if (IsLocalFunctionVariable && EditorSettings->bShowFunctionLocalVariableIcon)
		{
			IconTag = LOCTEXT("LocalFunctionVar", "Function local variable");
		}
	}

	if (Heading.IsEmpty())
	{
		return IconTag;
	}
	else if (!IconTag.IsEmpty())
	{
		Heading = FText::Format(FText::FromString("{0}\n{1}"), IconTag, Heading);
	}
	return Heading;
}

void UK2Node_Variable::GetNodeAttributes( TArray<TKeyValuePair<FString, FString>>& OutNodeAttributes ) const
{
	FProperty* VariableProperty = GetPropertyForVariable();
	const FString VariableName = VariableProperty ? VariableProperty->GetName() : TEXT( "InvalidVariable" );
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Type" ), TEXT( "Variable" ) ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Class" ), GetClass()->GetName() ));
	OutNodeAttributes.Add( TKeyValuePair<FString, FString>( TEXT( "Name" ), VariableName ));
}

void UK2Node_Variable::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	UClass* const NodeRefClass = VariableReference.GetMemberParentClass(InBlueprint->GeneratedClass);
	if (NodeRefClass && NodeRefClass->IsChildOf(InVariableClass) && InOldVarName == GetVarName())
	{
		Modify();

		if (VariableReference.IsLocalScope())
		{
			VariableReference.SetLocalMember(InNewVarName, VariableReference.GetMemberScopeName(), VariableReference.GetMemberGuid());
		}
		else if (VariableReference.IsSelfContext())
		{
			VariableReference.SetSelfMember(InNewVarName);
		}
		else
		{
			VariableReference.SetExternalMember(InNewVarName, NodeRefClass);
		}

		RenameUserDefinedPin(InOldVarName, InNewVarName);
	}
}

void UK2Node_Variable::ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement)
{
	if (VariableReference.IsLocalScope() || VariableReference.IsSelfContext())
	{
		VariableReference = InReplacement;
	}
	else
	{
		// Make a copy because ResolveMember is non-const
		FMemberReference Replacement = InReplacement;
		const FProperty* ResolvedProperty = Replacement.ResolveMember<FProperty>(InBlueprint);
		VariableReference.SetFromField<FProperty>(ResolvedProperty, InReplacementBlueprint->GeneratedClass);
	}
}

bool UK2Node_Variable::ReferencesVariable(const FName& InVarName, const UStruct* InScope) const
{
	if (InVarName == GetVarName())
	{
		if (InScope && VariableReference.GetMemberScopeName() != InScope->GetName())
		{
			// Variables are not in the same scope
			return false;
		}
		
		return true;
	}

	return false;
}

FSlateIcon UK2Node_Variable::GetVariableIconAndColor(const UStruct* VarScope, FName VarName, FLinearColor& IconColorOut)
{
	if(VarScope != NULL)
	{
		FProperty* Property = FindFProperty<FProperty>(VarScope, VarName);
		if(Property != NULL)
		{
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

			FEdGraphPinType PinType;
			if(K2Schema->ConvertPropertyToPinType(Property,  PinType)) // use schema to get the color
			{
				return GetVarIconFromPinType(PinType, IconColorOut);
			}
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.AllClasses.VariableIcon");
}


void UK2Node_Variable::CheckForErrors(const UEdGraphSchema_K2* Schema, FCompilerResultsLog& MessageLog)
{
	if(!VariableReference.IsSelfContext() && VariableReference.GetMemberParentClass(GetBlueprintClassFromNode()) != NULL)
	{
		// Check to see if we're not a self context, if we have a valid context.  It may have been purged because of a dead execution chain
		UEdGraphPin* ContextPin = Schema->FindSelfPin(*this, EGPD_Input);
		if((ContextPin != NULL) && (ContextPin->LinkedTo.Num() == 0) && (ContextPin->DefaultObject == NULL))
		{
			MessageLog.Error(*LOCTEXT("VarNodeError_InvalidVarTarget", "Variable node @@ uses an invalid target.  It may depend on a node that is not connected to the execution chain, and got purged.").ToString(), this);
		}
	}
}

void UK2Node_Variable::ReconstructNode()
{
	// update the variable reference if the property was renamed
	UClass* const VarClass = GetVariableSourceClass();
	if (VarClass)
	{
		bool bRemappedProperty = false;
		UClass* SearchClass = VarClass;
		while (SearchClass != nullptr)
		{
			FName NewPropertyName = FProperty::FindRedirectedPropertyName(SearchClass, VariableReference.GetMemberName());

			if (NewPropertyName != NAME_None)
			{
				if (VariableReference.IsSelfContext())
				{
					VariableReference.SetSelfMember(NewPropertyName);
				}
				else
				{
					VariableReference.SetExternalMember(NewPropertyName, VarClass);
				}

				// found, can break
				bRemappedProperty = true;
				break;
			}

			SearchClass = SearchClass->GetSuperClass();
		}

		if (!bRemappedProperty)
		{
			static FName OldVariableName(TEXT("UpdatedComponent"));
			static FName NewVariableName(TEXT("UpdatedPrimitive"));
			bRemappedProperty = RemapRestrictedLinkReference(OldVariableName, NewVariableName, UMovementComponent::StaticClass(), UPrimitiveComponent::StaticClass(), true);
		}
	}

	const FGuid VarGuid = VariableReference.GetMemberGuid();
	if (VarGuid.IsValid())
	{
		const FName VarName = UBlueprint::GetFieldNameFromClassByGuid<FProperty>(VarClass, VarGuid);
		if (VarName != NAME_None && VarName != VariableReference.GetMemberName())
		{
			if (VariableReference.IsSelfContext())
			{
				VariableReference.SetSelfMember( VarName );
			}
			else
			{
				VariableReference.SetExternalMember( VarName, VarClass );
			}
		}
	}

	Super::ReconstructNode();
}


bool UK2Node_Variable::RemapRestrictedLinkReference(FName OldVariableName, FName NewVariableName, const UClass* MatchInVariableClass, const UClass* RemapIfLinkedToClass, bool bLogWarning)
{
	bool bRemapped = false;
	if (VariableReference.GetMemberName() == OldVariableName)
	{
		UClass* const VarClass = GetVariableSourceClass();
		if (VarClass->IsChildOf(MatchInVariableClass))
		{
			UEdGraphPin* VariablePin = GetValuePin();
			if (VariablePin)
			{
				for (UEdGraphPin* OtherPin : VariablePin->LinkedTo)
				{
					if (OtherPin != nullptr && VariablePin->PinType.PinCategory == OtherPin->PinType.PinCategory)
					{
						// If any other pin we are linked to is a more restricted type, we need to do the remap.
						const UClass* OtherPinClass = Cast<UClass>(OtherPin->PinType.PinSubCategoryObject.Get());
						if (OtherPinClass && OtherPinClass->IsChildOf(RemapIfLinkedToClass))
						{
							if (VariableReference.IsSelfContext())
							{
								VariableReference.SetSelfMember(NewVariableName);
							}
							else
							{
								VariableReference.SetExternalMember(NewVariableName, VarClass);
							}
							bRemapped = true;
							break;
						}
					}
				}
			}
		}
	}

	if (bRemapped && bLogWarning && GetBlueprint())
	{
		FMessageLog("BlueprintLog").Warning(
			FText::Format(
			LOCTEXT("RemapRestrictedLinkReference", "{0}: Variable '{1}' was automatically changed to '{2}'. Verify that logic works as intended. (This warning will disappear once the blueprint has been resaved)"),
			FText::FromString(GetBlueprint()->GetPathName()),
			FText::FromString(MatchInVariableClass->GetName() + TEXT(".") + OldVariableName.ToString()),
			FText::FromString(MatchInVariableClass->GetName() + TEXT(".") + NewVariableName.ToString())
			));
	}

	return bRemapped;
}

FName UK2Node_Variable::GetCornerIcon() const
{
	if (const FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
	{
		const UActorComponent* Component = GetActorComponent(VariableProperty);
		const bool IsEditorOnly = VariableProperty->HasAnyPropertyFlags(CPF_EditorOnly) || (Component && Component->bIsEditorOnly);
		const bool IsReplicated = VariableProperty->HasAnyPropertyFlags(CPF_Net) || (Component && Component->GetIsReplicated());
		const bool IsFunctionParameter = VariableProperty->HasAnyPropertyFlags(CPF_Parm);
		const bool IsLocalFunctionVariable = !VariableProperty->HasAnyPropertyFlags(CPF_Parm) && VariableReference.IsLocalScope();
		const UBlueprintEditorSettings* EditorSettings = GetDefault<UBlueprintEditorSettings>();
		if (IsReplicated)
		{
			return TEXT("Graph.Replication.Replicated");
		}
		else if (IsEditorOnly)
		{
			return TEXT("Graph.Editor.EditorOnlyIcon");
		}
		else if (IsFunctionParameter && EditorSettings->bShowFunctionParameterIcon)
		{
			return TEXT("Graph.Function.FunctionParameterIcon");
		}
		else if (IsLocalFunctionVariable && EditorSettings->bShowFunctionLocalVariableIcon)
		{
			return TEXT("Graph.Function.FunctionLocalVariableIcon");
		}
	}

	return Super::GetCornerIcon();
}

bool UK2Node_Variable::HasExternalDependencies(TArray<class UStruct*>* OptionalOutput) const
{
	UBlueprint* SourceBlueprint = GetBlueprint();
	FProperty* VariableProperty = GetPropertyForVariable();
	UClass* PropertySourceClass = VariableProperty ? VariableProperty->GetOwnerClass() : nullptr;
	bool bResult = (PropertySourceClass && (PropertySourceClass->ClassGeneratedBy != SourceBlueprint));
	if (bResult && OptionalOutput)
	{
		OptionalOutput->AddUnique(PropertySourceClass);
	}

	// Also include underlying non-native variable types as external dependencies. Otherwise, contextual
	// type references serialized to bytecode can potentially be invalidated when the type is regenerated.
	if (const UEdGraphPin* VarPin = FindPin(GetVarName()))
	{
		if (UStruct* PinTypeStruct = Cast<UStruct>(VarPin->PinType.PinSubCategoryObject.Get()))
		{
			UClass* PinTypeClass = Cast<UClass>(PinTypeStruct);
			if (!PinTypeStruct->IsNative() && (!PinTypeClass || PinTypeClass->ClassGeneratedBy != SourceBlueprint))
			{
				bResult = true;
				if (OptionalOutput)
				{
					OptionalOutput->AddUnique(PinTypeStruct);
				}
			}
		}
	}
	
	const bool bSuperResult = Super::HasExternalDependencies(OptionalOutput);
	return bSuperResult || bResult;
}

FString UK2Node_Variable::GetDocumentationLink() const
{
	if( FProperty* Property = GetPropertyForVariable() )
	{
		// discover if the variable property is a non blueprint user variable
		UClass* SourceClass = Property->GetOwnerClass();
		if( SourceClass && SourceClass->ClassGeneratedBy == NULL )
		{
			UStruct* OwnerStruct = Property->GetOwnerStruct();

			if( OwnerStruct )
			{
				return FString::Printf( TEXT("Shared/Types/%s%s"), OwnerStruct->GetPrefixCPP(), *OwnerStruct->GetName() );
			}
		}
	}
	return TEXT( "" );
}

FString UK2Node_Variable::GetDocumentationExcerptName() const
{
	return GetVarNameString();
}

void UK2Node_Variable::AutowireNewNode(UEdGraphPin* FromPin)
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());

	// Do some auto-connection
	if (FromPin != NULL)
	{
		bool bConnected = false;
		if(FromPin->Direction == EGPD_Output)
		{
			// If the source pin has a valid PinSubCategoryObject, we might be doing BP Comms, so check if it is a class
			if(FromPin->PinType.PinSubCategoryObject.IsValid() && FromPin->PinType.PinSubCategoryObject->IsA(UClass::StaticClass()))
			{
				FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode());
				if(VariableProperty)
				{
					UClass* PropertyOwner = VariableProperty->GetOwnerClass();
					if (PropertyOwner != nullptr)
					{
						PropertyOwner = PropertyOwner->GetAuthoritativeClass();
					}

					// BP Comms is highly likely at this point, if the source pin's type is a child of the variable's owner class, let's conform the "Target" pin
					if(FromPin->PinType.PinSubCategoryObject == PropertyOwner || dynamic_cast<UClass*>(FromPin->PinType.PinSubCategoryObject.Get())->IsChildOf(PropertyOwner))
					{
						UEdGraphPin* TargetPin = FindPin(UEdGraphSchema_K2::PN_Self);
						if (TargetPin)
						{
							TargetPin->PinType.PinSubCategoryObject = PropertyOwner;

							if(K2Schema->TryCreateConnection(FromPin, TargetPin))
							{
								bConnected = true;

								// Setup the VariableReference correctly since it may no longer be a self member
								VariableReference.SetFromField<FProperty>(GetPropertyForVariable(), false);
								TargetPin->bHidden = false;
								FromPin->GetOwningNode()->NodeConnectionListChanged();
								this->NodeConnectionListChanged();
							}
						}
					}
				}
			}
		}

		if(!bConnected)
		{
			Super::AutowireNewNode(FromPin);
		}
	}
}

FBPVariableDescription const* UK2Node_Variable::GetBlueprintVarDescription() const
{
	FName const& VarName = VariableReference.GetMemberName();
	UStruct const* VariableScope = VariableReference.GetMemberScope(GetBlueprintClassFromNode());

	bool const bIsLocalVariable = (VariableScope != nullptr);
	if (bIsLocalVariable)
	{
		return FBlueprintEditorUtils::FindLocalVariable(GetBlueprint(), VariableScope, VarName);
	}
	else if (FProperty const* VarProperty = GetPropertyForVariable())
	{
		UClass const* SourceClass = VarProperty->GetOwnerClass();
		UBlueprint const* SourceBlueprint = (SourceClass != nullptr) ? Cast<UBlueprint>(SourceClass->ClassGeneratedBy) : nullptr;

		if (SourceBlueprint != nullptr)
		{
			int32 const VarIndex = FBlueprintEditorUtils::FindNewVariableIndex(SourceBlueprint, VarName);
			return &SourceBlueprint->NewVariables[VarIndex];
		}
	}
	return nullptr;
}

bool UK2Node_Variable::CanPasteHere(const UEdGraph* TargetGraph) const
{
	// Do not allow pasting of variables in BPs that cannot handle them
	if ( FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph)->BlueprintType == BPTYPE_MacroLibrary && VariableReference.IsSelfContext() )
	{
		// Self variables must be from a parent class to the macro BP
		if(FProperty* Property = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
		{
			const UClass* CurrentClass = GetBlueprint()->SkeletonGeneratedClass->GetAuthoritativeClass();
			const UClass* PropertyClass = Property->GetOwnerClass()->GetAuthoritativeClass();
			const bool bIsChildOf = CurrentClass && CurrentClass->IsChildOf(PropertyClass);
			return bIsChildOf;
		}
		return false;
	}
	return true;
}

void UK2Node_Variable::PostPasteNode()
{
	Super::PostPasteNode();

	UBlueprint* Blueprint = GetBlueprint();
	bool bInvalidateVariable = false;

	if (VariableReference.ResolveMember<FProperty>(Blueprint) == nullptr)
	{
		bInvalidateVariable = true;
	}
	else if (VariableReference.IsLocalScope())
	{
		// Local scoped variables should always validate whether they are being placed in the same graph as their scope
		// ResolveMember will not return nullptr when the graph changes but the Blueprint remains the same.
		const UStruct* MemberScope = VariableReference.GetMemberScope(GetBlueprintClassFromNode());
		UEdGraph* ScopeGraph = FBlueprintEditorUtils::FindScopeGraph(Blueprint, MemberScope);
		const bool bMemberScopeInvalid = (ScopeGraph && MemberScope && ScopeGraph->GetFName() != MemberScope->GetFName());
		if(ScopeGraph != GetGraph() || bMemberScopeInvalid)
		{
			bInvalidateVariable = true;
		}
	}
		
	if (bInvalidateVariable)
	{
		// This invalidates the local scope
		VariableReference.InvalidateScope();

		// If the current graph is a Function graph, look to see if there is a compatible local variable (same name)
		if (GetGraph()->GetSchema()->GetGraphType(GetGraph()) == GT_Function)
		{
			UEdGraph* FunctionGraph = FBlueprintEditorUtils::GetTopLevelGraph(GetGraph());
			FBPVariableDescription* VariableDescription = FBlueprintEditorUtils::FindLocalVariable(Blueprint, FunctionGraph, VariableReference.GetMemberName());
			bool bFoundParam = FunctionParameterExists(FunctionGraph, VariableReference.GetMemberName());
			if(VariableDescription || bFoundParam)
			{
				VariableReference.SetLocalMember(VariableReference.GetMemberName(), FunctionGraph->GetName(), VariableReference.GetMemberGuid());
			}
		}
		// If no variable was found, ResolveMember should automatically find a member variable with the same name in the current Blueprint and hook up to it as expected
	}
}

bool UK2Node_Variable::HasDeprecatedReference() const
{
	bool bDeprecated = false;
	FProperty* VariableProperty = nullptr; // Declare up here so we can reuse if we would have resolved twice

	// Check if the referenced variable is deprecated.
	if (VariableReference.IsDeprecated())
	{
		bDeprecated = true;
	}
	else
	{
		VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode());
		if (VariableProperty)
		{
			// Backcompat: Allow variables tagged only with 'DeprecationMessage' meta to be seen as deprecated if inherited from a native parent class.
			const bool bHasDeprecationMessage = VariableProperty->HasMetaData(FBlueprintMetadata::MD_DeprecationMessage);
			if (bHasDeprecationMessage && VariableProperty->GetOwnerUObject()->IsNative())
			{
				bDeprecated = true;
			}
		}
	}

	if (bDeprecated)
	{
		const UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetDefault<UBlueprintEditorProjectSettings>();
		if (!VariableProperty)
		{
			VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode());
		}


		if (VariableProperty)
		{
			const FString PathName = VariableProperty->GetPathName();
			if (BlueprintEditorProjectSettings->SuppressedDeprecationMessages.Contains(PathName))
			{
				bDeprecated = false;
			}
		}
	}

	return bDeprecated;
}

FEdGraphNodeDeprecationResponse UK2Node_Variable::GetDeprecationResponse(EEdGraphNodeDeprecationType DeprecationType) const
{
	FEdGraphNodeDeprecationResponse Response = Super::GetDeprecationResponse(DeprecationType);
	if (DeprecationType == EEdGraphNodeDeprecationType::NodeHasDeprecatedReference)
	{
		if (FProperty* VariableProperty = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
		{
			// Check the deprecation type to override the severity
			FString MessageType = VariableProperty->GetMetaData(FBlueprintMetadata::MD_DeprecatedProperty);
			Response.MessageType = FBlueprintEditorUtils::GetDeprecatedMessageType(MessageType);

			FText MemberName = FText::FromName(VariableReference.GetMemberName());
			FText DetailedMessage = FText::FromString(VariableProperty->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage));
			Response.MessageText = FBlueprintEditorUtils::GetDeprecatedMemberUsageNodeWarning(MemberName, DetailedMessage);
		}
	}

	return Response;
}

UObject* UK2Node_Variable::GetJumpTargetForDoubleClick() const
{
	// Jump to the RepNotify function graph if one exists
	UBlueprint* OwnerBlueprint = GetBlueprint();

	FName RepNotifyFunc = FBlueprintEditorUtils::GetBlueprintVariableRepNotifyFunc(OwnerBlueprint, GetVarName());
	if (RepNotifyFunc != NAME_None)
	{
		for (UEdGraph* Graph : OwnerBlueprint->FunctionGraphs)
		{
			if (Graph->GetFName() == RepNotifyFunc)
			{
				return Graph;
			}
		}
	}

	return nullptr;
}

bool UK2Node_Variable::CanJumpToDefinition() const
{
	const FProperty* VariableProperty = GetPropertyForVariable();
	const bool bNativeVariable = (VariableProperty != nullptr) && (VariableProperty->IsNative());
	const bool bCanJumpToNativeVariable = bNativeVariable && ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed();
	return bCanJumpToNativeVariable || (GetJumpTargetForDoubleClick() != nullptr);
}

void UK2Node_Variable::JumpToDefinition() const
{
	if (ensure(GUnrealEd) && GUnrealEd->GetUnrealEdOptions()->IsCPPAllowed())
	{
	// For native variables, try going to the variable definition in C++ if available
	if (FProperty* VariableProperty = GetPropertyForVariable())
	{
		if (VariableProperty->IsNative())
		{
			FSourceCodeNavigation::NavigateToProperty(VariableProperty);
			return;
		}
	}
	}

	// Otherwise, fall back to the inherited behavior
	Super::JumpToDefinition();
}

void UK2Node_Variable::GetNodeContextMenuActions(class UToolMenu* Menu, class UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	if (HasDeprecatedReference())
	{
		FText MenuEntryTitle = LOCTEXT("SuppressVariableDeprecationWarningTitle", "Suppress Deprecation Warning");
		FText MenuEntryTooltip = LOCTEXT("SuppressVariableDeprecationWarningTooltip", "Adds this variable to the suppressed deprecation warnings list in the Bluperint Editor Project Settings for this project.");

		FToolMenuSection& Section = Menu->AddSection("K2NodeVariable", LOCTEXT("VariableHeader", "Variable"));
		Section.AddMenuEntry(
			"SuppressDeprecationWarning",
			MenuEntryTitle,
			MenuEntryTooltip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateUObject(this, &UK2Node_Variable::SuppressDeprecationWarning),
				FCanExecuteAction::CreateUObject(this, &UK2Node_Variable::HasDeprecatedReference),
				FIsActionChecked()
			)
		);
	}
}

bool UK2Node_Variable::FunctionParameterExists(const UEdGraph* InFunctionGraph, const FName InParameterName)
{
	TArray<UK2Node_FunctionEntry*> Entry;
	InFunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);

	if (ensureMsgf(Entry.Num() == 1, TEXT("Couldn't find a Function Entry node in graph %s"), *InFunctionGraph->GetName()))
	{
		check(Entry[0]);

		return Entry[0]->UserDefinedPinExists(InParameterName);
	}

	return false;
}

const UActorComponent* UK2Node_Variable::GetActorComponent(const FProperty* VariableProperty) const
{
	if (!VariableProperty)
	{
		return nullptr;
	}

	UBlueprint* OwnerBlueprint = GetBlueprint();
	if(OwnerBlueprint && OwnerBlueprint->SimpleConstructionScript)
	{
		if (const AActor* EditorActorInstance = OwnerBlueprint->SimpleConstructionScript->GetComponentEditorActorInstance())
		{
			for (const UActorComponent* Component : EditorActorInstance->GetComponents())
			{
				if (!Component || Component->GetFName() != VariableProperty->GetFName())
				{
					continue;
				}
				return Component;
			}
		}
	}
	
	return nullptr;
}

void UK2Node_Variable::SuppressDeprecationWarning() const
{
	if (const FProperty* Property = VariableReference.ResolveMember<FProperty>(GetBlueprintClassFromNode()))
	{
		FString PathName = Property->GetPathName();
		UBlueprintEditorProjectSettings* BlueprintEditorProjectSettings = GetMutableDefault<UBlueprintEditorProjectSettings>();
		BlueprintEditorProjectSettings->SuppressedDeprecationMessages.Add(MoveTemp(PathName));
		BlueprintEditorProjectSettings->SaveConfig();
		BlueprintEditorProjectSettings->TryUpdateDefaultConfigFile("", false);
	}
}

#undef LOCTEXT_NAMESPACE
