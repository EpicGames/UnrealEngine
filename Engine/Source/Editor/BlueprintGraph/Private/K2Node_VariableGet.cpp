// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_VariableGet.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "Framework/Commands/UIAction.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_IfThenElse.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "KismetCompilerMisc.h"
#include "Misc/AssertionMacros.h"
#include "ScopedTransaction.h"
#include "Serialization/Archive.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_VariableGet

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_VariableGet)

#define LOCTEXT_NAMESPACE "K2Node"

class FKCHandler_VariableGet : public FNodeHandlingFunctor
{
public:
	FKCHandler_VariableGet(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override
	{
		// This net is a variable read
		ResolveAndRegisterScopedTerm(Context, Net, Context.VariableReferences);
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_Variable* VarNode = Cast<UK2Node_Variable>(Node);
		if (VarNode)
		{
			VarNode->CheckForErrors(CompilerContext.GetSchema(), Context.MessageLog);

			// Report an error that the local variable could not be found
			if(VarNode->VariableReference.IsLocalScope() && VarNode->GetPropertyForVariable() == NULL)
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("VariableName"), FText::FromName(VarNode->VariableReference.GetMemberName()));

				if(VarNode->VariableReference.GetMemberScopeName() != Context.Function->GetName())
				{
					Args.Add(TEXT("ScopeName"), FText::FromString(VarNode->VariableReference.GetMemberScopeName()));
					CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("LocalVariableNotFoundInScope_Error", "Unable to find local variable with name '{VariableName}' for @@, scope expected: @@, scope found: {ScopeName}"), Args).ToString(), Node, Node->GetGraph());
				}
				else
				{
					CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("LocalVariableNotFound_Error", "Unable to find local variable with name '{VariableName}' for @@"), Args).ToString(), Node);
				}
			}
		}

		FNodeHandlingFunctor::RegisterNets(Context, Node);
	}
};

namespace K2Node_VariableGetImpl
{
	/**
	 * Shared utility method for retrieving a UK2Node_VariableGet's bare tooltip.
	 * 
	 * @param  VarName	The name of the variable that the node represents.
	 * @return A formatted text string, describing what the VariableGet node does.
	 */
	static FText GetBaseTooltip(FName VarName)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("VarName"), FText::FromName(VarName));

		return FText::Format(LOCTEXT("GetVariableTooltip", "Read the value of variable {VarName}"), Args);
	}

	static EGetNodeVariation GetNodeVariation(const FEdGraphPinType& InPinType)
	{
		EGetNodeVariation Result = EGetNodeVariation::Pure;

		if (!InPinType.IsContainer())
		{
			if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Boolean)
			{
				Result = EGetNodeVariation::Branch;
			}
			else if (InPinType.PinCategory == UEdGraphSchema_K2::PC_Object || InPinType.PinCategory == UEdGraphSchema_K2::PC_Class || InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject || InPinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
			{
				Result = EGetNodeVariation::ValidatedObject;
			}
		}

		return Result;
	}
}

UK2Node_VariableGet::UK2Node_VariableGet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CurrentVariation(EGetNodeVariation::Pure)
{
}

void UK2Node_VariableGet::SetPurity(bool bNewPurity)
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	const UEdGraphPin* ValuePin = GetValuePin();
	const EGetNodeVariation SupportedVariation = K2Node_VariableGetImpl::GetNodeVariation(ValuePin->PinType);
	const EGetNodeVariation DesiredVariation = bNewPurity ? EGetNodeVariation::Pure : SupportedVariation;

	if (CurrentVariation != DesiredVariation)
	{
		TogglePurity(DesiredVariation);
	}
}

void UK2Node_VariableGet::CreateImpurePins(TArray<UEdGraphPin*>* InOldPinsPtr)
{
	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	if (!K2Schema->DoesGraphSupportImpureFunctions(GetGraph()))
	{
		CurrentVariation = EGetNodeVariation::Pure;
	}

	if (CurrentVariation != EGetNodeVariation::Pure)
	{
		FEdGraphPinType PinType;
		const FProperty* VariableProperty = GetPropertyForVariable();

		// We need the pin's type, to both see if it's an array and if it is of the correct types to remain an impure node
		if (VariableProperty)
		{
			K2Schema->ConvertPropertyToPinType(VariableProperty, PinType);
		}
		// If there is no property and we are given some old pins to look at, find the old value pin and use the type there
		// This allows nodes to be pasted into other BPs without access to the property
		else if (InOldPinsPtr)
		{
			// find old variable pin and use the type.
			const FName PinName = GetVarName();
			for (const UEdGraphPin* Pin : *InOldPinsPtr)
			{
				if (Pin && PinName == Pin->PinName)
				{
					PinType = Pin->PinType;
					break;
				}
			}
		}

		const EGetNodeVariation SupportedVariation = K2Node_VariableGetImpl::GetNodeVariation(PinType);
		if (SupportedVariation == EGetNodeVariation::Branch)
		{
			// Input - Execution Pin
			CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

			// Output - Execution Pins
			UEdGraphPin* ValidPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
			check(ValidPin);
			ValidPin->PinFriendlyName = LOCTEXT("True", "True");

			UEdGraphPin* InvalidPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Else);
			check(InvalidPin);
			InvalidPin->PinFriendlyName = LOCTEXT("False", "False");
		}
		else if (SupportedVariation == EGetNodeVariation::ValidatedObject)
		{
			// Input - Execution Pin
			CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);

			// Output - Execution Pins
			UEdGraphPin* ValidPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);
			check(ValidPin);
			ValidPin->PinFriendlyName = LOCTEXT("Valid", "Is Valid");

			UEdGraphPin* InvalidPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Else);
			check(InvalidPin);
			InvalidPin->PinFriendlyName = LOCTEXT("Invalid", "Is Not Valid");
		}
		
		// Note that the type can changed independently of a manual toggle (eg: changing the variable's type)
		CurrentVariation = SupportedVariation;
	}
}

void UK2Node_VariableGet::AllocateDefaultPins()
{
	if (GetVarName() != NAME_None)
	{
		CreateImpurePins(nullptr);

		if (CreatePinForVariable(EGPD_Output))
		{
			CreatePinForSelf();
		}
	}

	Super::AllocateDefaultPins();
}

void UK2Node_VariableGet::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	if (GetVarName() != NAME_None)
	{
		CreateImpurePins(&OldPins);

		if (!CreatePinForVariable(EGPD_Output))
		{
			if (!RecreatePinForVariable(EGPD_Output, OldPins))
			{
				return;
			}
		}
		CreatePinForSelf();

		RestoreSplitPins(OldPins);
	}
}

FText UK2Node_VariableGet::GetPropertyTooltip(FProperty const* VariableProperty)
{
	FName VarName = NAME_None;
	if (VariableProperty != nullptr)
	{
		VarName = VariableProperty->GetFName();

		UClass* SourceClass = VariableProperty->GetOwnerClass();
		// discover if the variable property is a non blueprint user variable
		bool const bIsNativeVariable = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy == nullptr);

		FText SubTooltip;
		if (bIsNativeVariable)
		{
			FText const PropertyTooltip = VariableProperty->GetToolTipText();
			if (!PropertyTooltip.IsEmpty())
			{
				// See if the native property has a tooltip
				SubTooltip = PropertyTooltip;
				FString TooltipName = FString::Printf(TEXT("%s.%s"), *VarName.ToString(), *FBlueprintMetadata::MD_Tooltip.ToString());
				FText::FindTextInLiveTable_Advanced(*VariableProperty->GetFullGroupName(true), *TooltipName, SubTooltip);
			}
		}
		else if (SourceClass)
		{
			if (UBlueprint* VarBlueprint = Cast<UBlueprint>(SourceClass->ClassGeneratedBy))
			{
				FString UserTooltipData;
				if (FBlueprintEditorUtils::GetBlueprintVariableMetaData(VarBlueprint, VarName, VariableProperty->GetOwnerStruct(), FBlueprintMetadata::MD_Tooltip, UserTooltipData))
				{
					SubTooltip = FText::FromString(UserTooltipData);
				}
			}
		}

		if (!SubTooltip.IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("VarName"), FText::FromName(VarName));
			Args.Add(TEXT("PropertyTooltip"), SubTooltip);

			return FText::Format(LOCTEXT("GetVariableProperty_Tooltip", "Read the value of variable {VarName}\n{PropertyTooltip}"), Args);
		}
	}
	return K2Node_VariableGetImpl::GetBaseTooltip(VarName);
}

FText UK2Node_VariableGet::GetBlueprintVarTooltip(FBPVariableDescription const& VarDesc)
{
	int32 const MetaIndex = VarDesc.FindMetaDataEntryIndexForKey(FBlueprintMetadata::MD_Tooltip);
	bool const bHasTooltipData = (MetaIndex != INDEX_NONE);

	if (bHasTooltipData)
	{
		FString UserTooltipData = VarDesc.GetMetaData(FBlueprintMetadata::MD_Tooltip);

		FFormatNamedArguments Args;
		Args.Add(TEXT("VarName"), FText::FromName(VarDesc.VarName));
		Args.Add(TEXT("UserTooltip"), FText::FromString(UserTooltipData));

		return FText::Format(LOCTEXT("GetBlueprintVariable_Tooltip", "Read the value of variable {VarName}\n{UserTooltip}"), Args);
	}
	return K2Node_VariableGetImpl::GetBaseTooltip(VarDesc.VarName);
}

FText UK2Node_VariableGet::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		if (FProperty* Property = GetPropertyForVariable())
		{
			CachedTooltip.SetCachedText(GetPropertyTooltip(Property), this);
		}
		else if (FBPVariableDescription const* VarDesc = GetBlueprintVarDescription())
		{
			CachedTooltip.SetCachedText(GetBlueprintVarTooltip(*VarDesc), this);
		}
		else
		{
			CachedTooltip.SetCachedText(K2Node_VariableGetImpl::GetBaseTooltip(GetVarName()), this);
		}
	}
	return CachedTooltip;
}

FText UK2Node_VariableGet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	// If there is only one variable being read, the title can be made the variable name
	FName OutputPinName;
	int32 NumOutputsFound = 0;

	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];

		if (ensure(Pin) && Pin->Direction == EGPD_Output)
		{
			++NumOutputsFound;
			OutputPinName = Pin->PinName;
		}
	}

	if (NumOutputsFound != 1)
	{
		return LOCTEXT("Get", "Get");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinName"), FText::FromName(OutputPinName));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(LOCTEXT("GetPinName", "Get {PinName}"), Args), this);
	}
	return CachedNodeTitle;
}

FNodeHandlingFunctor* UK2Node_VariableGet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_VariableGet(CompilerContext);
}

void UK2Node_VariableGet::GetNodeContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	Super::GetNodeContextMenuActions(Menu, Context);

	check(Menu);
	check(Context);

	const UEdGraphSchema_K2* K2Schema = CastChecked<UEdGraphSchema_K2>(GetSchema());
	const UEdGraphPin* ValuePin = GetValuePin();
	
	if (ValuePin)
	{
		const EGetNodeVariation SupportedVariation = K2Node_VariableGetImpl::GetNodeVariation(ValuePin->PinType);
		const bool bCanShowToggleVariationMenu =
			K2Schema->DoesGraphSupportImpureFunctions(GetGraph()) &&
			(SupportedVariation != EGetNodeVariation::Pure) &&
			!Context->bIsDebugging
		;

		if (bCanShowToggleVariationMenu)
		{
			FText MenuEntryTitle;
			FText MenuEntryTooltip;

			if (CurrentVariation == EGetNodeVariation::Pure)
			{
				if (SupportedVariation == EGetNodeVariation::ValidatedObject)
				{
					MenuEntryTitle = LOCTEXT("ConvertToImpureGetObjectTitle", "Convert to Validated Get");
					MenuEntryTooltip = LOCTEXT("ConvertToImpureGetObjectTooltip", "Adds in branching execution pins so that you can separately handle when the returned value is valid/invalid.");
				}
				else if (SupportedVariation == EGetNodeVariation::Branch)
				{
					MenuEntryTitle = LOCTEXT("ConvertToImpureGetBooleanTitle", "Convert to Branch");
					MenuEntryTooltip = LOCTEXT("ConvertToImpureGetBooleanTooltip", "Adds in branching execution pins so that you can separately handle when the returned value is true/false.");
				}
			}
			else
			{
				MenuEntryTitle = LOCTEXT("ConvertToPureGetTitle", "Convert to pure Get");
				MenuEntryTooltip = LOCTEXT("ConvertToPureGetTooltip", "Removes the execution pins to make the node more versatile.");
			}

			FToolMenuSection& Section = Menu->AddSection("K2NodeVariableGet", LOCTEXT("VariableGetHeader", "Variable Get"));
			Section.AddMenuEntry(
				"TogglePurity",
				MenuEntryTitle,
				MenuEntryTooltip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateUObject(const_cast<UK2Node_VariableGet*>(this), &UK2Node_VariableGet::TogglePurity, SupportedVariation),
					FCanExecuteAction(),
					FIsActionChecked()
				)
			);
		}
	}
}

void UK2Node_VariableGet::TogglePurity(EGetNodeVariation BoundVariation)
{
	FText TransactionTitle;

	EGetNodeVariation PendingVariation = EGetNodeVariation::Pure;
	if (CurrentVariation == EGetNodeVariation::Pure)
	{
		if (BoundVariation == EGetNodeVariation::ValidatedObject)
		{
			TransactionTitle = LOCTEXT("ToggleImpureGetObject", "Convert to Validated Get");
		}
		else if (BoundVariation == EGetNodeVariation::Branch)
		{
			TransactionTitle = LOCTEXT("ToggleImpureGetBoolean", "Convert to Branch");
		}
		else
		{
			check(false);
		}

		PendingVariation = BoundVariation;
	}
	else
	{
		TransactionTitle = LOCTEXT("TogglePureGet", "Convert to Pure Get");
		PendingVariation = EGetNodeVariation::Pure;
	}

	const FScopedTransaction Transaction(TransactionTitle);
	Modify();
	CurrentVariation = PendingVariation;
	ReconstructNode();
}

void UK2Node_VariableGet::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	// Some expansions, such as timelines, will create gets for non-blueprint visible properties, and we don't want to validate against that
	if (IsIntermediateNode())
	{
		return;
	}

	// The validation below does not apply to local variables; they are always readable within their context.
	if (VariableReference.IsLocalScope())
	{
		return;
	}

	if (FProperty* Property = GetPropertyForVariable())
	{
		const FBlueprintEditorUtils::EPropertyReadableState PropertyReadableState = FBlueprintEditorUtils::IsPropertyReadableInBlueprint(GetBlueprint(), Property);

		if (PropertyReadableState != FBlueprintEditorUtils::EPropertyReadableState::Readable)
		{
			FFormatNamedArguments Args;
			if (UObject* Class = Property->GetOwner<UObject>())
			{
				Args.Add(TEXT("VariableName"), FText::AsCultureInvariant(FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Property->GetName())));
			}
			else
			{
				Args.Add(TEXT("VariableName"), FText::AsCultureInvariant(Property->GetName()));
			}

			if (PropertyReadableState == FBlueprintEditorUtils::EPropertyReadableState::NotBlueprintVisible)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("UnableToGet_NotVisible", "{VariableName} is not blueprint visible (BlueprintReadOnly or BlueprintReadWrite). Please fix mark up or cease accessing as this will be made an error in a future release. @@"), Args).ToString(), this);
			}
			else if (PropertyReadableState == FBlueprintEditorUtils::EPropertyReadableState::Private)
			{
				MessageLog.Error(*FText::Format(LOCTEXT("UnableToGet_ReadOnly", "{VariableName} is private and not accessible in this context. Please fix mark up or cease accessing as this will be an error in a future release. @@"), Args).ToString(), this);
			}
			else
			{
				check(false);
			}
		}
	}
}

void UK2Node_VariableGet::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	FProperty* VariableProperty = GetPropertyForVariable();

	// Do not attempt to expand the node when not a pure get nor when there is no property. Normal compilation error detection will detect the missing property.
	if ((CurrentVariation != EGetNodeVariation::Pure) && VariableProperty)
	{
		UEdGraphPin* ValuePin = GetValuePin();
		check(ValuePin);

		// Impure nodes need 2-3 intermediate nodes depending on the variation.
		// 
		// For validated objects, we need:
		// 1. A pure Get node
		// 2. An IsValid node
		// 3. A Branch node (only impure part)
		//
		// For branches, we only need:
		// 1. A pure Get node
		// 2. A Branch node (only impure part)

		// Create the pure Get node
		UK2Node_VariableGet* VariableGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, SourceGraph);
		VariableGetNode->VariableReference = VariableReference;
		VariableGetNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(VariableGetNode, this);

		// Move pin links from Get node we are expanding, to the new pure one we've created
		CompilerContext.MovePinLinksToIntermediate(*ValuePin, *VariableGetNode->GetValuePin());
		if (!VariableReference.IsLocalScope())
		{
			CompilerContext.MovePinLinksToIntermediate(*FindPin(UEdGraphSchema_K2::PN_Self), *VariableGetNode->FindPin(UEdGraphSchema_K2::PN_Self));
		}

		// By default, we'll assume that this is the branch variation.
		// Otherwise, the validated object variation will change the source pin.
		UEdGraphPin* SourceBoolPin = VariableGetNode->GetValuePin();

		if (CurrentVariation == EGetNodeVariation::ValidatedObject)
		{
			// Create the IsValid node
			UK2Node_CallFunction* IsValidFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);

			// Based on if the type is an "Object" or a "Class" changes which function to use
			if (ValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Object)
			{
				IsValidFunction->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, IsValid)));
			}
			else if (ValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_Class)
			{
				IsValidFunction->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, IsValidClass)));
			}
			else if (ValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftObject)
			{
				IsValidFunction->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, IsValidSoftObjectReference)));
			}
			else if (ValuePin->PinType.PinCategory == UEdGraphSchema_K2::PC_SoftClass)
			{
				IsValidFunction->SetFromFunction(UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_MEMBER_NAME_CHECKED(UKismetSystemLibrary, IsValidSoftClassReference)));
			}
			IsValidFunction->AllocateDefaultPins();
			CompilerContext.MessageLog.NotifyIntermediateObjectCreation(IsValidFunction, this);

			// Connect the value pin from the new Get node to the IsValid
			UEdGraphPin* ObjectPin = IsValidFunction->Pins[1];
			check(ObjectPin->Direction == EGPD_Input);
			ObjectPin->MakeLinkTo(VariableGetNode->GetValuePin());
			SourceBoolPin = IsValidFunction->Pins[2];
		}

		// Create the Branch node
		UK2Node_IfThenElse* BranchNode = CompilerContext.SpawnIntermediateNode<UK2Node_IfThenElse>(this, SourceGraph);
		BranchNode->AllocateDefaultPins();
		CompilerContext.MessageLog.NotifyIntermediateObjectCreation(BranchNode, this);

		// Connect the bool output pin from IsValid node to the Branch node
		check(SourceBoolPin->Direction == EGPD_Output);
		SourceBoolPin->MakeLinkTo(BranchNode->GetConditionPin());

		// Connect the Branch node to the input of the impure Get node
		CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *BranchNode->GetExecPin());

		// Move the two Branch pins to the Branch node
		CompilerContext.MovePinLinksToIntermediate(*FindPin(UEdGraphSchema_K2::PN_Then), *BranchNode->FindPin(UEdGraphSchema_K2::PN_Then));
		CompilerContext.MovePinLinksToIntermediate(*FindPin(UEdGraphSchema_K2::PN_Else), *BranchNode->FindPin(UEdGraphSchema_K2::PN_Else));

		BreakAllNodeLinks();
	}

	// If property has a BlueprintGetter accessor, then replace the variable get node with a call function
	if (VariableProperty)
	{
		const FString& GetFunctionName = VariableProperty->GetMetaData(FBlueprintMetadata::MD_PropertyGetFunction);
		if (!GetFunctionName.IsEmpty())
		{
			UClass* OwnerClass = VariableProperty->GetOwnerClass();
			UFunction* GetFunction = OwnerClass->FindFunctionByName(*GetFunctionName);
			if (!GetFunction)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("MissingGetter", "Getter function not found for @@").ToString(), this);
				return;
			}

			UK2Node_CallFunction* CallFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
			CallFuncNode->SetFromFunction(GetFunction);
			CallFuncNode->AllocateDefaultPins();

			const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

			// Move Self pin connections
			CompilerContext.MovePinLinksToIntermediate(*K2Schema->FindSelfPin(*this, EGPD_Input), *K2Schema->FindSelfPin(*CallFuncNode, EGPD_Input));

			// Move Value pin connections
			CompilerContext.MovePinLinksToIntermediate(*GetValuePin(), *CallFuncNode->GetReturnValuePin());
		}
	}
}

void UK2Node_VariableGet::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Prior to the addition of CurrentVariation, bIsPureGet implied a 'validate object' variation.
	// We also reset bIsPureGet to its default value to prevent triggering this data migration path again.
	if (!bIsPureGet_DEPRECATED && Ar.IsLoading())
	{
		CurrentVariation = EGetNodeVariation::ValidatedObject;
		bIsPureGet_DEPRECATED = true;
	}
}

#undef LOCTEXT_NAMESPACE
