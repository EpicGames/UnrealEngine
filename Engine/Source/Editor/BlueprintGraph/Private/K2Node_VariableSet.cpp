// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_VariableSet.h"

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "VariableSetHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_VariableSet)

#define LOCTEXT_NAMESPACE "K2Node_VariableSet"

namespace K2Node_VariableSetImpl
{
	/**
	 * Shared utility method for retrieving a UK2Node_VariableSet's bare tooltip.
	 * 
	 * @param  VarName	The name of the variable that the node represents.
	 * @return A formatted text string, describing what the VariableSet node does.
	 */
	static FText GetBaseTooltip(FName VarName);

	/**
	 * Returns true if the specified variable RepNotify AND is defined in a 
	 * blueprint. Most (all?) native rep notifies are intended to be client 
	 * only. We are moving away from this paradigm in blueprints. So for now 
	 * this is somewhat of a hold over to avoid nasty bugs where a K2 set node 
	 * is calling a native function that the designer has no idea what it is 
	 * doing.
	 * 
	 * @param  VariableProperty	The variable property you wish to check.
	 * @return True if the specified variable RepNotify AND is defined in a blueprint.
	 */
	static bool PropertyHasLocalRepNotify(FProperty const* VariableProperty);
}

static FText K2Node_VariableSetImpl::GetBaseTooltip(FName VarName)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("VarName"), FText::FromName(VarName));

	return FText::Format(LOCTEXT("SetVariableTooltip", "Set the value of variable {VarName}"), Args);

}

static bool K2Node_VariableSetImpl::PropertyHasLocalRepNotify(FProperty const* VariableProperty)
{
	if (VariableProperty != nullptr)
	{
		// We check that the variable is 'defined in a blueprint' so as to avoid 
		// natively defined RepNotifies being called unintentionally. Most(all?) 
		// native rep notifies are intended to be client only. We are moving 
		// away from this paradigm in blueprints. So for now this is somewhat of 
		// a hold over to avoid nasty bugs where a K2 set node is calling a 
		// native function that the designer has no idea what it is doing.
		UBlueprintGeneratedClass* VariableSourceClass = Cast<UBlueprintGeneratedClass>(VariableProperty->GetOwnerClass());
		bool const bIsBlueprintProperty = (VariableSourceClass != nullptr);

		if (bIsBlueprintProperty && (VariableProperty->RepNotifyFunc != NAME_None))
		{
			// Find function (ok if its defined in native class)
			UFunction* Function = VariableSourceClass->FindFunctionByName(VariableProperty->RepNotifyFunc);

			// If valid repnotify func
			if ((Function != nullptr) && (Function->NumParms == 0) && (Function->GetReturnProperty() == nullptr))
			{
				return true;
			}
		}
	}
	return false;
}

UK2Node_VariableSet::UK2Node_VariableSet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_VariableSet::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	if (GetVarName() != NAME_None)
	{
		if(CreatePinForVariable(EGPD_Input))
		{
			CreatePinForSelf();
		}

		if(CreatePinForVariable(EGPD_Output, GetVariableOutputPinName()))
		{
			CreateOutputPinTooltip();
		}
	}

	Super::AllocateDefaultPins();
}

void UK2Node_VariableSet::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	if (GetVarName() != NAME_None)
	{
		if(!CreatePinForVariable(EGPD_Input))
		{
			if(!RecreatePinForVariable(EGPD_Input, OldPins))
			{
				return;
			}
		}

		if(!CreatePinForVariable(EGPD_Output, GetVariableOutputPinName()))
		{
			if(!RecreatePinForVariable(EGPD_Output, OldPins, GetVariableOutputPinName()))
			{
				return;
			}
		}
		CreateOutputPinTooltip();
		CreatePinForSelf();
	}

	RestoreSplitPins(OldPins);
}



FText UK2Node_VariableSet::GetPropertyTooltip(FProperty const* VariableProperty)
{
	FText TextFormat;
	FFormatNamedArguments Args;

	FName VarName = NAME_None;
	if (VariableProperty != nullptr)
	{
		VarName = VariableProperty->GetFName();

		UClass* SourceClass = VariableProperty->GetOwnerClass();
		// discover if the variable property is a non blueprint user variable
		bool const bIsNativeVariable = (SourceClass != nullptr) && (SourceClass->ClassGeneratedBy == nullptr);

		FText SubTooltip;
		if (bIsNativeVariable)
		{
			FText const PropertyTooltip = VariableProperty->GetToolTipText();
			if (!PropertyTooltip.IsEmpty())
			{
				// See if the native property has a tooltip
				SubTooltip = PropertyTooltip;
				FString TooltipName = FString::Printf(TEXT("%s.%s"), *VarName.ToString(), *FBlueprintMetadata::MD_Tooltip.ToString());
				FText::FindTextInLiveTable_Advanced(*VariableProperty->GetFullGroupName(true), *TooltipName, SubTooltip);
			}
		}
		else if (SourceClass)
		{
			if (UBlueprint* VarBlueprint = Cast<UBlueprint>(SourceClass->ClassGeneratedBy))
			{
				FString UserTooltipData;
				if (FBlueprintEditorUtils::GetBlueprintVariableMetaData(VarBlueprint, VarName, VariableProperty->GetOwnerStruct(), FBlueprintMetadata::MD_Tooltip, UserTooltipData))
				{
					SubTooltip = FText::FromString(UserTooltipData);
				}
			}
		}

		bool const bHasLocalRepNotify = K2Node_VariableSetImpl::PropertyHasLocalRepNotify(VariableProperty);
		bool const bFieldNotify = VariableProperty->HasMetaData(FBlueprintMetadata::MD_FieldNotify);
		if (bHasLocalRepNotify)
		{
			Args.Add(TEXT("ReplicationNotifyName"), FText::FromName(VariableProperty->RepNotifyFunc));
		}
		if (!SubTooltip.IsEmpty())
		{
			Args.Add(TEXT("PropertyTooltip"), SubTooltip);
			if (bHasLocalRepNotify && bFieldNotify)
			{
				TextFormat = LOCTEXT("SetVariablePropertyWithRepNotifyWithBroadcast_Tooltip", "Set the value of variable {VarName} and call {ReplicationNotifyName} with broadcast.\n{PropertyTooltip}");
			}
			else if (bHasLocalRepNotify)
			{
				TextFormat = LOCTEXT("SetVariablePropertyWithRepNotify_Tooltip", "Set the value of variable {VarName} and call {ReplicationNotifyName}\n{PropertyTooltip}");
			}
			else if (bFieldNotify)
			{
				TextFormat = LOCTEXT("SetVariablePropertyWithBroadcast_Tooltip", "Set the value of variable {VarName} and broadcast.\n{PropertyTooltip}");
			}
			else
			{
				TextFormat = LOCTEXT("SetVariableProperty_Tooltip", "Set the value of variable {VarName}\n{PropertyTooltip}");
			}
		}
		else if (bHasLocalRepNotify && bFieldNotify)
		{
			TextFormat = LOCTEXT("SetVariableWithRepNotifyWithBroadcast_Tooltip", "Set the value of variable {VarName} and call {ReplicationNotifyName} with broadcast.");
		}
		else if (bHasLocalRepNotify)
		{
			TextFormat = LOCTEXT("SetVariableWithRepNotify_Tooltip", "Set the value of variable {VarName} and call {ReplicationNotifyName}");
		}
		else if (bFieldNotify)
		{
			TextFormat = LOCTEXT("SetVariableWithBroadcast_Tooltip", "Set the value of variable {VarName} and broadcast");
		}
	}

	if (TextFormat.IsEmpty())
	{
		return K2Node_VariableSetImpl::GetBaseTooltip(VarName);
	}
	else
	{
		Args.Add(TEXT("VarName"), FText::FromName(VarName));
		return FText::Format(TextFormat, Args);
	}
}

FText UK2Node_VariableSet::GetBlueprintVarTooltip(FBPVariableDescription const& VarDesc)
{
	int32 const MetaIndex = VarDesc.FindMetaDataEntryIndexForKey(FBlueprintMetadata::MD_Tooltip);
	bool const bHasTooltipData = (MetaIndex != INDEX_NONE);

	if (bHasTooltipData)
	{
		FString UserTooltipData = VarDesc.GetMetaData(FBlueprintMetadata::MD_Tooltip);

		FFormatNamedArguments Args;
		Args.Add(TEXT("VarName"), FText::FromName(VarDesc.VarName));
		Args.Add(TEXT("UserTooltip"), FText::FromString(UserTooltipData));

		return FText::Format(LOCTEXT("SetBlueprintVariable_Tooltip", "Set the value of variable {VarName}\n{UserTooltip}"), Args);
	}
	return K2Node_VariableSetImpl::GetBaseTooltip(VarDesc.VarName);
}

FText UK2Node_VariableSet::GetTooltipText() const
{
	if (CachedTooltip.IsOutOfDate(this))
	{
		if (FProperty* Property = GetPropertyForVariable())
		{
			CachedTooltip.SetCachedText(GetPropertyTooltip(Property), this);
		}
		else if (FBPVariableDescription const* VarDesc = GetBlueprintVarDescription())
		{
			CachedTooltip.SetCachedText(GetBlueprintVarTooltip(*VarDesc), this);
		}
		else
		{
			CachedTooltip.SetCachedText(K2Node_VariableSetImpl::GetBaseTooltip(GetVarName()), this);
		}
	}
	return CachedTooltip;
}

FText UK2Node_VariableSet::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// If there is only one variable being written (one non-meta input pin), the title can be made the variable name
	FName InputPinName;
	int32 NumInputsFound = 0;

	for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Pins[PinIndex];
		if ((Pin->Direction == EGPD_Input) && (!K2Schema->IsMetaPin(*Pin)))
		{
			++NumInputsFound;
			InputPinName = Pin->PinName;
		}
	}

	const bool bHasLocalRepNotify = HasLocalRepNotify();
	const bool bIsFieldNotify = IsFieldNotifyProperty();
	if (NumInputsFound != 1)
	{
		if (bHasLocalRepNotify && bIsFieldNotify)
		{
			return NSLOCTEXT("K2Node", "SetWithNotifyWithBroadcast", "Set with Notify and Broadcast");
		}
		else if (bHasLocalRepNotify)
		{
			return NSLOCTEXT("K2Node", "SetWithNotify", "Set with Notify");
		}
		else if (bIsFieldNotify)
		{
			return NSLOCTEXT("K2Node", "SetWithBroadcast", "Set with Broadcast");
		}
		return NSLOCTEXT("K2Node", "Set", "Set");
	}

	// @TODO: The variable name mutates as the user makes changes to the 
	//        underlying property, so until we can catch all those cases, we're
	//        going to leave this optimization off
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinName"), FText::FromName(InputPinName));

		// FText::Format() is slow, so we cache this to save on performance
		if (bHasLocalRepNotify && bIsFieldNotify)
		{
			CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SetWithNotifyWithBroadcastPinName", "Set with Notify and Broadcast {PinName}"), Args), this);
		}
		else if (bHasLocalRepNotify)
		{
			CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SetWithNotifyPinName", "Set with Notify {PinName}"), Args), this);
		}
		else if (bIsFieldNotify)
		{
			CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SetWithBroadcastPinName", "Set with Broadcast {PinName}"), Args), this);
		}
		else
		{
			CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SetPinName", "Set {PinName}"), Args), this);
		}
	}
	return CachedNodeTitle;
}

/** Returns true if the variable we are setting has a RepNotify AND was defined in a blueprint
 *		The 'defined in a blueprint' is to avoid natively defined RepNotifies being called unintentionally.
 *		Most (all?) native rep notifies are intended to be client only. We are moving away from this paradigm in blueprints
 *		So for now this is somewhat of a hold over to avoid nasty bugs where a K2 set node is calling a native function that the
 *		designer has no idea what it is doing.
 */
bool UK2Node_VariableSet::HasLocalRepNotify() const
{
	return K2Node_VariableSetImpl::PropertyHasLocalRepNotify(GetPropertyForVariable());
}

bool UK2Node_VariableSet::ShouldFlushDormancyOnSet() const
{
	const UClass* VariableSourceClass = GetVariableSourceClass();
	if (VariableSourceClass == nullptr || !VariableSourceClass->IsChildOf(AActor::StaticClass()))
	{
		return false;
	}

	// Flush net dormancy before setting a replicated property
	FProperty* Property = FindFProperty<FProperty>(VariableSourceClass, GetVarName());
	return (Property && (Property->PropertyFlags & CPF_Net));
}

bool UK2Node_VariableSet::IsNetProperty() const
{
	FProperty* Property = GetPropertyForVariable();
	return Property && (Property->PropertyFlags & CPF_Net);
}

FName UK2Node_VariableSet::GetRepNotifyName() const
{
	if (FProperty* Property = GetPropertyForVariable())
	{
		return Property->RepNotifyFunc;
	}
	return NAME_None;
}

bool UK2Node_VariableSet::IsFieldNotifyProperty() const
{
	FProperty* Property = GetPropertyForVariable();
	return Property ? Property->HasMetaData(FBlueprintMetadata::MD_FieldNotify) : false;
}

bool UK2Node_VariableSet::HasFieldNotificationBroadcast() const
{
	FProperty* Property = GetPropertyForVariable();
	return Property ? FKismetCompilerUtilities::IsPropertyUsesFieldNotificationSetValueAndBroadcast(Property) : false;
}

FNodeHandlingFunctor* UK2Node_VariableSet::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_VariableSet(CompilerContext);
}

FName UK2Node_VariableSet::GetVariableOutputPinName() const
{
	return TEXT("Output_Get");
}

void UK2Node_VariableSet::CreateOutputPinTooltip()
{
	UEdGraphPin* Pin = FindPin(GetVariableOutputPinName());
	check(Pin);
	Pin->PinToolTip = NSLOCTEXT("K2Node", "SetPinOutputTooltip", "Retrieves the value of the variable, can use instead of a separate Get node").ToString();
}

FText UK2Node_VariableSet::GetPinNameOverride(const UEdGraphPin& Pin) const
{
	// Stop the output pin for the variable, effectively the "get" pin, from displaying a name.
	if( !Pin.bOrphanedPin && 
		Pin.ParentPin == nullptr && 
		(Pin.Direction == EGPD_Output || Pin.PinType.PinCategory == UEdGraphSchema_K2::PC_Exec))
	{
		return FText::GetEmpty();
	}

	return !Pin.PinFriendlyName.IsEmpty() ? Pin.PinFriendlyName : FText::FromName(Pin.PinName);
}

void UK2Node_VariableSet::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	// Some expansions will create sets for non-blueprint visible properties, and we don't want to validate against that
	if (!IsIntermediateNode())
	{
		if (FProperty* Property = GetPropertyForVariable())
		{
			const FBlueprintEditorUtils::EPropertyWritableState PropertyWritableState = FBlueprintEditorUtils::IsPropertyWritableInBlueprint(GetBlueprint(), Property);

			if (PropertyWritableState != FBlueprintEditorUtils::EPropertyWritableState::Writable)
			{
				FFormatNamedArguments Args;
				if (UObject* Class = Property->GetOwner<UObject>())
				{
					Args.Add(TEXT("VariableName"), FText::AsCultureInvariant(FString::Printf(TEXT("%s.%s"), *Class->GetName(), *Property->GetName())));
				}
				else
				{
					Args.Add(TEXT("VariableName"), FText::AsCultureInvariant(Property->GetName()));
				}

				if (PropertyWritableState == FBlueprintEditorUtils::EPropertyWritableState::BlueprintReadOnly || PropertyWritableState == FBlueprintEditorUtils::EPropertyWritableState::NotBlueprintVisible)
				{
					MessageLog.Error(*FText::Format(LOCTEXT("UnableToSet_NotWritable", "{VariableName} is not blueprint writable. @@"), Args).ToString(), this);
				}
				else if (PropertyWritableState == FBlueprintEditorUtils::EPropertyWritableState::Private)
				{
					MessageLog.Error(*LOCTEXT("UnableToSet_ReadOnly", "{VariableName} is private and not accessible in this context. @@").ToString(), this);
				}
				else
				{
					check(false);
				}
			}
		}
	}
}

void UK2Node_VariableSet::ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	if (CompilerContext.bIsFullCompile)
	{
		FProperty* VariableProperty = GetPropertyForVariable();

		const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

		if (UEdGraphPin* VariableGetPin = FindPin(GetVariableOutputPinName()))
		{
			// If the output pin is linked, we need to spawn a separate "Get" node and hook it up.
			if (VariableGetPin->LinkedTo.Num())
			{
				if (VariableProperty)
				{
					UK2Node_VariableGet* VariableGetNode = CompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, SourceGraph);
					VariableGetNode->VariableReference = VariableReference;
					VariableGetNode->AllocateDefaultPins();
					CompilerContext.MessageLog.NotifyIntermediateObjectCreation(VariableGetNode, this);
					CompilerContext.MovePinLinksToIntermediate(*VariableGetPin, *VariableGetNode->FindPin(GetVarName()));

					// Duplicate the connection to the self pin.
					UEdGraphPin* SetSelfPin = K2Schema->FindSelfPin(*this, EGPD_Input);
					UEdGraphPin* GetSelfPin = K2Schema->FindSelfPin(*VariableGetNode, EGPD_Input);
					if (SetSelfPin && GetSelfPin)
					{
						CompilerContext.CopyPinLinksToIntermediate(*SetSelfPin, *GetSelfPin);
					}
				}
			}
			Pins.Remove(VariableGetPin);
			VariableGetPin->MarkAsGarbage();
		}

		// If property has a BlueprintSetter accessor, then replace the variable get node with a call function
		if (VariableProperty)
		{
			// todo check with BP team if we need to test if the variable has native Setter
			const FString& SetFunctionName = VariableProperty->GetMetaData(FBlueprintMetadata::MD_PropertySetFunction);
			if (!SetFunctionName.IsEmpty())
			{
				UClass* OwnerClass = VariableProperty->GetOwnerClass();
				UFunction* SetFunction = OwnerClass->FindFunctionByName(*SetFunctionName);
				if (!SetFunction)
				{
					CompilerContext.MessageLog.Error(*LOCTEXT("MissingSetter", "Setter function not found for @@").ToString(), this);
					return;
				}

				UK2Node_CallFunction* CallFuncNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
				CallFuncNode->SetFromFunction(SetFunction);
				CallFuncNode->AllocateDefaultPins();

				// Move Exec pin connections
				CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *CallFuncNode->GetExecPin());

				// Move Then pin connections
				CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Then, EGPD_Output), *CallFuncNode->GetThenPin());
				
				// Move Self pin connections
				if (UEdGraphPin* SetSelfPin = K2Schema->FindSelfPin(*this, EGPD_Input))
				{
					CompilerContext.MovePinLinksToIntermediate(*SetSelfPin, *K2Schema->FindSelfPin(*CallFuncNode, EGPD_Input));
				}

				// Move Value pin connections
				UEdGraphPin* SetFunctionValuePin = nullptr;
				for (UEdGraphPin* CallFuncPin : CallFuncNode->Pins)
				{
					if (!K2Schema->IsMetaPin(*CallFuncPin))
					{
						check(CallFuncPin->Direction == EGPD_Input);
						SetFunctionValuePin = CallFuncPin;
						break;
					}
				}
				check(SetFunctionValuePin);

				CompilerContext.MovePinLinksToIntermediate(*FindPin(GetVarName(), EGPD_Input), *SetFunctionValuePin);
			}
			else if (HasFieldNotificationBroadcast())
			{
				TTuple<UEdGraphPin*, UEdGraphPin*> ExecThenPins = FKismetCompilerUtilities::GenerateFieldNotificationSetNode(CompilerContext, SourceGraph, this, FindPinChecked(UEdGraphSchema_K2::PN_Self), VariableProperty, VariableReference, HasLocalRepNotify(), ShouldFlushDormancyOnSet(), IsNetProperty());

				// Move Exec pin connections
				CompilerContext.MovePinLinksToIntermediate(*GetExecPin(), *ExecThenPins.Get<0>());

				// Move Then pin connections
				CompilerContext.MovePinLinksToIntermediate(*FindPinChecked(UEdGraphSchema_K2::PN_Then, EGPD_Output), *ExecThenPins.Get<1>());
			}
		}
	}

}

#undef LOCTEXT_NAMESPACE
