// Copyright Epic Games, Inc. All Rights Reserved.


#include "K2Node_VariableSetRef.h"

#include "BPTerminal.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintActionFilter.h"
#include "BlueprintCompiledStatement.h"
#include "BlueprintNodeSpawner.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "EditorCategoryUtils.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "VariableSetHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_VariableSetRef)

static FName TargetVarPinName(TEXT("Target"));
static FName VarValuePinName(TEXT("Value"));

#define LOCTEXT_NAMESPACE "K2Node_VariableSetRef"

class FKCHandler_VariableSetRef : public FKCHandler_VariableSet
{
public:
	FKCHandler_VariableSetRef(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_VariableSet(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_VariableSetRef* VarRefNode = CastChecked<UK2Node_VariableSetRef>(Node);
		UEdGraphPin* ValuePin = VarRefNode->GetValuePin();
		ValidateAndRegisterNetIfLiteral(Context, ValuePin);

		{
			// The pins on UK2Node_VariableSetRef don't actually refer to storage, so there's nothing to register in the NetMap.
			// However, their pins' nets do, so we need to check if a cast is required for assignment to the target net.
			// The cast detection performed by RegisterImplicitCasts doesn't understand the reference semantics used in this node.
			// As a result, we need to manually handle any implicit casts.

			using namespace UE::KismetCompiler;

			UEdGraphPin* VariablePin = VarRefNode->GetTargetPin();
			UEdGraphPin* VariablePinNet = FEdGraphUtilities::GetNetFromPin(VariablePin);
			UEdGraphPin* ValuePinNet = FEdGraphUtilities::GetNetFromPin(ValuePin);

			if ((VariablePinNet != nullptr) && (ValuePinNet != nullptr))
			{
				CastingUtils::FConversion Conversion =
					CastingUtils::GetFloatingPointConversion(*ValuePinNet, *VariablePinNet);

				if (Conversion.Type != CastingUtils::FloatingPointCastType::None)
				{
					check(!ImplicitCastMap.Contains(VarRefNode));

					FBPTerminal* NewTerminal = CastingUtils::MakeImplicitCastTerminal(Context, VariablePinNet);

					ImplicitCastMap.Add(VarRefNode, CastingUtils::FImplicitCastParams{Conversion, NewTerminal, Node});
				}
			}
		}
	}

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override
	{
		UK2Node_VariableSetRef* VarRefNode = CastChecked<UK2Node_VariableSetRef>(Node);
		UEdGraphPin* VarTargetPin = VarRefNode->GetTargetPin();
		UEdGraphPin* ValuePin = VarRefNode->GetValuePin();

		InnerAssignment(Context, Node, VarTargetPin, ValuePin);

		// Generate the output impulse from this node
		GenerateSimpleThenGoto(Context, *Node);
	}

private:

	void InnerAssignment(FKismetFunctionContext& Context, UEdGraphNode* Node, UEdGraphPin* VariablePin, UEdGraphPin* ValuePin)
	{
		UEdGraphPin* VariablePinNet = FEdGraphUtilities::GetNetFromPin(VariablePin);
		UEdGraphPin* ValuePinNet = FEdGraphUtilities::GetNetFromPin(ValuePin);

		FBPTerminal** VariableTerm = Context.NetMap.Find(VariablePin);
		if (VariableTerm == nullptr)
		{
			VariableTerm = Context.NetMap.Find(VariablePinNet);
		}

		FBPTerminal** ValueTerm = Context.LiteralHackMap.Find(ValuePin);
		if (ValueTerm == nullptr)
		{
			ValueTerm = Context.NetMap.Find(ValuePinNet);
		}

		if ((VariableTerm != nullptr) && (ValueTerm != nullptr))
		{
			FBPTerminal* LHSTerm = *VariableTerm;
			FBPTerminal* RHSTerm = *ValueTerm;

			{
				using namespace UE::KismetCompiler;

				UK2Node_VariableSetRef* VarRefNode = CastChecked<UK2Node_VariableSetRef>(Node);
				if (CastingUtils::FImplicitCastParams* CastParams = ImplicitCastMap.Find(VarRefNode))
				{
					CastingUtils::InsertImplicitCastStatement(Context, *CastParams, RHSTerm);
					
					RHSTerm = CastParams->TargetTerminal;

					ImplicitCastMap.Remove(VarRefNode);

					// We've manually registered our cast statement, so it can be removed from the context.
					CastingUtils::RemoveRegisteredImplicitCast(Context, VariablePin);
					CastingUtils::RemoveRegisteredImplicitCast(Context, ValuePin);
				}
			}

			FBlueprintCompiledStatement& Statement = Context.AppendStatementForNode(Node);
			Statement.Type = KCST_Assignment;
			Statement.LHS = LHSTerm;
			Statement.RHS.Add(RHSTerm);

			if (!(*VariableTerm)->IsTermWritable())
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("WriteConst_Error", "Cannot write to const @@").ToString(), VariablePin);
			}
		}
		else
		{
			if (VariablePin != ValuePin)
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("ResolveValueIntoVariablePin_Error", "Failed to resolve term @@ passed into @@").ToString(), ValuePin, VariablePin);
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermPassed_Error", "Failed to resolve term passed into @@").ToString(), VariablePin);
			}
		}
	}

	TMap<UK2Node_VariableSetRef*, UE::KismetCompiler::CastingUtils::FImplicitCastParams> ImplicitCastMap;
};

UK2Node_VariableSetRef::UK2Node_VariableSetRef(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UK2Node_VariableSetRef::AllocateDefaultPins()
{
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Execute);
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Exec, UEdGraphSchema_K2::PN_Then);

	UEdGraphNode::FCreatePinParams PinParams;
	PinParams.bIsReference = true;
	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, TargetVarPinName, PinParams);

	CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Wildcard, VarValuePinName);
}

void UK2Node_VariableSetRef::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	AllocateDefaultPins();

 	// Coerce the type of the node from the old pin, if available
  	UEdGraphPin* OldTargetPin = nullptr;
  	for (UEdGraphPin* CurrPin : OldPins)
  	{
  		if (CurrPin->PinName == TargetVarPinName)
  		{
  			OldTargetPin = CurrPin;
  			break;
  		}
  	}
  
  	if( OldTargetPin )
  	{
  		UEdGraphPin* NewTargetPin = GetTargetPin();
  		CoerceTypeFromPin(OldTargetPin);
  	}
	CachedNodeTitle.MarkDirty();

	RestoreSplitPins(OldPins);
}

FText UK2Node_VariableSetRef::GetTooltipText() const
{
	return NSLOCTEXT("K2Node", "SetValueOfRefVariable", "Set the value of the connected pass-by-ref variable");
}

FText UK2Node_VariableSetRef::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* TargetPin = GetTargetPin();
	if ((TargetPin == nullptr) || (TargetPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard))
	{
		return NSLOCTEXT("K2Node", "SetRefVarNodeTitle", "Set By-Ref Var");
	}
	else if (CachedNodeTitle.IsOutOfDate(this))
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("PinType"), Schema->TypeToText(TargetPin->PinType));
		// FText::Format() is slow, so we cache this to save on performance
		CachedNodeTitle.SetCachedText(FText::Format(NSLOCTEXT("K2Node", "SetRefVarNodeTitle_Typed", "Set {PinType}"), Args), this);
	}
	return CachedNodeTitle;
}

bool UK2Node_VariableSetRef::IsActionFilteredOut(class FBlueprintActionFilter const& Filter)
{
	// Default to filtering this node out unless dragging off of a reference output pin
	bool bIsFilteredOut = false;
	FBlueprintActionContext const& FilterContext = Filter.Context;

	for (UEdGraphPin* Pin : FilterContext.Pins)
	{
		if(Pin->Direction == EGPD_Output && Pin->PinType.bIsReference == true)
		{
			bIsFilteredOut = false;
			break;
		}
	}
	return bIsFilteredOut;
}

void UK2Node_VariableSetRef::NotifyPinConnectionListChanged(UEdGraphPin* Pin)
{
	Super::NotifyPinConnectionListChanged(Pin);

	UEdGraphPin* TargetPin = GetTargetPin();
	UEdGraphPin* ValuePin = GetValuePin();

	if( (Pin == TargetPin) || (Pin == ValuePin) )
	{
		UEdGraphPin* ConnectedToPin = (Pin->LinkedTo.Num() > 0) ? Pin->LinkedTo[0] : nullptr;
		CoerceTypeFromPin(ConnectedToPin);

		// If both target and value pins are unlinked, then reset types to wildcard
		if(TargetPin->LinkedTo.Num() == 0 && ValuePin->LinkedTo.Num() == 0)
		{
			// collapse SubPins back into their parent if there are any
			auto TryRecombineSubPins = [](UEdGraphPin* ParentPin)
			{
				if (!ParentPin->SubPins.IsEmpty())
				{
					const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
					K2Schema->RecombinePin(ParentPin->SubPins[0]);
				}
			};
			
			// Pin disconnected...revert to wildcard
			TargetPin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			TargetPin->PinType.PinSubCategory = NAME_None;
			TargetPin->PinType.PinSubCategoryObject = nullptr;
			TargetPin->BreakAllPinLinks();
			TryRecombineSubPins(TargetPin);

			ValuePin->PinType.PinCategory = UEdGraphSchema_K2::PC_Wildcard;
			ValuePin->PinType.PinSubCategory = NAME_None;
			ValuePin->PinType.PinSubCategoryObject = nullptr;
			ValuePin->BreakAllPinLinks();
			TryRecombineSubPins(ValuePin);
		}

		CachedNodeTitle.MarkDirty();
		
		// Get the graph to refresh our title and default value info
		GetGraph()->NotifyNodeChanged(this);
	}
}

void UK2Node_VariableSetRef::CoerceTypeFromPin(const UEdGraphPin* Pin)
{
	UEdGraphPin* TargetPin = GetTargetPin();
	UEdGraphPin* ValuePin = GetValuePin();

	check(TargetPin && ValuePin);

	if( Pin && 
		(Pin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard ||
		(	Pin->PinType.PinCategory == TargetPin->PinType.PinCategory &&
			Pin->PinType.PinCategory == ValuePin->PinType.PinCategory )
		) )
	{
		check((Pin != TargetPin) || (Pin->PinType.bIsReference && !Pin->PinType.IsContainer()));

		TargetPin->PinType = Pin->PinType;
		TargetPin->PinType.bIsReference = true;

		ValuePin->PinType = Pin->PinType;
		ValuePin->PinType.bIsReference = false;
	}
}

UEdGraphPin* UK2Node_VariableSetRef::GetTargetPin() const
{
	return FindPin(TargetVarPinName);
}

UEdGraphPin* UK2Node_VariableSetRef::GetValuePin() const
{
	return FindPin(VarValuePinName);
}

FNodeHandlingFunctor* UK2Node_VariableSetRef::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FKCHandler_VariableSetRef(CompilerContext);
}

void UK2Node_VariableSetRef::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	// actions get registered under specific object-keys; the idea is that 
	// actions might have to be updated (or deleted) if their object-key is  
	// mutated (or removed)... here we use the node's class (so if the node 
	// type disappears, then the action should go with it)
	UClass* ActionKey = GetClass();
	// to keep from needlessly instantiating a UBlueprintNodeSpawner, first   
	// check to make sure that the registrar is looking for actions of this type
	// (could be regenerating actions for a specific asset, and therefore the 
	// registrar would only accept actions corresponding to that asset)
	if (ActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}

FText UK2Node_VariableSetRef::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Variables);
}


#undef LOCTEXT_NAMESPACE
