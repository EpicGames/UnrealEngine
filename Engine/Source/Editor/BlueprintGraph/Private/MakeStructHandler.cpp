// Copyright Epic Games, Inc. All Rights Reserved.

#include "MakeStructHandler.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node.h"
#include "K2Node_MakeStruct.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define LOCTEXT_NAMESPACE "FKCHandler_MakeStruct"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_MakeStruct

UEdGraphPin* FKCHandler_MakeStruct::FindStructPinChecked(UEdGraphNode* Node) const
{
	check(Node);
	UEdGraphPin* OutputPin = nullptr;
	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (Pin && (EGPD_Output == Pin->Direction) && !CompilerContext.GetSchema()->IsMetaPin(*Pin))
		{
			OutputPin = Pin;
			break;
		}
	}
	check(OutputPin);
	return OutputPin;
}

FKCHandler_MakeStruct::FKCHandler_MakeStruct(FKismetCompilerContext& InCompilerContext)
	: FNodeHandlingFunctor(InCompilerContext)
	, bAutoGenerateGotoForPure(true)
{
}

void FKCHandler_MakeStruct::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode)
{
	UK2Node_MakeStruct* Node = CastChecked<UK2Node_MakeStruct>(InNode);
	if (nullptr == Node->StructType)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_UnknownStructure_Error", "Unknown structure to break for @@").ToString(), Node);
		return;
	}
	
	if (!UK2Node_MakeStruct::CanBeMade(Node->StructType, Node->IsIntermediateNode()))
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_Error", "The structure @@ is not a BlueprintType. ").ToString(), Node);
		return;
	}
	return RegisterNetsImpl(Context, Node);
}
	
void FKCHandler_MakeStruct::RegisterNetsImpl(FKismetFunctionContext& Context, UK2Node_MakeStruct* Node)
{
	FNodeHandlingFunctor::RegisterNets(Context, Node);

	UEdGraphPin* OutputPin = FindStructPinChecked(Node);
	UEdGraphPin* Net = FEdGraphUtilities::GetNetFromPin(OutputPin);
	check(Net);
	FBPTerminal** FoundTerm = Context.NetMap.Find(Net);
	FBPTerminal* Term = FoundTerm ? *FoundTerm : nullptr;

	if (Term == nullptr)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_NoTerm_Error", "Failed to generate a term for the @@ pin; was it a struct reference that was left unset?").ToString(), OutputPin);
	}
	else
	{
		UStruct* StructInTerm = Cast<UStruct>(Term->Type.PinSubCategoryObject.Get());
		if (nullptr == StructInTerm || !StructInTerm->IsChildOf(Node->StructType))
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_NoMatch_Error", "Structures don't match for @@").ToString(), Node);
		}
	}
}
	

void FKCHandler_MakeStruct::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	if (!Net->bDefaultValueIsIgnored)
	{
		FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, Context.NetNameMap->MakeValidName(Net));
		Context.NetMap.Add(Net, Term);
	}
}

FBPTerminal* FKCHandler_MakeStruct::RegisterLiteral(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	FBPTerminal* ReturnTerm = nullptr;
	if (!Net->bDefaultValueIsIgnored)
	{
		ReturnTerm = FNodeHandlingFunctor::RegisterLiteral(Context, Net);
	}
	return ReturnTerm;
}

void FKCHandler_MakeStruct::Compile(FKismetFunctionContext& Context, UEdGraphNode* InNode)
{
	UK2Node_MakeStruct* Node = CastChecked<UK2Node_MakeStruct>(InNode);
	if (NULL == Node->StructType)
	{
		CompilerContext.MessageLog.Error(*LOCTEXT("MakeStruct_UnknownStructure_Error", "Unknown structure to break for @@").ToString(), Node);
		return;
	}

	UEdGraphPin* StructPin = FindStructPinChecked(Node);
	UEdGraphPin* OutputStructNet = FEdGraphUtilities::GetNetFromPin(StructPin);
	FBPTerminal** FoundTerm = Context.NetMap.Find(OutputStructNet);
	FBPTerminal* OutputStructTerm = FoundTerm ? *FoundTerm : NULL;
	check(OutputStructTerm);

	// A set of edit condition properties that should be set to signal an override state
	// when the condition is not exposed as a unique input alongside any bound properties.
	TMap<FProperty*, bool> OverrideProperties;

	const UEdGraphSchema_K2* Schema = CompilerContext.GetSchema();

	for (UEdGraphPin* Pin : Node->Pins)
	{
		if (Pin && !Pin->bOrphanedPin && (Pin != StructPin) && (Pin->Direction == EGPD_Input) && !Schema->IsMetaPin(*Pin))
		{
			FProperty* BoundProperty = FindFProperty<FProperty>(Node->StructType, Pin->PinName);
			check(BoundProperty);

			// If the pin is not connectible, do not forward the net
			if (!Pin->bNotConnectable)
			{
				if (FBPTerminal** FoundSrcTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(Pin)))
				{
					FBPTerminal* SrcTerm = FoundSrcTerm ? *FoundSrcTerm : nullptr;
					check(SrcTerm);

					FBPTerminal* DstTerm = Context.CreateLocalTerminal();
					DstTerm->CopyFromPin(Pin, Context.NetNameMap->MakeValidName(Pin));
					DstTerm->AssociatedVarProperty = BoundProperty;
					DstTerm->Context = OutputStructTerm;

					FKismetCompilerUtilities::CreateObjectAssignmentStatement(Context, Node, SrcTerm, DstTerm);
				}
			}

			// Determine if the property bound to the input pin has an associated override flag that's linked via
			// an edit condition. If we find an override property and it's not included in the optional pin set,
			// it signals that we need to implicitly set it at runtime since the value bound to it is exposed as
			// an input. This convention allows us to roughly emulate the Property Editor which injects the override
			// flag's value when the user modifies/exposes the bound property's value as an override at edit time.
			// 
			// Note: This API returns a Boolean flag type only. Other condition types are not supported since the
			// "override" concept involves a direct association with a Boolean edit condition. Override properties
			// must also be visible to the BP runtime as well as writable, or the terms used in the implicit
			// assignment statement below may otherwise fail to compile.
			FBoolProperty* OverrideProperty = Node->GetOverrideConditionForProperty(BoundProperty);
			if (OverrideProperty && OverrideProperty->HasAllPropertyFlags(CPF_BlueprintVisible) && !OverrideProperty->HasAllPropertyFlags(CPF_BlueprintReadOnly))
			{
				// Need to dig up what the state of the override property should be
				const FOptionalPinFromProperty* BoundPropertyEntryPtr = Node->ShowPinForProperties.FindByPredicate([BoundProperty](const FOptionalPinFromProperty& PropertyEntry)
				{
					// If we are showing the pin, then we are overriding the property
					return PropertyEntry.bHasOverridePin && PropertyEntry.bShowPin && PropertyEntry.PropertyName == BoundProperty->GetFName();
				});

				if (BoundPropertyEntryPtr)
				{
					const FOptionalPinFromProperty& PropertyEntry = *BoundPropertyEntryPtr;

					if (!PropertyEntry.bIsOverridePinVisible || (PropertyEntry.bIsOverridePinVisible && !PropertyEntry.bIsOverrideEnabled && PropertyEntry.bIsSetValuePinVisible))
					{
						CompilerContext.MessageLog.Warning(*LOCTEXT("MakeStruct_InvalidOverrideSetting", "Selected override setting on @@ in @@ is no longer a supported workflow and it is advised that you refactor your Blueprint to not use it!").ToString(), Pin, Node->GetBlueprint());
					}

					// This flag is a bit of a misnomer - it's generally true, except for assets saved prior to the convention described above, in which case the pin
					// may have been explicitly hidden by the user, and thus this flag would be false to indicate that. For backwards-compatibility, we don't want to
					// implicitly set the value in that case, because it means the user had explicitly opted not to show it, which implies it was not previously set.
					if (PropertyEntry.bIsOverridePinVisible)
					{
						// Note: bIsOverrideEnabled is generally true, but may be false for older assets that were saved with an exposed override pin with a default
						// value prior to the change that excludes all override properties from the optional input pin set.
						OverrideProperties.Add(OverrideProperty, PropertyEntry.bIsOverrideEnabled);
					}
				}
			}
		}
	}

	// Handle injecting the override property values into the node
	TArray<FBlueprintCompiledStatement*>& StatementList = Context.StatementsPerNode.FindOrAdd(Node);
	for (auto OverridePropIt = OverrideProperties.CreateConstIterator(); OverridePropIt; ++OverridePropIt)
	{
		FProperty* OverrideProperty = OverridePropIt.Key();
		const bool bIsOverrideEnabled = OverridePropIt.Value();

		FEdGraphPinType PinType;
		Schema->ConvertPropertyToPinType(OverrideProperty, /*out*/ PinType);

		// Create the term in the list
		FBPTerminal* OverrideTerm = new FBPTerminal();
		Context.VariableReferences.Add(OverrideTerm);
		OverrideTerm->Type = PinType;
		OverrideTerm->AssociatedVarProperty = OverrideProperty;
		OverrideTerm->Context = OutputStructTerm;

		FBlueprintCompiledStatement* AssignBoolStatement = new FBlueprintCompiledStatement();
		AssignBoolStatement->Type = KCST_Assignment;

		// Literal Bool Term to set the OverrideProperty to
		FBPTerminal* BoolTerm = Context.CreateLocalTerminal(ETerminalSpecification::TS_Literal);
		BoolTerm->Type.PinCategory = UEdGraphSchema_K2::PC_Boolean;
		BoolTerm->bIsLiteral = true;
		BoolTerm->Name = bIsOverrideEnabled ? TEXT("true") : TEXT("false");

		// Assigning the OverrideProperty to the literal bool term
		AssignBoolStatement->LHS = OverrideTerm;
		AssignBoolStatement->RHS.Add(BoolTerm);

		Context.AllGeneratedStatements.Add(AssignBoolStatement);
		StatementList.Add(AssignBoolStatement);
	}

	if (bAutoGenerateGotoForPure && !Node->IsNodePure())
	{
		GenerateSimpleThenGoto(Context, *Node);
	}
}

#undef LOCTEXT_NAMESPACE
