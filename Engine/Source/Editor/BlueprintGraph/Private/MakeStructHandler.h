// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetCompilerMisc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UK2Node_MakeStruct;
class UEdGraphPin;
struct FBPTerminal;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_MakeStruct

class FKCHandler_MakeStruct : public FNodeHandlingFunctor
{
public:
	FKCHandler_MakeStruct(FKismetCompilerContext& InCompilerContext);

	virtual UEdGraphPin* FindStructPinChecked(UEdGraphNode* Node) const;

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override;
	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	void RegisterNetsImpl(FKismetFunctionContext& Context, UK2Node_MakeStruct* Node);

	virtual FBPTerminal* RegisterLiteral(FKismetFunctionContext& Context, UEdGraphPin* Net) override;

	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* InNode) override;

protected:
	/** Prevents this handler from auto-generating a GoTo to jump to the end of the node's compiled code so that child handlers can override the functionality */
	bool bAutoGenerateGotoForPure;
};
