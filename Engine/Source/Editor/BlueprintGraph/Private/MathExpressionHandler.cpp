// Copyright Epic Games, Inc. All Rights Reserved.

#include "MathExpressionHandler.h"

#include "BPTerminal.h"
#include "BlueprintCompiledStatement.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MathExpression.h"
#include "K2Node_Tunnel.h"
#include "K2Node_VariableGet.h"
#include "Kismet/KismetMathLibrary.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCastingUtils.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "KCHandler_MathExpression"

struct KCHandler_MathExpressionHelper
{
	static bool CreateMap(UEdGraphNode& KeyNode, UEdGraphNode& ValueNode, EEdGraphPinDirection KeyPinDirection, const UEdGraphSchema_K2* Schema, TMap<UEdGraphPin*, UEdGraphPin*>& OutMap)
	{
		check(Schema);
		for (UEdGraphPin* InnerInputPin : KeyNode.Pins)
		{
			if (!InnerInputPin || InnerInputPin->Direction != KeyPinDirection)
			{
				return false;
			}

			UEdGraphPin* OuterInputPin = ValueNode.FindPin(InnerInputPin->PinName);
			if (!OuterInputPin
				|| !Schema->ArePinsCompatible(InnerInputPin, OuterInputPin, nullptr, false))
			{
				return false;
			}
			OutMap.Add(InnerInputPin, OuterInputPin);
		}
		return true;
	}
};

bool FKCHandler_MathExpression::CanBeCalledByMathExpression(const UFunction* Function)
{
	bool bGoodFunction = (nullptr != Function)
		&& Function->HasAllFunctionFlags(FUNC_Static | FUNC_BlueprintPure | FUNC_Final | FUNC_Native)
		&& !Function->HasAnyFunctionFlags(FUNC_BlueprintAuthorityOnly | FUNC_BlueprintCosmetic)
		&& !Function->GetOuterUClass()->IsChildOf(UInterface::StaticClass());
	if (!bGoodFunction)
	{
		return false;
	}

	for (TFieldIterator<const FProperty> It(Function); It && (It->PropertyFlags & CPF_Parm); ++It)
	{
		const FProperty* Property = *It;
		const bool bGoodProperty = Property && (!Property->HasAnyPropertyFlags(CPF_OutParm) || Property->HasAllPropertyFlags(CPF_ReturnParm));
		if (!bGoodProperty)
		{
			return false;
		}
	}
	return true;
}

FBlueprintCompiledStatement* FKCHandler_MathExpression::GenerateFunctionRPN(UEdGraphNode* CurrentNode, FKismetFunctionContext& Context, UK2Node_MathExpression& MENode, FBPTerminal* ResultTerm, TMap<UEdGraphPin*, UEdGraphPin*>& InnerToOuterInput)
{
	UK2Node_CallFunction* CallFunctionNode = Cast<UK2Node_CallFunction>(CurrentNode);
	UFunction* Function = CallFunctionNode ? CallFunctionNode->GetTargetFunction() : nullptr;
	if (!CanBeCalledByMathExpression(Function))
	{
		CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("WrongFunction_ErrorFmt", "Function '{0}' cannot be called inside Math Expression @@ - @@"), FText::FromString(GetNameSafe(Function))).ToString(), CallFunctionNode, &MENode);
		return nullptr;
	}

	FBlueprintCompiledStatement* Statement = new FBlueprintCompiledStatement();
	Statement->FunctionToCall = Function;
	Statement->FunctionContext = nullptr;
	Statement->Type = KCST_CallFunction;
	Statement->LHS = ResultTerm; // required only for the first node

	check(CallFunctionNode);

	TArray<FBPTerminal*> RHSTerms;
	for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & CPF_Parm); ++It)
	{
		FProperty* Property = *It;
		if (Property && !Property->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm))
		{
			UEdGraphPin* PinMatch = CallFunctionNode->FindPin(Property->GetFName());
			UEdGraphPin* PinToTry = nullptr;
			{
				const bool bGoodPin = PinMatch && FKismetCompilerUtilities::IsTypeCompatibleWithProperty(PinMatch, Property, CompilerContext.MessageLog, CompilerContext.GetSchema(), Context.NewClass);
				PinToTry = bGoodPin ? FEdGraphUtilities::GetNetFromPin(PinMatch) : nullptr;
			}

			FBPTerminal* RHSTerm = nullptr;
			{
				UEdGraphPin** OuterInputPtr = PinToTry ? InnerToOuterInput.Find(PinToTry) : nullptr;
				UEdGraphPin* OuterInputNet = (OuterInputPtr && *OuterInputPtr) ? FEdGraphUtilities::GetNetFromPin(*OuterInputPtr) : nullptr;
				FBPTerminal** OuterTerm = OuterInputNet ? Context.NetMap.Find(OuterInputNet) : nullptr;
				// Input is an outer term
				if (OuterTerm && *OuterTerm)
				{
					RHSTerm = *OuterTerm;
				}
			}

			if (!RHSTerm)
			{
				FBPTerminal** Term = PinToTry ? Context.NetMap.Find(PinToTry) : nullptr;
				const bool bValidTerm = Term && *Term;
				// Input is a literal term
				// Input is a variable
				if (bValidTerm && ((*Term)->bIsLiteral || (*Term)->AssociatedVarProperty))
				{
					RHSTerm = *Term;
				}
				// Input is an InlineGeneratedParameter
				else if (bValidTerm)
				{
					ensure(!(*Term)->InlineGeneratedParameter);
					UEdGraphNode* SourceNode = PinToTry ? PinToTry->GetOwningNodeUnchecked() : nullptr;
					FBlueprintCompiledStatement* InlineGeneratedParameterStatement = GenerateFunctionRPN(SourceNode, Context, MENode, nullptr, InnerToOuterInput);
					if (InlineGeneratedParameterStatement)
					{
						Context.AllGeneratedStatements.Add(InlineGeneratedParameterStatement);
						RHSTerm = *Term;
						RHSTerm->InlineGeneratedParameter = InlineGeneratedParameterStatement;
					}
				}
			}

			if (RHSTerm)
			{
				using namespace UE::KismetCompiler;

				const CastingUtils::FImplicitCastParams* CastParams =
					Context.ImplicitCastMap.Find(PinMatch);

				if (CastParams)
				{
					check(CastParams->TargetTerminal);

					UFunction* CastFunction = nullptr;

					switch (CastParams->Conversion.Type)
					{
					case CastingUtils::FloatingPointCastType::DoubleToFloat:
						CastFunction = 
							UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Conv_DoubleToFloat));
						break;

					case CastingUtils::FloatingPointCastType::FloatToDouble:
						CastFunction = 
							UKismetMathLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetMathLibrary, Conv_FloatToDouble));
						break;

					default:
						checkf(false, TEXT("Unsupported cast type used in math expression node: %d"), EnumToUnderlyingType(CastParams->Conversion.Type));
					}

					check(CastFunction);

					FBlueprintCompiledStatement* CastStatement = new FBlueprintCompiledStatement();
					CastStatement->FunctionToCall = CastFunction;
					CastStatement->Type = KCST_CallFunction;
					CastStatement->RHS.Add(RHSTerm);

					RHSTerm = CastParams->TargetTerminal;
					CastParams->TargetTerminal->InlineGeneratedParameter = CastStatement;
					Context.AllGeneratedStatements.Add(CastStatement);

					CastingUtils::RemoveRegisteredImplicitCast(Context, PinMatch);
				}

				RHSTerms.Add(RHSTerm);
			}
			else
			{
				CompilerContext.MessageLog.Error(*FText::Format(LOCTEXT("FindPinParameter_ErrorFmt", "Could not find a pin for the parameter {0} of {1} on @@"), FText::FromString(GetNameSafe(Property)), FText::FromString(GetNameSafe(Function))).ToString(), CallFunctionNode);
			}
		}
	}
	Statement->RHS = RHSTerms;
	return Statement;
}

void FKCHandler_MathExpression::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode)
{
	using namespace UE::KismetCompiler;

	FNodeHandlingFunctor::RegisterNets(Context, InNode);

	UK2Node_MathExpression* Node_MathExpression = CastChecked<UK2Node_MathExpression>(InNode);
	UK2Node_Tunnel* InnerEntryNode = Node_MathExpression->GetEntryNode();
	UK2Node_Tunnel* InnerExitNode = Node_MathExpression->GetExitNode();
	check(Node_MathExpression->BoundGraph && InnerEntryNode && InnerExitNode);

	for (UEdGraphNode* InnerGraphNode : Node_MathExpression->BoundGraph->Nodes)
	{
		if (!InnerGraphNode || (InnerGraphNode == InnerEntryNode) || (InnerGraphNode == InnerExitNode))
		{
			continue;
		}

		if (UK2Node_VariableGet* GetVarNode = Cast<UK2Node_VariableGet>(InnerGraphNode))
		{
			FNodeHandlingFunctor* VarHandler = GetVarNode->CreateNodeHandler(CompilerContext);
			if (ensure(VarHandler))
			{
				VarHandler->RegisterNets(Context, GetVarNode);
			}
			continue;
		}

		for (UEdGraphPin* Pin : InnerGraphNode->Pins)
		{
			// Register fake terms for InlineGeneratedValues
			if (Pin && (Pin->Direction == EEdGraphPinDirection::EGPD_Output) && Pin->LinkedTo.Num())
			{
				UEdGraphPin* Linked = Pin->LinkedTo[0];
				UEdGraphNode* LinkedOwnerNode = Linked ? Linked->GetOwningNodeUnchecked() : nullptr;
				if (LinkedOwnerNode && (InnerExitNode != LinkedOwnerNode))
				{
					FBPTerminal* Term = new FBPTerminal();
					Context.InlineGeneratedValues.Add(Term);
					Term->CopyFromPin(Pin, Context.NetNameMap->MakeValidName(Pin));
					Context.NetMap.Add(Pin, Term);
				}
			}

			// Register Literals
			if (Pin && (Pin->Direction == EEdGraphPinDirection::EGPD_Input) && !Pin->LinkedTo.Num())
			{
				RegisterLiteral(Context, Pin);
			}
		}
	}

	// We have two nodes that handle input: the math expression node and the inner entry node.
	// For any pins connected to the entry node, we need to compare their types to the nets
	// of the math expression node. 
	// 
	// This is a fairly convoluted scenario that RegisterImplicitCasts fails to detect,
	// and it only affects pins connected to the inner entry node.

	int EntryNodeCursor = 0;
	for (UEdGraphPin* Pin : Node_MathExpression->Pins)
	{
		if (Pin && (Pin->Direction == EEdGraphPinDirection::EGPD_Input))
		{
			UEdGraphPin* PinNet = FEdGraphUtilities::GetNetFromPin(Pin);
			if (PinNet)
			{
				UEdGraphPin* InnerEntryNodePin = InnerEntryNode->Pins[EntryNodeCursor];
				if (InnerEntryNodePin && (InnerEntryNodePin->GetName() == Pin->GetName()))
				{
					for (UEdGraphPin* DestinationPin : InnerEntryNodePin->LinkedTo)
					{
						if (DestinationPin)
						{
							CastingUtils::FConversion Conversion =
								CastingUtils::GetFloatingPointConversion(*PinNet, *DestinationPin);

							if (Conversion.Type != CastingUtils::FloatingPointCastType::None)
							{
								FBPTerminal* NewTerm = CastingUtils::MakeImplicitCastTerminal(Context, DestinationPin);
								UEdGraphNode* OwningNode = DestinationPin->GetOwningNode();

								Context.ImplicitCastMap.Add(DestinationPin, CastingUtils::FImplicitCastParams{Conversion, NewTerm, OwningNode});
							}
						}
					}
				}
				else
				{
					Context.MessageLog.Error(*LOCTEXT("Compile_PinMismatchError", "ICE - mismatched pins found on @@ and @@!").ToString(), Node_MathExpression, InnerEntryNode);
				}
			}

			++EntryNodeCursor;
		}
	}
}

void FKCHandler_MathExpression::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) 
{
	FBPTerminal* Term = Context.CreateLocalTerminalFromPinAutoChooseScope(Net, *Context.NetNameMap->MakeValidName(Net));
	Context.NetMap.Add(Net, Term);
}

void FKCHandler_MathExpression::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_MathExpression* Node_MathExpression = CastChecked<UK2Node_MathExpression>(Node);
	check(Context.Schema);

	UK2Node_Tunnel* InnerExitNode = Node_MathExpression->GetExitNode();
	UK2Node_Tunnel* InnerEntryNode = Node_MathExpression->GetEntryNode();

	if (!InnerExitNode || !InnerEntryNode || (InnerExitNode->Pins.Num() != 1) || ((InnerExitNode->Pins.Num() + InnerEntryNode->Pins.Num()) != Node->Pins.Num()))
	{
		Context.MessageLog.Error(*LOCTEXT("Compile_WrongInnerPinError", "ICE - wrong inner pins - @@").ToString(), Node);
		return;
	}

	TMap<UEdGraphPin*, UEdGraphPin*> InnerToOuterInput;
	bool bProperMap = KCHandler_MathExpressionHelper::CreateMap(*InnerEntryNode, *Node_MathExpression, EEdGraphPinDirection::EGPD_Output, Context.Schema, InnerToOuterInput);

	TMap<UEdGraphPin*, UEdGraphPin*> InnerToOuterOutput;
	bProperMap &= KCHandler_MathExpressionHelper::CreateMap(*InnerExitNode, *Node_MathExpression, EEdGraphPinDirection::EGPD_Input, Context.Schema, InnerToOuterOutput);

	if (!bProperMap)
	{
		Context.MessageLog.Error(*LOCTEXT("Compile_WrongMap", "ICE - cannot map pins - @@").ToString(), Node);
		return;
	}

	UEdGraphPin* InnerOutputPin = InnerExitNode->Pins[0];
	UEdGraphPin** OuterOutputPinPtr = InnerToOuterOutput.Find(InnerOutputPin);
	if (!InnerOutputPin || (InnerOutputPin->LinkedTo.Num() != 1) || !InnerOutputPin->LinkedTo[0] || !OuterOutputPinPtr || !*OuterOutputPinPtr)
	{
		Context.MessageLog.Error(*LOCTEXT("Compile_WrongOutputLink", "ICE - wrong output link - @@").ToString(), Node);
		return;
	}

	FBPTerminal** OutputTermPtr = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(*OuterOutputPinPtr));
	FBPTerminal* OutputTerm = OutputTermPtr ? *OutputTermPtr : nullptr;
	UEdGraphNode* LastInnerNode = InnerOutputPin->LinkedTo[0]->GetOwningNodeUnchecked();

	FBlueprintCompiledStatement* DetachedStatement = GenerateFunctionRPN(LastInnerNode, Context, *Node_MathExpression, OutputTerm, InnerToOuterInput);
	if (DetachedStatement)
	{
		using namespace UE::KismetCompiler;
	
		Context.AllGeneratedStatements.Add(DetachedStatement);
		TArray<FBlueprintCompiledStatement*>& StatementList = Context.StatementsPerNode.FindOrAdd(Node);
		StatementList.Add(DetachedStatement);

		// The casts that we care about occur *within* the nodes generated by the math expression.
		// The expression node merely serves as a proxy, so we can silently remove the casts
		// on the input pins that were registered by the compiler.

		for (UEdGraphPin* Pin : Node_MathExpression->Pins)
		{
			check(Pin);
			if (!Context.Schema->IsMetaPin(*Pin) && (Pin->Direction == EGPD_Input))
			{
				CastingUtils::RemoveRegisteredImplicitCast(Context, Pin);
			}
		}
	}
	else
	{
		Context.MessageLog.Error(*LOCTEXT("Compile_CannotGenerateFunction", "ICE - cannot generate function - @@").ToString(), Node);
		return;
	}
}

#undef LOCTEXT_NAMESPACE
