// Copyright Epic Games, Inc. All Rights Reserved.

#include "PushModelHelpers.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/MemberReference.h"
#include "HAL/Platform.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Self.h"
#include "KismetCompiledFunctionContext.h"
#include "Misc/AssertionMacros.h"
#include "Net/NetPushModelHelpers.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

UEdGraphNode* FKCPushModelHelpers::ConstructMarkDirtyNodeForProperty(FKismetFunctionContext& Context, FProperty* RepProperty, UEdGraphPin* PropertyObjectPin)
{
	/**
	 * This code is for push model property dirty tracking.
	 * It works by injecting extra nodes while compiling that will call UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex.
	 * These nodes will be generated even when push model is disabled, but when evaluated will be NoOps if PushModel is compiled
	 * out. This is done to prevent the need for recooking data if Push Model is enabled or disabled for evaluation / testing.
	 * TODO: Maybe have a CVar or something else that can prevent these nodes from being added?
	 *
	 * That function will be called with the Object that owns the property (either Self or whatever is connected to the Target pin
	 * of the BP Node), the RepIndex of the property, and the Property's name.
	 *
	 * Note, this assumes that there's no way that a native class can add or remove replicated properties without also
	 * recompiling blueprints. The only scenario that seems possible is cooked games with custom built binaries, and seems
	 * unlikely (only really for testing purposes).
	 *
	 * If that becomes a problem, we can instead switch to using the property name and resorting to a FindField call at runtime,
	 * but that will be significantly more expensive.
	 */

	static const FName MarkPropertyDirtyFuncName(TEXT("MarkPropertyDirtyFromRepIndex"));
	static const FName ObjectPinName(TEXT("Object"));
	static const FName RepIndexPinName(TEXT("RepIndex"));
	static const FName PropertyNamePinName(TEXT("PropertyName"));

	if (!ensure(RepProperty) || !ensure(PropertyObjectPin))
	{
		return nullptr;
	}

	UClass* OwningClass = RepProperty->GetTypedOwner<UClass>();
	if (!ensure(OwningClass))
	{
		return nullptr;
	}

	// We need to make sure this class already has its property offsets setup, otherwise
	// the order of our replicated properties won't match, meaning the RepIndex will be
	// invalid.
	if (RepProperty->GetOffset_ForGC() == 0)
	{
		// Make sure that we're using the correct class and that it has replication data set up.
		if (OwningClass->ClassGeneratedBy == Context.Blueprint && Context.NewClass && Context.NewClass != OwningClass)
		{
			OwningClass = Context.NewClass;
			RepProperty = FindFieldChecked<FProperty>(OwningClass, RepProperty->GetFName());
		}

		if (RepProperty->GetOffset_ForGC() == 0)
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(OwningClass->ClassGeneratedBy))
			{
				if (UClass* UseClass = Blueprint->GeneratedClass)
				{
					OwningClass = UseClass;
					RepProperty = FindFieldChecked<FProperty>(OwningClass, RepProperty->GetFName());
				}
			}
		}
	}

	ensureAlwaysMsgf(RepProperty->GetOffset_ForGC() != 0,
		TEXT("Class does not have Property Offsets setup. This will cause issues with Push Model. Blueprint=%s, Class=%s, Property=%s"),
		*Context.Blueprint->GetPathName(), *OwningClass->GetPathName(), *RepProperty->GetName());

	if (!OwningClass->HasAnyClassFlags(CLASS_ReplicationDataIsSetUp))
	{
		OwningClass->SetUpRuntimeReplicationData();
	}

	// Create the node that will call MarkPropertyDirty.
	UK2Node_CallFunction* MarkPropertyDirtyNode = Context.SourceGraph->CreateIntermediateNode<UK2Node_CallFunction>();
	MarkPropertyDirtyNode->FunctionReference.SetExternalMember(MarkPropertyDirtyFuncName, UNetPushModelHelpers::StaticClass());
	MarkPropertyDirtyNode->AllocateDefaultPins();

	// Create the Pins for RepIndex, PropertyName, and Object.
	UEdGraphPin* RepIndexPin = MarkPropertyDirtyNode->FindPinChecked(RepIndexPinName);
	RepIndexPin->DefaultValue = FString::FromInt(RepProperty->RepIndex);

	UEdGraphPin* PropertyNamePin = MarkPropertyDirtyNode->FindPinChecked(PropertyNamePinName);
	PropertyNamePin->DefaultValue = RepProperty->GetFName().ToString();

	// If the property is linked to some other object, go ahead and grab that.
	// Otherwise, create an intermediate self Pin and use that.
	if (PropertyObjectPin->LinkedTo.Num() > 0)
	{
		PropertyObjectPin = PropertyObjectPin->LinkedTo[0];
	}
	else
	{
		UK2Node_Self* SelfNode = Context.SourceGraph->CreateIntermediateNode<UK2Node_Self>();
		SelfNode->AllocateDefaultPins();
		PropertyObjectPin = SelfNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
	}

	UEdGraphPin* ObjectPin = MarkPropertyDirtyNode->FindPinChecked(ObjectPinName);
	ObjectPin->MakeLinkTo(PropertyObjectPin);

	return MarkPropertyDirtyNode;
}