// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//////////////////////////////////////////////////////////////////////////
// FKCPushModelHelpers

struct FKCPushModelHelpers
{
	/**
	 * Generates nodes that will call the necessary functions for marking Push Model Properties dirty.
	 * These nodes should be generated and linked whenever we know there's a replicated property that
	 * is likely to have its value changed.
	 *
	 * @param Context				Current compilcation context.
	 * @param RepProperty			The Replicated Property that's being reference.
	 * @param PropertyObjectPin		The pin we'll use to grab a reference to the Object that owns the property.
	 */
	static class UEdGraphNode* ConstructMarkDirtyNodeForProperty(struct FKismetFunctionContext& Context, class FProperty* RepProperty, class UEdGraphPin* PropertyObjectPin);
};