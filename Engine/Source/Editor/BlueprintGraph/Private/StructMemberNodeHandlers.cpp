// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructMemberNodeHandlers.h"

#include "BPTerminal.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/IndirectArray.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "K2Node_StructMemberGet.h"
#include "K2Node_StructMemberSet.h"
#include "K2Node_StructOperation.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

static FBPTerminal* RegisterStructVar(FCompilerResultsLog& MessageLog, FKismetFunctionContext& Context, UK2Node_StructOperation* MemberSetNode)
{

	// Find the self pin
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
	UEdGraphPin* SelfPin = Schema->FindSelfPin(*MemberSetNode, EGPD_Input);

	// Determine the search scope for the struct property (not the member)
	UStruct* SearchScope = (SelfPin != NULL) ? Context.GetScopeFromPinType(SelfPin->PinType, Context.NewClass) : Context.Function;

	// Now find the variable
	bool bIsSparseProperty;
	if (FProperty* BoundProperty = FKismetCompilerUtilities::FindNamedPropertyInScope(SearchScope, MemberSetNode->GetVarName(), bIsSparseProperty))
	{
		// Create the term in the list
		FBPTerminal* Term = new FBPTerminal();
		Context.VariableReferences.Add(Term);

		Schema->ConvertPropertyToPinType(BoundProperty, /*out*/ Term->Type);
		Term->Source = MemberSetNode;
		Term->Name = MemberSetNode->GetVarNameString();
		Term->SetContextTypeStruct();

		Term->AssociatedVarProperty = BoundProperty;
		//@TODO: Needed? Context.NetMap.Add(Net, Term);

		// Read-only variables and variables in const classes are both const
		if (BoundProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly) || (Context.IsConstFunction() && Context.NewClass->IsChildOf(SearchScope)))
		{
			Term->bIsConst = true;
		}

		// Resolve the context term
		if (SelfPin != NULL)
		{
			FBPTerminal** pContextTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(SelfPin));
			Term->Context = (pContextTerm != NULL) ? *pContextTerm : NULL;
		}

		return Term;
	}
	else
	{
		MessageLog.Error(TEXT("Failed to find struct variable used in @@"), MemberSetNode);
		return NULL;
	}
}

static void ResolveAndRegisterScopedStructTerm(FCompilerResultsLog& MessageLog, FKismetFunctionContext& Context, UScriptStruct* StructType, UEdGraphPin* Net, FBPTerminal* ContextTerm)
{
	// Find the property for the struct
	if (FProperty* BoundProperty = FindFProperty<FProperty>(StructType, Net->PinName))
	{
		// Create the term in the list
		FBPTerminal* Term = new FBPTerminal();
		Context.VariableReferences.Add(Term);
		Term->CopyFromPin(Net, Net->PinName.ToString());
		Term->AssociatedVarProperty = BoundProperty;
		Context.NetMap.Add(Net, Term);
		Term->Context = ContextTerm;

		// Read-only variables and variables in const classes are both const
		if (BoundProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly) || (Context.IsConstFunction() &&
			Context.NewClass && Context.NewClass->IsChildOf(StructType)))
		{
			Term->bIsConst = true;
		}
	}
	else
	{
		MessageLog.Error(TEXT("Failed to find a struct member for @@"), Net);
	}
}

void FKCHandler_StructMemberVariableGet::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	// This net is a variable read
	ResolveAndRegisterScopedTerm(Context, Net, Context.VariableReferences);
}

void FKCHandler_StructMemberVariableGet::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode)
{
	UK2Node_StructMemberGet* MemberGetNode = CastChecked<UK2Node_StructMemberGet>(InNode);
	MemberGetNode->CheckForErrors(CompilerContext.GetSchema(), Context.MessageLog);

	if (FBPTerminal* ContextTerm = RegisterStructVar(CompilerContext.MessageLog, Context, MemberGetNode))
	{
		// Register a term for each output pin
		for (int32 PinIndex = 0; PinIndex < MemberGetNode->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Net = MemberGetNode->Pins[PinIndex];

			ResolveAndRegisterScopedStructTerm(CompilerContext.MessageLog, Context, MemberGetNode->StructType, Net, ContextTerm);
		}
	}
}

void FKCHandler_StructMemberVariableSet::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode)
{
	UK2Node_StructMemberSet* MemberSetNode = CastChecked<UK2Node_StructMemberSet>(InNode);
	MemberSetNode->CheckForErrors(CompilerContext.GetSchema(), Context.MessageLog);

	if (FBPTerminal* ContextTerm = RegisterStructVar(CompilerContext.MessageLog, Context, MemberSetNode))
	{
		// Register a term for each input pin
		for (int32 PinIndex = 0; PinIndex < MemberSetNode->Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Net = MemberSetNode->Pins[PinIndex];
			if (!CompilerContext.GetSchema()->IsMetaPin(*Net) && (Net->Direction == EGPD_Input))
			{
				if (ValidateAndRegisterNetIfLiteral(Context, Net))
				{
					ResolveAndRegisterScopedStructTerm(CompilerContext.MessageLog, Context, MemberSetNode->StructType, Net, ContextTerm);
				}
			}
		}
	}
}
