// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetCompilerMisc.h"
#include "VariableSetHandler.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UEdGraphPin;
struct FKismetFunctionContext;

class FKCHandler_StructMemberVariableGet : public FNodeHandlingFunctor
{
public:
	FKCHandler_StructMemberVariableGet(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override;
};

class FKCHandler_StructMemberVariableSet : public FKCHandler_VariableSet
{
public:
	FKCHandler_StructMemberVariableSet(FKismetCompilerContext& InCompilerContext)
		: FKCHandler_VariableSet(InCompilerContext)
	{
	}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override;
};
