// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Misc/AutomationTest.h"
#include "BlueprintActionDatabase.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintTypePromotion.h"
#include "EdGraph/EdGraphPin.h"
#include "K2Node_PromotableOperator.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "GameFramework/Actor.h"			// For making a dummy BP
#include "Kismet2/KismetEditorUtilities.h"
#include "Kismet2/WildcardNodeUtils.h"

#if WITH_DEV_AUTOMATION_TESTS

/////////////////////////////////////////////////////
// Helpers to make dummy blueprints/pins/nodes

namespace TypePromoTestUtils
{
	/**
	* Spawn a test promotable operator node that we can use to ensure type propegation works
	* correctly.
	*
	* @param Graph		The graph to spawn this node in
	* @param OpName		Name of the promotable op to spawn (Multiply, Add, Subtract, etc)
	*
	* @return UK2Node_PromotableOperator* if the op exists,
	*/
	static UK2Node_PromotableOperator* SpawnPromotableNode(UEdGraph* Graph, const FName OpName)
	{
		if (!Graph)
		{
			return nullptr;
		}

		// The spawner will be null if type promo isn't enabled
		if (UBlueprintFunctionNodeSpawner* Spawner = FTypePromotion::GetOperatorSpawner(OpName))
		{
			// Spawn a new node!
			IBlueprintNodeBinder::FBindingSet Bindings;
			FVector2D SpawnLoc{};
			UK2Node_PromotableOperator* NewOpNode = Cast<UK2Node_PromotableOperator>(Spawner->Invoke(Graph, Bindings, SpawnLoc));

			return NewOpNode;
		}

		return nullptr;
	}

	/**
	* Mark this array of spawned test pins as pending kill to ensure that
	* they get cleaned up properly by GC
	*
	* @param InPins		The array of spawned test pins to mark pending kill
	*/
	static void CleanupTestPins(TArray<UEdGraphPin*>& InPins)
	{
		// Clear our test pins
		for (UEdGraphPin* TestPin : InPins)
		{
			if (TestPin)
			{
				TestPin->MarkAsGarbage();
			}
		}
		InPins.Empty();
	}


	/**
	* Attempts to create a connection between the two given pins and tests that the connection was valid
	*
	* @param OpNodePin	The pin on a promotable operator node
	* @param OtherPin	The other pin
	*/
	static bool TestPromotedConnection(UEdGraphPin* OpNodePin, UEdGraphPin* OtherPin)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		const bool bConnected = K2Schema->TryCreateConnection(OpNodePin, OtherPin);
		UK2Node_PromotableOperator* OwningNode = Cast<UK2Node_PromotableOperator>(OpNodePin->GetOwningNode());

		if (bConnected && OwningNode)
		{
			OwningNode->NotifyPinConnectionListChanged(OpNodePin);
		}

		return bConnected;
	}

	static FString GetPinListDisplayName(const TArray<UEdGraphPin*>& TestPins)
	{
		const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
		FString PinTypesString = "";
		for (int32 i = 0; i < TestPins.Num(); ++i)
		{
			UEdGraphPin* Pin = TestPins[i];
			PinTypesString += K2Schema->TypeToText(Pin->PinType).ToString();
			if (i != TestPins.Num() - 1)
			{
				PinTypesString += ", ";
			}
		}
		return PinTypesString;
	}
}

#define MakeTestableBP(BPName, GraphName)												\
		UBlueprint* BPName = FKismetEditorUtilities::CreateBlueprint(					\
		AActor::StaticClass(),															\
		GetTransientPackage(),															\
		TEXT(#BPName),																	\
		BPTYPE_Normal,																	\
		UBlueprint::StaticClass(),														\
		UBlueprintGeneratedClass::StaticClass(),										\
		NAME_None																		\
	);																					\
	UEdGraph* GraphName = BPName ? FBlueprintEditorUtils::FindEventGraph( BPName ) : nullptr;	

#define MakeTestableNode(NodeName, OwningGraph)											\
		UK2Node* NodeName = NewObject<UK2Node_CallFunction>(/*outer*/ OwningGraph );	\
		OwningGraph->AddNode(NodeName);

#define MakeTestPin(OwningNode, PinArray, InPinName, InPinType, PinDirection)	UEdGraphPin* InPinName = UEdGraphPin::CreatePin(OwningNode);		\
																				InPinName->PinType.PinCategory = InPinType;							\
																				PinArray.Add(InPinName);											\
																				InPinName->Direction = PinDirection;

#define MakeRealNumberTestPin(OwningNode, PinArray, InPinName, PinDirection)	UEdGraphPin* InPinName = UEdGraphPin::CreatePin(OwningNode);		\
																				InPinName->PinType.PinCategory = UEdGraphSchema_K2::PC_Real;		\
																				InPinName->PinType.PinSubCategory = UEdGraphSchema_K2::PC_Double;	\
																				PinArray.Add(InPinName);											\
																				InPinName->Direction = PinDirection;

#define MakeRealNumberTestPins(OwningNode, OutArray)													\
	MakeRealNumberTestPin(OwningNode, OutArray, RealPinA, EGPD_Input);									\
	MakeRealNumberTestPin(OwningNode, OutArray, RealPinB, EGPD_Input);									\
	MakeRealNumberTestPin(OwningNode, OutArray, RealOutputPin, EGPD_Output);		


#define MakeTestPins(OwningNode, OutArray)																\
	MakeRealNumberTestPins(OwningNode, OutArray)														\
	MakeTestPin(OwningNode, OutArray, Int64PinA, UEdGraphSchema_K2::PC_Int64, EGPD_Output);				\
	MakeTestPin(OwningNode, OutArray, Int64PinB, UEdGraphSchema_K2::PC_Int64, EGPD_Input);				\
	MakeTestPin(OwningNode, OutArray, BytePinA, UEdGraphSchema_K2::PC_Byte, EGPD_Output);				\
	MakeTestPin(OwningNode, OutArray, WildPinA, UEdGraphSchema_K2::PC_Wildcard, EGPD_Output);			\
	MakeTestPin(OwningNode, OutArray, WildPinB, UEdGraphSchema_K2::PC_Wildcard, EGPD_Input);			\
	MakeTestPin(OwningNode, OutArray, BytePinB, UEdGraphSchema_K2::PC_Byte, EGPD_Input);				\
	MakeTestPin(OwningNode, OutArray, BoolPinA, UEdGraphSchema_K2::PC_Boolean, EGPD_Output);			\
	MakeTestPin(OwningNode, OutArray, BoolPinB, UEdGraphSchema_K2::PC_Boolean, EGPD_Input);				\
	MakeTestPin(OwningNode, OutArray, BoolOutputPin, UEdGraphSchema_K2::PC_Boolean, EGPD_Output);		\
	MakeTestPin(OwningNode, OutArray, IntPinA, UEdGraphSchema_K2::PC_Int, EGPD_Output);					\
	MakeTestPin(OwningNode, OutArray, VecInputPinA, UEdGraphSchema_K2::PC_Struct, EGPD_Input);			\
	VecInputPinA->PinType.PinSubCategoryObject = TBaseStructure<FVector>::Get();						\
	MakeTestPin(OwningNode, OutArray, VecInputPinB, UEdGraphSchema_K2::PC_Struct, EGPD_Input);			\
	VecInputPinB->PinType.PinSubCategoryObject = TBaseStructure<FVector>::Get();						\
	MakeTestPin(OwningNode, OutArray, VecOutputPinA, UEdGraphSchema_K2::PC_Struct, EGPD_Output);		\
	VecOutputPinA->PinType.PinSubCategoryObject = TBaseStructure<FVector>::Get();						\
	MakeTestPin(OwningNode, OutArray, Vec2DOutputPinA, UEdGraphSchema_K2::PC_Struct, EGPD_Output);		\
	Vec2DOutputPinA->PinType.PinSubCategoryObject = TBaseStructure<FVector2D>::Get();					\
	MakeTestPin(OwningNode, OutArray, Vec4OutPin, UEdGraphSchema_K2::PC_Struct, EGPD_Output);			\
	Vec4OutPin->PinType.PinSubCategoryObject = TBaseStructure<FVector4>::Get();							\
	MakeTestPin(OwningNode, OutArray, RotOutPin, UEdGraphSchema_K2::PC_Struct, EGPD_Output);			\
	RotOutPin->PinType.PinSubCategoryObject = TBaseStructure<FRotator>::Get();							\
	MakeTestPin(OwningNode, OutArray, QuatOutPin, UEdGraphSchema_K2::PC_Struct, EGPD_Output);			\
	QuatOutPin->PinType.PinSubCategoryObject = TBaseStructure<FQuat>::Get();

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTypePromotionTest, "Blueprints.Compiler.TypePromotion", EAutomationTestFlags::EditorContext | EAutomationTestFlags::SmokeFilter)
bool FTypePromotionTest::RunTest(const FString& Parameters)
{
	FEdGraphPinType RealPin = {};		RealPin.PinCategory = UEdGraphSchema_K2::PC_Real; RealPin.PinSubCategory = UEdGraphSchema_K2::PC_Double;
	FEdGraphPinType IntPin = {};		IntPin.PinCategory = UEdGraphSchema_K2::PC_Int;
	FEdGraphPinType Int64Pin = {};		Int64Pin.PinCategory = UEdGraphSchema_K2::PC_Int64;
	FEdGraphPinType BytePin = {};		BytePin.PinCategory = UEdGraphSchema_K2::PC_Byte;
	FEdGraphPinType VecPin = {};		VecPin.PinCategory = UEdGraphSchema_K2::PC_Struct; VecPin.PinSubCategoryObject = TBaseStructure<FVector>::Get();

	// Test promotions that should happen
	TestEqual(TEXT("Testing real to vector"), FTypePromotion::GetHigherType(RealPin, VecPin), FTypePromotion::ETypeComparisonResult::TypeBHigher);

	TestEqual(TEXT("Testing int to real"), FTypePromotion::GetHigherType(IntPin, RealPin), FTypePromotion::ETypeComparisonResult::TypeBHigher);
	TestEqual(TEXT("Testing int to int64"), FTypePromotion::GetHigherType(IntPin, Int64Pin), FTypePromotion::ETypeComparisonResult::TypeBHigher);

	TestEqual(TEXT("Testing byte to int"), FTypePromotion::GetHigherType(BytePin, IntPin), FTypePromotion::ETypeComparisonResult::TypeBHigher);
	TestEqual(TEXT("Testing byte to int64"), FTypePromotion::GetHigherType(BytePin, Int64Pin), FTypePromotion::ETypeComparisonResult::TypeBHigher);

	TestEqual(TEXT("Testing real to int64"), FTypePromotion::GetHigherType(RealPin, Int64Pin), FTypePromotion::ETypeComparisonResult::TypeBHigher);

	// Test Equality of pins
	TestEqual(TEXT("Testing byte == byte"), FTypePromotion::GetHigherType(BytePin, BytePin), FTypePromotion::ETypeComparisonResult::TypesEqual);
	TestEqual(TEXT("Testing real == real"), FTypePromotion::GetHigherType(RealPin, RealPin), FTypePromotion::ETypeComparisonResult::TypesEqual);
	TestEqual(TEXT("Testing int == int"), FTypePromotion::GetHigherType(IntPin, IntPin), FTypePromotion::ETypeComparisonResult::TypesEqual);
	TestEqual(TEXT("Testing int64 == int64"), FTypePromotion::GetHigherType(Int64Pin, Int64Pin), FTypePromotion::ETypeComparisonResult::TypesEqual);

	// Test promotions that should not happen
	TestEqual(TEXT("Testing int64 cannot go to byte"), FTypePromotion::GetHigherType(Int64Pin, BytePin), FTypePromotion::ETypeComparisonResult::TypeAHigher);
	TestEqual(TEXT("Testing int64 cannot go to int"), FTypePromotion::GetHigherType(Int64Pin, IntPin), FTypePromotion::ETypeComparisonResult::TypeAHigher);

	return true;
}

// Test that when given an array of UEdGraphPins we can find the appropriate UFunction that best
// matches them. This is the core of how the Type Promotion system works at BP compile time
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFindBestMatchingFunc, "Blueprints.Compiler.FindBestMatchingFunc", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FFindBestMatchingFunc::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	UEdGraphNode* TestNode = NewObject<UEdGraphNode>();

	////////////////////////////////////////////////////
	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	#define TestMatchingFunc(OpName, TestPins, ExpectedFuncName)															\
	{																														\
		const UFunction* FoundFunc = FTypePromotion::FindBestMatchingFunc(OpName, TestPins);								\
		const FName ExpectedName = FName(ExpectedFuncName);																	\
		const FString TestNullMessage = FString::Printf(TEXT(" Find Function '%s' null check"), *ExpectedName.ToString());	\
		if (TestNotNull(TestNullMessage, FoundFunc))																		\
		{																													\
		 	const FString PinTypesString = TypePromoTestUtils::GetPinListDisplayName(TestPins);								\
			const FString TestMessage = FString::Printf(TEXT("Given pins %s Expecting Function '%s' and got '%s'"),			\
				*PinTypesString,																							\
				*ExpectedName.ToString(),																					\
				*FoundFunc->GetFName().ToString());																			\
				TestEqual(TestMessage, FoundFunc->GetFName(), ExpectedName);												\
		}																													\
	}

	{
		TArray<UEdGraphPin*> TestPins =
		{
			Vec2DOutputPinA,
		};
		TestMatchingFunc(TEXT("Add"), TestPins, TEXT("Add_Vector2DVector2D"));
	}

	// Multiply_VectorVector given A real input, vector input, and a vector output
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, VecInputPinB, VecOutputPinA,
		};
		TestMatchingFunc(TEXT("Multiply"), TestPins, TEXT("Multiply_VectorVector"));
	}

	// Multiply_VectorVector given a real, vector, real
	// Order shouldn't matter when passing these pins in, which is what we are testing here
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, VecOutputPinA, VecInputPinA,
		};
		TestMatchingFunc(TEXT("Multiply"), TestPins, TEXT("Multiply_VectorVector"));
	}

	// Multiply_VectorVector given two vector inputs and a vector output
	{
		TArray<UEdGraphPin*> TestPins =
		{
			VecInputPinA, VecInputPinB, VecOutputPinA,
		};
		TestMatchingFunc(TEXT("Multiply"), TestPins, TEXT("Multiply_VectorVector"));
	}

	// Add_DoubleDouble
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, RealPinB, RealOutputPin
		};
		TestMatchingFunc(TEXT("Add"), TestPins, TEXT("Add_DoubleDouble"));
	}

	// Subtract_DoubleDouble
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, RealPinB, RealOutputPin
		};
		TestMatchingFunc(TEXT("Subtract"), TestPins, TEXT("Subtract_DoubleDouble"));
	}

	// Add_DoubleDouble given only one real pin. This simulates the first connection being made to a 
	// promotable operator, in which case we should default to a regular old real + real
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA,
		};
		TestMatchingFunc(TEXT("Add"), TestPins, TEXT("Add_DoubleDouble"));
	}

	// Less_DoubleDouble given a real
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, BoolOutputPin
		};
		TestMatchingFunc(TEXT("Less"), TestPins, TEXT("Less_DoubleDouble"));
	}

	// Less_DoubleDouble given just a single real
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA,
		};
		TestMatchingFunc(TEXT("Less"), TestPins, TEXT("Less_DoubleDouble"));
	}

	// Greater_DoubleDouble given reals
	{
		TArray<UEdGraphPin*> TestPins =
		{
			RealPinA, RealPinA
		};
		TestMatchingFunc(TEXT("Greater"), TestPins, TEXT("Greater_DoubleDouble"));
	}
	
	TypePromoTestUtils::CleanupTestPins(PinTypes);
	TestNode->MarkAsGarbage();

	return true;
}

// Test that every type in the promotion table has a best matching function for each operator
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableTypeToOperator, "Blueprints.Compiler.TypeToOperator", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableTypeToOperator::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	UEdGraphNode* TestNode = NewObject<UEdGraphNode>();

	const TSet<FName> AllOpNames = FTypePromotion::GetAllOpNames();

	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const TMap<FName, TArray<FName>>* const PromoTable = FTypePromotion::GetPrimitivePromotionTable();
	TestNotNull(TEXT("Primitive Promotion table exists"), PromoTable);

	for (const FName OpName : AllOpNames)
	{
		// Ensure that there is a best matching function for each type in the promo table
		for (const TPair<FName, TArray<FName>>& Pair : *PromoTable)
		{
			const FName TypeName = Pair.Key;
			if (TypeName == UEdGraphSchema_K2::PC_Wildcard)
			{
				continue;
			}

			UEdGraphPin* TypePin = *PinTypes.FindByPredicate([&TypeName](const UEdGraphPin* Pin) -> bool { return Pin && Pin->PinType.PinCategory == TypeName && Pin->Direction == EGPD_Output; });
			TestNotNull(TEXT("Testable Pin Type Found"), TypePin);

			const UFunction* BestMatchFunc = FTypePromotion::FindBestMatchingFunc(OpName, { TypePin });
			FString BottomStatusMessage = FString::Printf(TEXT("'%s' Operator has a match with '%s' pin type."), *TypeName.ToString(), *K2Schema->TypeToText(TypePin->PinType).ToString());

			TestNotNull(BottomStatusMessage, BestMatchFunc);
		}
	}

	// Cleanup test
	{
		TypePromoTestUtils::CleanupTestPins(PinTypes);
		TestNode->MarkAsGarbage();
	}

	return true;
}

// Test the default state of all operator nodes to ensure they are correct.
// Comparison operators (Greater Than, Less Than, etc) should have two 
// wildcard inputs and one boolean output. All others should be all wildcards.
// The node's set function should also match the operator correctly and 
// it should have the 'OperationName' variable set.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableOpDefaultState, "Blueprints.Nodes.PromotableOp.DefaultState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableOpDefaultState::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	MakeTestableBP(WildcardStartTestBP, TestWildcardGraph);

	const TSet<FName> AllOpNames = FTypePromotion::GetAllOpNames();

	for (const FName& OpName : AllOpNames)
	{
		const bool bIsComparisonOp = FTypePromotion::GetComparisonOpNames().Contains(OpName);
		const FString OpNameString = OpName.ToString();

		UK2Node_PromotableOperator* OpNode = TypePromoTestUtils::SpawnPromotableNode(TestWildcardGraph, OpName);
		TestNotNull(FString::Printf(TEXT("Spawning a '%s' operator node"), *OpNameString), OpNode);
		
		// The 'OperationName' variable is correct
		TestTrue(FString::Printf(TEXT("Operation Name '%s' matches after spawning node"), *OpNameString), OpNode->GetOperationName() == OpName);

		// The target function has been set when the node is spawned
		const UFunction* TargetFunc = OpNode->GetTargetFunction();
		TestNotNull(FString::Printf(TEXT("'%s' Operation function is not null"), *OpNameString), TargetFunc);
		
		// The target function is of the correct operation type
		const FName TargetFunctionOpName = FTypePromotion::GetOpNameFromFunction(TargetFunc);
		TestTrue(FString::Printf(TEXT("'%s' Operation function matches requested operation"), *OpNameString), TargetFunctionOpName == OpName);

		// Test pin types
		const UEdGraphPin* TopInputPin = OpNode->FindPin(TEXT("A"), EGPD_Input);
		const UEdGraphPin* BottomInputPin = OpNode->FindPin(TEXT("B"), EGPD_Input);
		const UEdGraphPin* OutputPin = OpNode->GetOutputPin();
		
		// Comparison operators should be spawned with two wildcard inputs and a boolean output
		if (bIsComparisonOp)
		{
			TestTrue(TEXT("Top is pin wildcard"), FWildcardNodeUtils::IsWildcardPin(TopInputPin));
			TestTrue(TEXT("Bottom Pin is wildcard"), FWildcardNodeUtils::IsWildcardPin(BottomInputPin));
			TestTrue(TEXT("Bottom Pin is a bool"), OutputPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Boolean);
		}
		// All other operators should be spawned with all wildcard inputs and output
		else
		{
			TestTrue(TEXT("Top is pin wildcard"), FWildcardNodeUtils::IsWildcardPin(TopInputPin));
			TestTrue(TEXT("Bottom Pin is wildcard"), FWildcardNodeUtils::IsWildcardPin(BottomInputPin));
			TestTrue(TEXT("Bottom Pin is a wildcard"), FWildcardNodeUtils::IsWildcardPin(OutputPin));
		}
	}

	// Cleanup test BP and graph
	{
		WildcardStartTestBP->MarkAsGarbage();
		WildcardStartTestBP->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
		TestWildcardGraph->MarkAsGarbage();
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableOpTolerancePin, "Blueprints.Nodes.PromotableOp.TolerancePin", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableOpTolerancePin::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	MakeTestableBP(ToleranceTestBP, ToleranceGraph);
	MakeTestableNode(TestNode, ToleranceGraph)
	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	// These functions should be spawned with a tolerance float pin
	// EqualEqual_Vector2DVector2D
	// NotEqual_Vector2DVector2D
	// EqualEqual_VectorVector
	// NotEqual_VectorVector
	// EqualEqual_Vector4Vector4
	// NotEqual_Vector4Vector4
	// EqualEqual_RotatorRotator
	// NotEqual_RotatorRotator
	// NotEqual_QuatQuat
	// EqualEqual_QuatQuat
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
	// Make TMap of the op -> tpe
	TArray<TPair<FName, UEdGraphPin*>> ToleranceFunctions;
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("EqualEqual"), VecOutputPinA));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("NotEqual"), VecOutputPinA));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("EqualEqual"), Vec2DOutputPinA));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("NotEqual"), Vec2DOutputPinA));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("EqualEqual"), Vec4OutPin));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("NotEqual"), Vec4OutPin));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("EqualEqual"), RotOutPin));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("NotEqual"), RotOutPin));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("EqualEqual"), QuatOutPin));
	ToleranceFunctions.Add(TPair<FName, UEdGraphPin*>(TEXT("NotEqual"), QuatOutPin));

	// iterate the map
	// for each value of the map
	// Spawn the operator node of the op type
	// Connect the pin type
	// Test that there is an error tolerance pin
	// test that the pin type is a float
	// When disconnecting the Pin the tolerance pin should be removed, unless a default value is there
	// when copy/pasting the node the tolerance pin should not be kept unless it has a value
	TArray<UEdGraphPin*> PinsToConsider;
	for(const TPair<FName, UEdGraphPin*>& Pair  : ToleranceFunctions)
	{
		const FName OpName = Pair.Key;
		UEdGraphPin* TestingPin = Pair.Value;

		UK2Node_PromotableOperator* OpNode = TypePromoTestUtils::SpawnPromotableNode(ToleranceGraph, OpName);
		TestNotNull(FString::Printf(TEXT("Spawning a '%s' operator node"), *OpName.ToString()), OpNode);

		check(OpNode);

		// Test pin types
		UEdGraphPin* TopInputPin = OpNode->FindPin(TEXT("A"), EGPD_Input);
		UEdGraphPin* BottomInputPin = OpNode->FindPin(TEXT("B"), EGPD_Input);
		UEdGraphPin* OutputPin = OpNode->GetOutputPin();
		check(TopInputPin && BottomInputPin && OutputPin);
		UEdGraphPin* TolerancePin = OpNode->FindTolerancePin();
		TestNull(TEXT("No tolerance by default"), TolerancePin);

		const int32 StartingPinCount = OpNode->Pins.Num();

		const bool bConnected = TypePromoTestUtils::TestPromotedConnection(TopInputPin, TestingPin);
		TestTrue(TEXT("Connection to additional pin success"), bConnected);
		const int32 EndingPinCount = OpNode->Pins.Num();
		TolerancePin = OpNode->FindTolerancePin();
		const FString Message = FString::Printf(TEXT("'%s' operator node connecting to '%s' pin has a tolerance pin"), *OpName.ToString(), *K2Schema->TypeToText(TestingPin->PinType).ToString());
		TestNotNull(Message, TolerancePin);

		TestTrue(TEXT("Added a new pin"), EndingPinCount == StartingPinCount + 1);
	}

	// Cleanup test BP and graph
	{
		TypePromoTestUtils::CleanupTestPins(PinTypes);

		ToleranceTestBP->MarkAsGarbage();
		ToleranceTestBP->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
		TestNode->MarkAsGarbage();
		ToleranceGraph->MarkAsGarbage();
	}

	return true;
}

// Test that promotable operator nodes can correctly have pins added to them
// and that comparison operators cannot have pins added to them.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableOpNodeAddPinInterface, "Blueprints.Nodes.PromotableOp.AddPinInterface", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableOpNodeAddPinInterface::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	MakeTestableBP(TestBP, TestGraph);
	MakeTestableNode(TestNode, TestGraph);

	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	const TSet<FName> ComparisonOpNames = FTypePromotion::GetComparisonOpNames();
	for (const FName& OpName : ComparisonOpNames)
	{
		const FString OpNameString = OpName.ToString();

		UK2Node_PromotableOperator* OpNode = TypePromoTestUtils::SpawnPromotableNode(TestGraph, OpName);
		TestNotNull(FString::Printf(TEXT("'%s' Comparison op spawned"), *OpNameString), OpNode);

		TestFalse(FString::Printf(TEXT("'%s' Comparison op cannot add pin"), *OpNameString), OpNode->CanAddPin());
	}

	// Anything that is not a comparison operator can have a pin added to it
	{
		UK2Node_PromotableOperator* MultiplyNode = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Multiply"));
		TestNotNull(TEXT("Multiply Node spawn"), MultiplyNode);
		TestTrue(TEXT("Multiply can add pin"), MultiplyNode->CanAddPin());
	}

	{
		UK2Node_PromotableOperator* MultiplyNode = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Multiply"));
		TestNotNull(TEXT("Multiply Node spawn"), MultiplyNode);
		TestTrue(TEXT("Multiply can add pin"), MultiplyNode->CanAddPin());

		const int32 StartingPinCount = MultiplyNode->Pins.Num();
		MultiplyNode->AddInputPin();
		const int32 EndingPinCount = MultiplyNode->Pins.Num();

		TestTrue(TEXT("Multiply node had a pin added to it"), EndingPinCount == StartingPinCount + 1);
		const UEdGraphPin* AdditonalPin = MultiplyNode->GetAdditionalPin(EndingPinCount - StartingPinCount);
		TestNotNull(TEXT("Additional Pin is not null"), AdditonalPin);
		TestTrue(TEXT("New Pin is wildcard"), FWildcardNodeUtils::IsWildcardPin(AdditonalPin));
		TestTrue(TEXT("New Pin can be removed"), MultiplyNode->CanRemovePin(AdditonalPin));

		const UEdGraphPin* InputPinA = MultiplyNode->FindPin(FName("A"), EGPD_Input);
		TestNotNull(TEXT("First input pin is not null"), InputPinA);
		TestTrue(TEXT("First Pin can be removed"), MultiplyNode->CanRemovePin(InputPinA));

		const UEdGraphPin* InputPinB = MultiplyNode->FindPin(FName("B"), EGPD_Input);
		TestNotNull(TEXT("Second input pin is not null"), InputPinB);
		TestTrue(TEXT("Second Pin can be removed"), MultiplyNode->CanRemovePin(InputPinB));
	}

	// Anything that is not a comparison operator can have a pin added to it
	{
		UK2Node_PromotableOperator* AddNode = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Add"));
		TestNotNull(TEXT("Add Node spawn"), AddNode);
		TestTrue(TEXT("Add can add pin"), AddNode->CanAddPin());
		
		const UEdGraphPin* TopInputPin = AddNode->FindPin(TEXT("A"), EGPD_Input);
		const UEdGraphPin* BottomInputPin = AddNode->FindPin(TEXT("B"), EGPD_Input);
		const UEdGraphPin* OutputPin = AddNode->GetOutputPin();
		const int32 StartingPinCount = AddNode->Pins.Num();
		AddNode->AddInputPin();
		
		UEdGraphPin* AdditonalPin = AddNode->GetAdditionalPin(2);
		TestNotNull(TEXT("Additional Pin is not null"), AdditonalPin);
		
		// Connect a real pin to the additional input pin
		const bool bConnected = TypePromoTestUtils::TestPromotedConnection(AdditonalPin, RealOutputPin);
		TestTrue(TEXT("Connection to additional pin success"), bConnected);

		// The other pins have propagated correctly with this new connection 
		TestTrue(TEXT("Top Pin type propegates to new connection"), TopInputPin->PinType.PinCategory == RealOutputPin->PinType.PinCategory);
		TestTrue(TEXT("Bottom Pin type propegates to new connection"), BottomInputPin->PinType.PinCategory == RealOutputPin->PinType.PinCategory);
		TestTrue(TEXT("Out Pin type propegates to new connection"), OutputPin->PinType.PinCategory == RealOutputPin->PinType.PinCategory);

		// Removing the only pin with a connection with reset the node to wildcard
		AddNode->RemoveInputPin(AdditonalPin);
		TestTrue(TEXT("Top Pin type propegates to wildcard on connection break"), FWildcardNodeUtils::IsWildcardPin(TopInputPin));
		TestTrue(TEXT("Bottom Pin type propegates to wildcard on connection break"), FWildcardNodeUtils::IsWildcardPin(BottomInputPin));
		TestTrue(TEXT("Out Pin type propegates to wildcard on connection break"), FWildcardNodeUtils::IsWildcardPin(OutputPin));
		
		TestTrue(TEXT("Additional pin was successfully removed"), StartingPinCount == AddNode->Pins.Num());
	}

	// Cleanup
	{
		TypePromoTestUtils::CleanupTestPins(PinTypes);

		TestBP->MarkAsGarbage();
		TestBP->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
		TestGraph->MarkAsGarbage();
		TestNode->MarkAsGarbage();
	}

	return true;
}

// Test that making connections to a Promotable Operator node results in the correct propagation of types
// throughout the whole node and that the node has the correct UFunction that it will expand to upon compiling.
// This will also test that pin connections are broken if they are connected to an invalid promotion, 
// and that pin connections are preserved if a valid promotion is occuring. 
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableOperatorConnectionChanged, "Blueprints.Nodes.PromotableOp.ConnectionChanged", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableOperatorConnectionChanged::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	MakeTestableBP(BP_ConnectionChanged, TestGraph);
	MakeTestableNode(TestNode, TestGraph);

	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Test that adding a float pin to the top input on an add node makes the whole thing a float
	{
		UK2Node_PromotableOperator* AddNode = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Add"));
		TestNotNull(TEXT("AddNode Node spawn"), AddNode);

		UEdGraphPin* TopInputPin = AddNode->FindPin(TEXT("A"), EGPD_Input);
		UEdGraphPin* BottomInputPin = AddNode->FindPin(TEXT("B"), EGPD_Input);
		UEdGraphPin* OutputPin = AddNode->GetOutputPin();

		check(TopInputPin && BottomInputPin && OutputPin);

		const bool bConnected = K2Schema->TryCreateConnection(TopInputPin, RealOutputPin);
		AddNode->NotifyPinConnectionListChanged(TopInputPin);

		TestTrue(TEXT("Bottom Pin type propagates to real"), bConnected && BottomInputPin->PinType.PinCategory == RealPinB->PinType.PinCategory);
	}

	// Connecting a vector output should make the other input be a vector as well
	{
		UK2Node_PromotableOperator* Node = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Multiply"));
		TestNotNull(TEXT("Multiply Node spawn"), Node);

		UEdGraphPin* TopInputPin = Node->FindPin(TEXT("A"), EGPD_Input);
		UEdGraphPin* OutputPin = Node->GetOutputPin();

		check(TopInputPin);

		const bool bConnected = K2Schema->TryCreateConnection(OutputPin, VecInputPinA);
		Node->NotifyPinConnectionListChanged(OutputPin);

		TestTrue(TEXT("Bottom Pin type propegates to float"), bConnected && TopInputPin->PinType.PinCategory == VecOutputPinA->PinType.PinCategory);
	}

	// Connecting a vector output should make the other input be a vector as well
	{
		UK2Node_PromotableOperator* Node = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Multiply"));
		TestNotNull(TEXT("Multiply Node spawn"), Node);

		UEdGraphPin* TopInputPin = Node->FindPin(TEXT("A"), EGPD_Input);
		UEdGraphPin* BottomInputPin = Node->FindPin(TEXT("B"), EGPD_Input);
		UEdGraphPin* OutputPin = Node->GetOutputPin();

		// Connect a float to the top pin
		const bool bConnected = K2Schema->TryCreateConnection(OutputPin, VecInputPinA);
		Node->NotifyPinConnectionListChanged(OutputPin);
		TestTrue(TEXT("Bottom Pin type propegates to float"), bConnected && TopInputPin->PinType.PinCategory == VecOutputPinA->PinType.PinCategory);

		// The output should be a float right now

		// Connect a double to the bottom pin

		// The output should be a double now
	}

	// Cleanup
	{
		TypePromoTestUtils::CleanupTestPins(PinTypes);

		BP_ConnectionChanged->MarkAsGarbage();
		BP_ConnectionChanged->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
		TestGraph->MarkAsGarbage();
		TestNode->MarkAsGarbage();
	}

	return true;
}

// Test the connections between primitive types and ensure that each one gets
// the correct output type pin
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPromotableOperatorPrimitivePromotions, "Blueprints.Nodes.PromotableOp.PrimitivePromotions", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FPromotableOperatorPrimitivePromotions::RunTest(const FString& Parameters)
{
	if (!TypePromoDebug::IsTypePromoEnabled())
	{
		return true;
	}

	// Refresh the actions within this test in case the editor is open but hasn't loaded BlueprintGraph yet
	FTypePromotion::ClearNodeSpawners();
	FBlueprintActionDatabase::Get().RefreshAll();

	MakeTestableBP(BP_Primitive_Connections, TestGraph);
	MakeTestableNode(TestNode, TestGraph);

	// Create test pins!
	TArray<UEdGraphPin*> PinTypes = {};
	MakeTestPins(TestNode, PinTypes);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	const TMap<FName, TArray<FName>>* const PromoTable = FTypePromotion::GetPrimitivePromotionTable();
	TestNotNull(TEXT("Primitive Promotion table exists"), PromoTable);
	
	if (!PromoTable)
	{
		return false;
	}


	for(TPair<FName, TArray<FName>> Pair : *PromoTable)
	{
		FName TypeName = Pair.Key;
		if(TypeName == UEdGraphSchema_K2::PC_Wildcard)
		{
			continue;
		}
		
		UEdGraphPin* TypePin = *PinTypes.FindByPredicate([&TypeName](const UEdGraphPin* Pin) -> bool { return Pin && Pin->PinType.PinCategory == TypeName && Pin->Direction == EGPD_Output; });
		
		for (const FName AvailablePromoType : Pair.Value)
		{
			UK2Node_PromotableOperator* Node = TypePromoTestUtils::SpawnPromotableNode(TestGraph, TEXT("Add"));
			TestNotNull(TEXT("Multiply Node spawn"), Node);

			UEdGraphPin* TopInputPin = Node->FindPin(TEXT("A"), EGPD_Input);
			UEdGraphPin* BottomInputPin = Node->FindPin(TEXT("B"), EGPD_Input);
			UEdGraphPin* OutputPin = Node->GetOutputPin();
			UEdGraphPin* PinToConnectTo = *PinTypes.FindByPredicate([&AvailablePromoType](const UEdGraphPin* Pin) -> bool { return Pin && Pin->PinType.PinCategory == AvailablePromoType && Pin->Direction == EGPD_Output; });

			// Connect to the top input pin
			const bool bConnectedTop = TypePromoTestUtils::TestPromotedConnection(TopInputPin, TypePin);
			FString StatusMessage = FString::Printf(TEXT("Connecting '%s' to '%s'"), *K2Schema->TypeToText(TopInputPin->PinType).ToString(), *K2Schema->TypeToText(TypePin->PinType).ToString());

			TestTrue(StatusMessage, bConnectedTop);

			// The other pins should now all be set to the first pin type
			TestTrue(TEXT("Bottom Pin type propegates to new connection"), BottomInputPin->PinType.PinCategory == TypePin->PinType.PinCategory);
			TestTrue(TEXT("Output Pin type propegates to new connection"), OutputPin->PinType.PinCategory == TypePin->PinType.PinCategory);

			// Connect the bottom pin to the type that the first one can be promoted to
			const bool bConnectedBottom = TypePromoTestUtils::TestPromotedConnection(BottomInputPin, PinToConnectTo);
			FString BottomStatusMessage = FString::Printf(TEXT("Bottom Pin '%s' Connecting to '%s' "), *K2Schema->TypeToText(BottomInputPin->PinType).ToString(), *K2Schema->TypeToText(PinToConnectTo->PinType).ToString());

			TestTrue(BottomStatusMessage, bConnectedBottom);
			
			// The top should be same type, and the output type should have been updated to the be new higher type
			TestTrue(TEXT("Top Pin type propegates to new connection"), TopInputPin->PinType.PinCategory == TypePin->PinType.PinCategory);
			const FString FinalStatusMessage = FString::Printf(TEXT("Output pin propegates to new con: Output: '%s' New Pin: '%s'"), *K2Schema->TypeToText(OutputPin->PinType).ToString(), *K2Schema->TypeToText(PinToConnectTo->PinType).ToString());

			TestTrue(FinalStatusMessage, OutputPin->PinType.PinCategory == PinToConnectTo->PinType.PinCategory);
		}
	}

	// Cleanup
	{
		TypePromoTestUtils::CleanupTestPins(PinTypes);

		BP_Primitive_Connections->MarkAsGarbage();
		BP_Primitive_Connections->Rename(nullptr, nullptr, REN_DontCreateRedirectors);
		TestGraph->MarkAsGarbage();
		TestNode->MarkAsGarbage();
	}

	return true;
}

#undef MakeTestPin

#endif //WITH_DEV_AUTOMATION_TESTS