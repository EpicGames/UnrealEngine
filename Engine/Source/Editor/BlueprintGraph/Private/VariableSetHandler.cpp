// Copyright Epic Games, Inc. All Rights Reserved.

#include "VariableSetHandler.h"

#include "BPTerminal.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphUtilities.h"
#include "Engine/MemberReference.h"
#include "GameFramework/Actor.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Variable.h"
#include "K2Node_VariableSet.h"
#include "Kismet2/CompilerResultsLog.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompiler.h"
#include "Misc/AssertionMacros.h"
#include "PushModelHelpers.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "VariableSetHandler"

//////////////////////////////////////////////////////////////////////////
// FKCHandler_VariableSet

void FKCHandler_VariableSet::RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net)
{
	// This net is a variable write
	ResolveAndRegisterScopedTerm(Context, Net, Context.VariableReferences);
}

void FKCHandler_VariableSet::RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	if(UK2Node_Variable* SetterNode = Cast<UK2Node_Variable>(Node))
	{
		SetterNode->CheckForErrors(CompilerContext.GetSchema(), Context.MessageLog);

		// Report an error that the local variable could not be found
		if(SetterNode->VariableReference.IsLocalScope() && SetterNode->GetPropertyForVariable() == nullptr)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("VariableName"), FText::FromName(SetterNode->VariableReference.GetMemberName()));

			if(SetterNode->VariableReference.GetMemberScopeName() != Context.Function->GetName())
			{
				Args.Add(TEXT("ScopeName"), FText::FromString(SetterNode->VariableReference.GetMemberScopeName()));
				CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("LocalVariableNotFoundInScope_Error", "Unable to find local variable with name '{VariableName}' for @@, scope expected: @@, scope found: {ScopeName}"), Args).ToString(), Node, Node->GetGraph());
			}
			else
			{
				CompilerContext.MessageLog.Warning(*FText::Format(LOCTEXT("LocalVariableNotFound_Error", "Unable to find local variable with name '{VariableName}' for @@"), Args).ToString(), Node);
			}
		}
	}

	for (UEdGraphPin* Net : Node->Pins)
	{
		if (!Net->bOrphanedPin && (Net->Direction == EGPD_Input) && !CompilerContext.GetSchema()->IsMetaPin(*Net))
		{
			if (ValidateAndRegisterNetIfLiteral(Context, Net))
			{
				RegisterNet(Context, Net);
			}
		}
	}
}

void FKCHandler_VariableSet::InnerAssignment(FKismetFunctionContext& Context, UEdGraphNode* Node, UEdGraphPin* VariablePin, UEdGraphPin* ValuePin)
{
	FBPTerminal** VariableTerm = Context.NetMap.Find(VariablePin);
	if (VariableTerm == nullptr)
	{
		VariableTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(VariablePin));
	}

	FBPTerminal** ValueTerm = Context.LiteralHackMap.Find(ValuePin);
	if (ValueTerm == nullptr)
	{
		ValueTerm = Context.NetMap.Find(FEdGraphUtilities::GetNetFromPin(ValuePin));
	}

	if (VariableTerm && ValueTerm)
	{
		FKismetCompilerUtilities::CreateObjectAssignmentStatement(Context,
																  Node,
																  *ValueTerm,
																  *VariableTerm,
																  (UsesVariablePinAsKey() ? VariablePin : nullptr));

		if (!(*VariableTerm)->IsTermWritable())
		{
			// If the term is not explicitly marked as read-only, then we're attempting to set a variable on a const target
			if(!(*VariableTerm)->AssociatedVarProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				if(Context.EnforceConstCorrectness())
				{
					CompilerContext.MessageLog.Error(*LOCTEXT("WriteToReadOnlyContext_Error", "Variable @@ is read-only within this context and cannot be set to a new value").ToString(), VariablePin);
				}
				else
				{
					// Warn, but still allow compilation to succeed
					CompilerContext.MessageLog.Warning(*LOCTEXT("WriteToReadOnlyContext_Warning", "Variable @@ is considered to be read-only within this context and should not be set to a new value").ToString(), VariablePin);
				}
			}
			else
			{
				CompilerContext.MessageLog.Error(*LOCTEXT("WriteConst_Error", "Cannot write to const @@").ToString(), VariablePin);
			}
		}
	}
	else
	{
		if (VariablePin != ValuePin)
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ResolveValueIntoVariablePin_Error", "Failed to resolve term @@ passed into @@").ToString(), ValuePin, VariablePin);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ResolveTermPassed_Error", "Failed to resolve term passed into @@").ToString(), VariablePin);
		}
	}
}

void FKCHandler_VariableSet::GenerateAssigments(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	// SubCategory is an object type or "" for the stack frame, default scope is Self
	// Each input pin is the name of a variable

	// Each input pin represents an assignment statement
	for (int32 PinIndex = 0; PinIndex < Node->Pins.Num(); ++PinIndex)
	{
		UEdGraphPin* Pin = Node->Pins[PinIndex];

		if (CompilerContext.GetSchema()->IsMetaPin(*Pin))
		{
		}
		else if (Pin->Direction == EGPD_Input)
		{
			InnerAssignment(Context, Node, Pin, Pin);
		}
		else
		{
			CompilerContext.MessageLog.Error(*LOCTEXT("ExpectedOnlyInputPins_Error", "Expected only input pins on @@ but found @@").ToString(), Node, Pin);
		}
	}
}

void FKCHandler_VariableSet::Compile(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	GenerateAssigments(Context, Node);

	// Generate the output impulse from this node
	GenerateSimpleThenGoto(Context, *Node);
}

void FKCHandler_VariableSet::Transform(FKismetFunctionContext& Context, UEdGraphNode* Node)
{
	UK2Node_VariableSet* SetNode = Cast<UK2Node_VariableSet>(Node);
	if (!SetNode)
	{
		return;
	}

	// Expands node out to include a (local) call to the RepNotify function if necessary

	const bool bIsLocalScope = SetNode->VariableReference.IsLocalScope();

	// Local function properties can't be used in replication.
	// Additionally, if HasFieldNotificationBroadcast is true, then the net code broadcast will be executed in native code.
	if (!bIsLocalScope && !SetNode->HasFieldNotificationBroadcast())
	{
		if (SetNode->ShouldFlushDormancyOnSet())
		{
			// Create CallFuncNode
			UK2Node_CallFunction* CallFuncNode = Node->GetGraph()->CreateIntermediateNode<UK2Node_CallFunction>();
			CallFuncNode->FunctionReference.SetExternalMember(NAME_FlushNetDormancy, AActor::StaticClass());
			CallFuncNode->AllocateDefaultPins();

			// Copy self pin
			UEdGraphPin* NewSelfPin = CallFuncNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
			UEdGraphPin* OldSelfPin = Node->FindPinChecked(UEdGraphSchema_K2::PN_Self);
			NewSelfPin->CopyPersistentDataFromOldPin(*OldSelfPin);

			// link new CallFuncNode -> Set Node
			UEdGraphPin* OldExecPin = Node->FindPin(UEdGraphSchema_K2::PN_Execute);
			check(OldExecPin);

			UEdGraphPin* NewExecPin = CallFuncNode->GetExecPin();
			if (ensure(NewExecPin))
			{
				NewExecPin->CopyPersistentDataFromOldPin(*OldExecPin);
				OldExecPin->BreakAllPinLinks();
				CallFuncNode->GetThenPin()->MakeLinkTo(OldExecPin);
			}
		}

		if (SetNode->HasLocalRepNotify())
		{
			UK2Node_CallFunction* CallFuncNode = Node->GetGraph()->CreateIntermediateNode<UK2Node_CallFunction>();
			CallFuncNode->FunctionReference.SetExternalMember(SetNode->GetRepNotifyName(), SetNode->GetVariableSourceClass());
			CallFuncNode->AllocateDefaultPins();

			// Copy self pin
			UEdGraphPin* NewSelfPin = CallFuncNode->FindPinChecked(UEdGraphSchema_K2::PN_Self);
			UEdGraphPin* OldSelfPin = Node->FindPinChecked(UEdGraphSchema_K2::PN_Self);
			NewSelfPin->CopyPersistentDataFromOldPin(*OldSelfPin);

			// link Set Node -> new CallFuncNode
			UEdGraphPin* OldThenPin = Node->FindPin(UEdGraphSchema_K2::PN_Then);
			check(OldThenPin);

			UEdGraphPin* NewThenPin = CallFuncNode->GetThenPin();
			if (ensure(NewThenPin))
			{
				// Link Set Node -> Notify
				NewThenPin->CopyPersistentDataFromOldPin(*OldThenPin);
				OldThenPin->BreakAllPinLinks();
				OldThenPin->MakeLinkTo(CallFuncNode->GetExecPin());
			}
		}

		if (SetNode->IsNetProperty())
		{
			/**
			 * This code is for property dirty tracking.
			 * It works by injecting in extra nodes while compiling that will call UNetPushModelHelpers::MarkPropertyDirtyFromRepIndex.
			 * See FKCPushModelHelpers::ConstructMarkDirtyNodeForProperty for node generation.
			 */
			if (FProperty* Property = SetNode->GetPropertyForVariable())
			{
				if (UEdGraphNode* MarkPropertyDirtyNode = FKCPushModelHelpers::ConstructMarkDirtyNodeForProperty(Context, Property, Node->FindPinChecked(UEdGraphSchema_K2::PN_Self)))
				{
					// Hook up our exec pins.
					UEdGraphPin* OldThenPin = Node->FindPinChecked(UEdGraphSchema_K2::PN_Then);
					UEdGraphPin* NewThenPin = MarkPropertyDirtyNode->FindPinChecked(UEdGraphSchema_K2::PN_Then);
					UEdGraphPin* NewInPin = MarkPropertyDirtyNode->FindPinChecked(UEdGraphSchema_K2::PN_Execute);

					NewThenPin->CopyPersistentDataFromOldPin(*OldThenPin);
					OldThenPin->BreakAllPinLinks();
					OldThenPin->MakeLinkTo(NewInPin);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
