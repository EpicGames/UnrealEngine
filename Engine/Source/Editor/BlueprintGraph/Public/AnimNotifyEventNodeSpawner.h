// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEventNodeSpawner.h"

#include "AnimNotifyEventNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

UCLASS(MinimalAPI, Transient)
class UAnimNotifyEventNodeSpawner : public UBlueprintEventNodeSpawner
{
	GENERATED_BODY()

public:
	/**
	 * Creates a new UAnimNotifyEventNodeSpawner 
	 *
	 * @param  NotifyName	The name you want assigned to the event.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UAnimNotifyEventNodeSpawner* Create(const FSoftObjectPath& InSkeletonObjectPath, FName InNotifyName);

	/** @return the skeleton object path */
	const FSoftObjectPath& GetSkeletonObjectPath() const { return SkeletonObjectPath; }

private:
	/** The skeleton that supplied this notify, used for filtering */
	UPROPERTY()
	FSoftObjectPath SkeletonObjectPath;
};

#undef UE_API
