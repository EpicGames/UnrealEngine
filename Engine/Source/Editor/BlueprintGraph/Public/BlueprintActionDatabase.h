// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/NamePermissionList.h"
#include "Stats/Stats.h"
#include "Tickable.h"
#include "TickableEditorObject.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"

#define UE_API BLUEPRINTGRAPH_API

class FBlueprintActionDatabaseRegistrar;
class FReferenceCollector;
class UBlueprint;
class UBlueprintNodeSpawner;
class UClass;
class UEnum;
class UField;
class UFunction;
class UObject;
class UScriptStruct;
struct FEdGraphPinType;

/**
 * Serves as a container for all available blueprint actions (no matter the 
 * type of blueprint/graph they belong in). The actions stored here are not tied
 * to a specific ui menu; each action is a UBlueprintNodeSpawner which is 
 * charged with spawning a specific node type. Should be set up in a way where
 * class specific actions are refreshed when the associated class is regenerated.
 * 
 * @TODO:  Hook up to handle class recompile events, along with enum/struct asset creation events.
 */
class FBlueprintActionDatabase : public FGCObject, public FTickableEditorObject
{
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnDatabaseEntryUpdated, UObject*);

	typedef TMap<FObjectKey, int32>									FPrimingQueue;
	typedef TArray<TObjectPtr<UBlueprintNodeSpawner>>							FActionList;
	typedef TMap<FObjectKey, FActionList>							FActionRegistry;
	typedef TMap<FSoftObjectPath, FActionList>	FUnloadedActionRegistry;

public:
	/** Destructor */
	UE_API virtual ~FBlueprintActionDatabase();

	/**
	 * Getter to access the database singleton. Will populate the database first 
	 * if this is the first time accessing it.
	 *
	 * @return The singleton instance of FBlueprintActionDatabase.
	 */
	static UE_API FBlueprintActionDatabase& Get();

	/** Getter to access the datbase singleton, will return null if the database has not been initialized */
	static UE_API FBlueprintActionDatabase* TryGet();

	// FTickableEditorObject interface
	UE_API virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	// End FTickableEditorObject interface

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	// End FGCObject interface

	/**
	 * Will populate the database first if it hasn't been created yet, and then 
	 * returns it in its entirety. 
	 * 
	 * Each node spawner is categorized by a class orasset. A spawner that 
	 * corresponds to a specific class field (like a function, property, enum, 
	 * etc.) will be listed under that field's class owner. Remaining spawners 
	 * that can't be categorized this way will be registered by asset or node 
	 * type.
	 *
	 * @return The entire action database, which maps specific objects to arrays of associated node-spawners.
	 */
	UE_API FActionRegistry const& GetAllActions();

	/**
	 * Populates the action database from scratch. Loops over every known class
	 * and records a set of node-spawners associated with each.
	 */
	UE_API void RefreshAll();

	/**
	 * Populates the action database with all level script actions from all active editor worlds.
	 */
	UE_API void RefreshWorlds();

	/**
	 * Removes the entry with the given key on the next tick.
	 */
	UE_API void DeferredRemoveEntry(FObjectKey const& InKey);

	/**
	 * Finds the database entry for the specified class and wipes it, 
	 * repopulating it with a fresh set of associated node-spawners. 
	 *
	 * @param  Class	The class entry you want rebuilt.
	 */
	UE_API void RefreshClassActions(UClass* const Class);

	/**
	 * Finds the database entry for the specified asset and wipes it,
	 * repopulating it with a fresh set of associated node-spawners.
	 * 
	 * @param  AssetObject	The asset entry you want rebuilt.
	 */
	UE_API void RefreshAssetActions(UObject* const AssetObject);

	/**
	 * Updates all component related actions
	 */
	UE_API void RefreshComponentActions();

	/**
	 * Finds the database entry for the specified object and wipes it. The entry 
	 * won't be rebuilt, unless RefreshAssetActions() is explicitly called after.
	 * 
	 * @param  AssetObject	
	 * @return True if an entry was found and removed.
	 */
	UE_API bool ClearAssetActions(UObject* const AssetObject);

	/**
	 * Finds the database entry for the specified object and wipes it. The entry
	 * won't be rebuilt, unless RefreshAssetActions() is explicitly called after.
	 *
	 * @param  AssetObjectKey
	 * @return True if an entry was found and removed.
	 */
	UE_API bool ClearAssetActions(const FObjectKey& AssetObjectKey);
	
	/**
	 * Finds the database entry for the specified unloaded asset and wipes it.
	 * The entry won't be rebuilt, unless RefreshAssetActions() is explicitly called after.
	 *
	 * @param ObjectPath	Object's path to lookup into the database
	 */
	UE_API void ClearUnloadedAssetActions(const FSoftObjectPath& ObjectPath);

	/**
	 * Moves the unloaded asset actions from one location to another
	 *
	 * @param SourceObjectPath	The object path that the data can currently be found under
	 * @param TargetObjectPath	The object path that the data should be moved to
	 */
	UE_API void MoveUnloadedAssetActions(const FSoftObjectPath& SourceObjectPath, const FSoftObjectPath& TargetObjectPath);

	/** */
	FOnDatabaseEntryUpdated& OnEntryUpdated() { return EntryRefreshDelegate; }
	/** */
	FOnDatabaseEntryUpdated& OnEntryRemoved() { return EntryRemovedDelegate; }

	/**
	 * Field visibility is different depending on context, this enum differentiates between the different
	 * contexts that fields can be present in
	 */
	enum class EPermissionsContext
	{
		/** A property on a class - from a user perspective a variable in the My Blueprint tab */
		Property,
		
		/** An asset - e.g. an anim blueprint asset player or skeleton notify */
		Asset,
		
		/** A K2 node in a graph - e.g. a function call */
		Node,
		
		/** An exposed pin on a K2 node */
		Pin
	};
	
	/** Check whether the global filter applies to this class */
	static UE_API bool IsClassAllowed(UClass const* InClass, EPermissionsContext InContext);
	static UE_API bool IsClassAllowed(const FTopLevelAssetPath& InClassPath, EPermissionsContext InContext);

	/** Check whether we have any class filtering applied */
	static UE_API bool HasClassFiltering();

	/** Check whether permissions allow this field */
	static UE_API bool IsFieldAllowed(UField const* InField, EPermissionsContext InContext);

	/** Check whether permissions allow this function */
	static UE_API bool IsFunctionAllowed(UFunction const* InFunction, EPermissionsContext InContext);

	/** Check whether permissions allow this enum */
	static UE_API bool IsEnumAllowed(UEnum const* InEnum, EPermissionsContext InContext);
	static UE_API bool IsEnumAllowed(const FTopLevelAssetPath& InEnumPath, EPermissionsContext InContext);

	/** Check whether permissions allow this struct */
	static UE_API bool IsStructAllowed(UScriptStruct const* InStruct, EPermissionsContext InContext);
	static UE_API bool IsStructAllowed(const FTopLevelAssetPath& InStructPath, EPermissionsContext InContext);
	
	/** Check whether permissions allow this pin type. InUnloadedAssetPath will be used in the case where a InPinType.PinSubCategoryObject is unable to be resolved. */
	static UE_API bool IsPinTypeAllowed(const FEdGraphPinType& InPinType, const FTopLevelAssetPath& InUnloadedAssetPath = FTopLevelAssetPath());
	
private:
	/** Private constructor for singleton purposes. */
	UE_API FBlueprintActionDatabase();
	UE_API void Init();

	/**
	 * 
	 * 
	 * @param  Registrar	
	 */
	UE_API void RegisterAllNodeActions(FBlueprintActionDatabaseRegistrar& Registrar);

	/**
	 * This exists only because we need a pointer to associate our delegates with
	 */
	UE_API void OnBlueprintChanged( UBlueprint* );

	/** Refresh other systems before a full or partial refresh */
	UE_API void PreRefresh(bool bRefreshAll);
private:
	/** 
	 * A map of associated node-spawners for each class/asset. A spawner that 
	 * corresponds to a specific class field (like a function, property, enum, 
	 * etc.) will be mapped under that field's class outer. Other spawners (that
	 * can't be associated with a class outer), will be filed under the desired
	 * node's type, or an associated asset.
	 */
	FActionRegistry ActionRegistry;

	/** 
	 * A map of associated object paths for each node-class that is associated
	 * with it. This is used for unloaded assets that will need to be replaced
	 * after the asset is loaded with the final (and more complete) nodespawner.
	 */
	FUnloadedActionRegistry UnloadedActionRegistry;

	/** 
	 * References newly allocated actions that need to be "primed". Priming is 
	 * something we do on Tick() aimed at speeding up performance (like pre-
	 * caching each spawner's template-node, etc.).
	 */
	FPrimingQueue ActionPrimingQueue;

	/** List of action keys to be removed on the next tick. */
	TArray<FObjectKey> ActionRemoveQueue;

	/** */
	FOnDatabaseEntryUpdated EntryRefreshDelegate;
	FOnDatabaseEntryUpdated EntryRemovedDelegate;

	/** Handles to registered delegates. */
	FDelegateHandle OnAssetLoadedDelegateHandle;
	FDelegateHandle OnAssetAddedDelegateHandle;
	FDelegateHandle OnAssetRemovedDelegateHandle;
	FDelegateHandle OnAssetRenamedDelegateHandle;
	FDelegateHandle OnAssetsPreDeleteDelegateHandle;
	FDelegateHandle OnBlueprintUnloadedDelegateHandle;
	FDelegateHandle OnWorldAddedDelegateHandle;
	FDelegateHandle OnWorldDestroyedDelegateHandle;
	FDelegateHandle RefreshLevelScriptActionsDelegateHandle;
	FDelegateHandle OnModulesChangedDelegateHandle;
	FDelegateHandle OnReloadCompleteDelegateHandle;

	/** Pointer to the shared list of currently existing component types */
	const TArray<struct FComponentTypeEntry>* ComponentTypes;
};

#undef UE_API
