// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "BlueprintActionDatabase.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/Platform.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

#define UE_API BLUEPRINTGRAPH_API

class IAssetTools;
class UBlueprintNodeSpawner;
class UClass;
class UEdGraphNode;
class UEnum;
class UField;
class UFunction;
class UObject;
class UScriptStruct;
struct FAssetData;

/**
 * From the BlueprintActionDatabase, passed around to all UK2Nodes, giving each 
 * a chance to register its own actions (specifically for UK2Nodes in other
 * modules that the database doesn't have access to).
 */
class FBlueprintActionDatabaseRegistrar
{
public:
	/**
	 * Attempts to suss out the key that this action should be registered under;
	 * if it doesn't find a better one then it associates the action with the
	 * node filling this out.
	 *
	 * @param  NodeSpawner	The new node spawner that you wish to register.
	 */
	UE_API bool AddBlueprintAction(UBlueprintNodeSpawner* NodeSpawner);

	/**
	 * Each action should be recorded under a specific UField key; primarily to
	 * refresh those actions when the corresponding asset is updated (Blueprint
	 * regenerated, struct added/deleted, etc.).
	 *
	 * @param  FieldOwner	A field object that the node is associated with (flags when this action should be updated).
	 * @param  NodeSpawner	The new node spawner that you wish to register.
	 */
	UE_API bool AddBlueprintAction(UClass const* ClassOwner, UBlueprintNodeSpawner* NodeSpawner);
	UE_API bool AddBlueprintAction(UEnum const* EnumOwner, UBlueprintNodeSpawner* NodeSpawner);
	UE_API bool AddBlueprintAction(UScriptStruct const* StructOwner, UBlueprintNodeSpawner* NodeSpawner);
	UE_API bool AddBlueprintAction(UField const* FieldOwner, UBlueprintNodeSpawner* NodeSpawner);
	UE_API bool AddBlueprintAction(FAssetData const& AssetDataOwner, UBlueprintNodeSpawner* NodeSpawner);

	/**
	 * Special case for asset bound actions (we want to clean-up/refresh these
	 * when the corresponding asset is updated). AssetOwner must be an asset!
	 *
	 * @param  AssetOwner	An asset object that the node is associated with (flags when this action should be updated).
	 * @param  NodeSpawner	The new node spawner that you wish to register.
	 */
	UE_API bool AddBlueprintAction(UObject const* AssetOwner, UBlueprintNodeSpawner* NodeSpawner);

	/**
	 * Occasionally (when an asset is added/refreshed), this registrar will be
	 * passed around to gather only specific keyed actions (see ActionKeyFilter).
	 * In that case, it will block registration of all unwanted keys.
	 * Functionality wise this doesn't matter to UK2Node, but UK2Node may be
	 * able to save on some work/allocations if it knew this beforehand.
	 *
	 * @param  OwnerKey		The key you wish to register your action(s) under.
	 * @return True if the OwnerKey would is allowed to register actions, false if it would be blocked.
	 */
	UE_API bool IsOpenForRegistration(UObject const* OwnerKey);
	UE_API bool IsOpenForRegistration(FAssetData const& AssetDataOwner);

	/** Returns the current key the registrar is being filtered using */
	UObject const* GetActionKeyFilter() const
	{
		return ActionKeyFilter;
	}

	/// 
	DECLARE_DELEGATE_RetVal_OneParam(UBlueprintNodeSpawner*, FMakeStructSpawnerDelegate, const UScriptStruct*);
	/**
	*
	*
	* @param  MakeActionCallback
	* @return
	*/
	UE_API int32 RegisterStructActions(const FMakeStructSpawnerDelegate& MakeActionCallback);

	/// 
	DECLARE_DELEGATE_RetVal_OneParam(UBlueprintNodeSpawner*, FMakeEnumSpawnerDelegate, const UEnum*);
	/**
	 *
	 *
	 * @param  MakeActionCallback
	 * @return
	 */
	UE_API int32 RegisterEnumActions(const FMakeEnumSpawnerDelegate& MakeActionCallback);

	/// 
	DECLARE_DELEGATE_RetVal_OneParam(UBlueprintNodeSpawner*, FMakeFuncSpawnerDelegate, const UFunction*);
	/**
	 *
	 *
	 * @param  MakeActionCallback
	 * @return
	 */
	template<class T>
	int32 RegisterClassFactoryActions(const FMakeFuncSpawnerDelegate& MakeActionCallback)
	{
		return RegisterClassFactoryActions(T::StaticClass(), MakeActionCallback);
	}

private:
	typedef FBlueprintActionDatabase::FActionRegistry			FActionRegistry;
	typedef FBlueprintActionDatabase::FUnloadedActionRegistry	FUnloadedActionRegistry;
	typedef FBlueprintActionDatabase::FPrimingQueue				FPrimingQueue;

	/** Only FBlueprintActionDatabase can spawn and distribute this. */
	friend class FBlueprintActionDatabase;
	UE_API FBlueprintActionDatabaseRegistrar(FActionRegistry& Database, FUnloadedActionRegistry& UnloadedDatabase, FPrimingQueue& PrimingQueue, TSubclassOf<UEdGraphNode> DefaultKey = {});

	/**
	 * 
	 * 
	 * @param  TargetType    
	 * @param  MakeActionCallback    
	 * @return 
	 */
	UE_API int32 RegisterClassFactoryActions(const UClass* TargetType, const FMakeFuncSpawnerDelegate& MakeActionCallback);

	/**
	 * Internal method that actually adds the action to the database.
	 * 
	 * @param  ActionKey	The key you want the action listed under.
	 * @param  NodeSpawner	The new node spawner that you wish to register.
	 */
	UE_API bool AddActionToDatabase(UObject const* ActionKey, UBlueprintNodeSpawner* NodeSpawner);

private:
	/** The node type that this is currently passed to (acts as a fallback database key). */
	TSubclassOf<UEdGraphNode> GeneratingClass;

	/** A reference to FBlueprintActionDatabase's internal map */
	FActionRegistry& ActionDatabase;

	/** A reference to FBlueprintActionDatabase's internal map */
	FUnloadedActionRegistry& UnloadedActionDatabase;

	/** 
	 * When an asset is added/updated we want to poll nodes again for 
	 * new/refreshed actions, but only for that specific asset. This serves as
	 * a filter in that scenario (to keep nodes from adding unrelated, probably
	 * duplicate, actions).
	 */
	UObject const* ActionKeyFilter;

	/** A reference to FBlueprintActionDatabase's action priming queue */
	FPrimingQueue& ActionPrimingQueue;

	/** Cached ptr to asset tools to avoid LoadModule calls per-AddBlueprintAction call */
	IAssetTools* AssetTools;
};

#undef UE_API
