// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintGraphModule.h"
#include "BlueprintNodeBinder.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API BLUEPRINTGRAPH_API

#define ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) && 0)

class FProperty;
class UClass;
class UFunction;
class UObject;
class IAssetReferenceFilter;
class IBlueprintEditor;
class UBlueprint;
class UBlueprintNodeSpawner;
class UEdGraph;
class UEdGraphPin;
class UEdGraphNode;

/*******************************************************************************
 * FBlueprintActionContext
 ******************************************************************************/

struct FBlueprintActionContext
{
	/**
	 * A weak reference to the Blueprint editor context in which actions are
	 * being filtered. Note that this may be NULL or invalid if we're filtering
	 * actions outside of an editor context (e.g. commandlet).
	 */
	TWeakPtr<IBlueprintEditor> EditorPtr;

	/** 
	 * A list of all blueprints you want actions for. Generally, this will
	 * only contain a single blueprint, but it can have many (where an action
	 * has to be available for every blueprint listed to pass the filter).
	 */
	TArray<UBlueprint*> Blueprints;

	/** 
	 * A list of graphs you want compatible actions for. Generally, this will
	 * contain a single graph, but it can have several (where an action has to 
	 * be viable for every graph to pass the filter).
	 */
	TArray<UEdGraph*> Graphs;

	/** 
	 * A list of pins you want compatible actions for. Generally, this will
	 * contain a single pin, but it can have several (where an action has to 
	 * be viable for every pin to pass the filter).
	 */
	TArray<UEdGraphPin*> Pins;

	/** 
	 * A list of objects the user currently has selected (things like blueprint 
	 * properties, level actors, content-browser assets, etc.). Bound actions
	 * have to be tied to one of these objects in order to pass the filter.
	 */
	TArray<FFieldVariant> SelectedObjects;
};

/*******************************************************************************
 * FBlueprintActionInfo
 ******************************************************************************/

/**
 * Info struct passed around to filter rejection tests. Wraps a 
 * UBlueprintNodeSpawner, and caches associated fields/files/etc. as they're 
 * requested (to optimize duplicated queries cross rejection tests).
 */
struct FBlueprintActionInfo
{
	/** */
	UE_API FBlueprintActionInfo(UObject const* ActionOwner, UBlueprintNodeSpawner const* Action);

	/** */
	UE_API FBlueprintActionInfo(FBlueprintActionInfo const& Rhs, IBlueprintNodeBinder::FBindingSet const& Bindings);

	/**
	 * Retrieves the key that the wrapped action is associated with in the 
	 * FBlueprintActionDatabase (either a UClass, or asset object).
	 * 
	 * @return The class/asset that this action conceptually belongs to.
	 */
	UE_API UObject const* GetActionOwner() const;

	/**
	 * Retrieves any bindings that the action will apply to spawned nodes.
	 * 
	 * @return The set of bindings tied to this action.
	 */
	UE_API IBlueprintNodeBinder::FBindingSet const& GetBindings() const;

	/**
	 * Retrieves a class associated with the wrapped action. Intended to be the 
	 * action's class "owner". Could be null if the action is keyed to an asset.
	 * Will not be the spawner's NodeClass (even if it is keyed to it in the 
	 * database).
	 * 
	 * @return The class that this action conceptually belongs to (null if the action is keyed to an asset that we cannot derive a class from).
	 */
	UE_API UClass const* GetOwnerClass();

	/**
	 * Retrieves the node class that the wrapped action will spawn (assume to 
	 * be not null).
	 * 
	 * @return The node type that the action will spawn.
	 */
	UE_API UClass const* GetNodeClass();

	/**
	 * Certain actions are associated with specific member fields (a member 
	 * function call, a variable get/set, etc.) This retrieves that member field 
	 * if there is one (not all actions have an associated field).
	 * 
	 * @return The member field associated with the wrapped action (null if there isn't one).
	 */
	UE_API FFieldVariant GetAssociatedMemberField();

	/**
	 * Certain actions are associated with specific properties (like delegate  
	 * node spawners, or variable get/set spawners) This retrieves that property
	 * from the wrapped action if it can (not all actions have an associated 
	 * property).
	 * 
	 * @return The property associated with the wrapped action (null if there isn't one).
	 */
	UE_API FProperty const* GetAssociatedProperty();

	/**
	 * Certain actions are associated with specific functions (like function    
	 * call spawners, or event spawners) This retrieves the function from the 
	 * wrapped action if it can (not all actions have an associated function).
	 * 
	 * @return The function associated with the wrapped action (null if there isn't one).
	 */
	UE_API UFunction const* GetAssociatedFunction();

	/** The raw action that this struct represent (const so we don't mutate the database) */
	UBlueprintNodeSpawner const* const NodeSpawner;

private:
	/** The class or asset-object that the NodeSpawner action is keyed to (in the action database)*/
	TWeakObjectPtr<const UObject> ActionOwner;

	/** Keeps track of the fields we've cached (needed in case one turns out to be null) */
	uint32 CacheFlags;
	
	/** */
	TWeakObjectPtr<const UClass>	CachedOwnerClass;
	FFieldVariant					CachedActionField;
	FProperty const*				CachedActionProperty;
	UFunction const*				CachedActionFunction;

	/** */
	IBlueprintNodeBinder::FBindingSet Bindings;
};

/*******************************************************************************
 * EBlueprintActionFilterRejectionTestFlags
 ******************************************************************************/

// @todo - Need to revisit this later; for now these flags remain in place for backcompat but are currently not in use.
enum class EActionFilterTestFlags
{
	None = 0,

	/**
	 * these tests will be have their combined result cached for faster lookup.
	 * note: rejection tests can only be cached if they either
	 *		A: Don't access any mutable data
	 *		or B: Only access mutable data that is listed as a filter key in FActionFilterCache::GetCachedFilterResults
	 */
	 CacheResults = 1 << 0,

	 Default = None, // this would allow us to change the default behavior
};

ENUM_CLASS_FLAGS(EActionFilterTestFlags);

/*******************************************************************************
 * FActionFilterTest
******************************************************************************/

// @todo - Need to revisit this later; for now this wrapper type remains in place for backcompat, but flags are currently not in use.
struct FActionFilterTest : TSharedFromThis<FActionFilterTest>
{
	static UE_API TAutoConsoleVariable<bool> CVarEnableCaching;

	/** The filter uses a series of rejection tests matching */
	typedef FBlueprintGraphModule::FActionMenuRejectionTest FRejectionDelegate;

	UE_API FActionFilterTest(const FRejectionDelegate Delegate, FString Name, EActionFilterTestFlags Flags = EActionFilterTestFlags::Default);

	TSharedRef<FActionFilterTest> WithFlags(EActionFilterTestFlags InFlags)
	{
		EnumAddFlags(Flags, InFlags);
		return AsShared();
	}

	UE_API bool Call(FBlueprintActionFilter const& Filter, FBlueprintActionInfo& ActionInfo);
	UE_API bool operator==(const FActionFilterTest& other) const;

	FRejectionDelegate RejectionDelegate;
	FString Name; // this name must be unique
	EActionFilterTestFlags Flags = EActionFilterTestFlags::Default;
};

// @todo - Need to revisit this; currently defined for backcompat but we may deprecate its use later as it is somewhat rigid.
// constructor for TSharedRef<FBlueprintActionFilterRejectionTest> that auto deduces the name
#define MAKE_ACTION_FILTER_REJECTION_TEST(TestFunc, ...) \
	MakeShared<FActionFilterTest>(FActionFilterTest::FRejectionDelegate::CreateStatic(TestFunc, ##__VA_ARGS__), TEXT(#TestFunc))

/*******************************************************************************
 * FBlueprintActionFilter
 ******************************************************************************/

class FBlueprintActionFilter
{
public:
	/** The filter uses a series of rejection tests matching */
	typedef FBlueprintGraphModule::FActionMenuRejectionTest FRejectionTestDelegate;

	enum EFlags // Flags, which configure certain rejection tests.
	{
		BPFILTER_NoFlags					= (0),

		/** Deprecated class actions will not be filtered out*/
		BPFILTER_PermitDeprecated			= (1<<0),

		/** 
		 * Rejects actions associated with global/static ("persistent") fields.
		 * However, static members of TargetClasses are kept, and not outright 
		 * rejected by this (if TargetClasses is left empty, then all 
		 * "persistent" fields are rejected).
		 */
		BPFILTER_RejectGlobalFields			= (1<<1),

		/**
		 * Makes PermittedNodeType tests more aggressive by rejecting node 
		 * sub-classes, (actions would have to explicitly match a class listed
		 * in PermittedNodeTypes). 
		 */
		BPFILTER_RejectPermittedSubClasses	= (1<<2),

		/**
		 * Makes RejectedNodeType tests less aggressive by permitting node 
		 * sub-classes, (actions would have to explicitly match a class listed
		 * in RejectedNodeTypes). 
		 */
		BPFILTER_PermitRejectionSubClasses	= (1<<3),

		/**
		 * Actions that reference member variables or functions that belong
		 * to a struct that exists outside the scope of the current set of
		 * imported namespaces will be filtered out.
		 */
		BPFILTER_RejectNonImportedFields	= (1<<4),

		/**
		 * Rejects actions that would spawn nodes with thread safety characteristics
		 * that are incompatible with the specification of the containing graph
		 */
		BPFILTER_RejectIncompatibleThreadSafety = (1<<5)
	};

	/** @param InFlags - Filter configuration flags. */
	UE_API FBlueprintActionFilter(const EFlags InFlags = BPFILTER_NoFlags);
	
	/**
	 * Contains the full blueprint/graph/pin context that this is filtering 
	 * actions for.
	 */
	FBlueprintActionContext Context;	
	
	/**
	 * A list of allowed node types. If a spawner's NodeClass isn't one of
	 * these types, then it is filtered out. Use the "RejectPermittedNodeSubClasses"
	 * flag to aggressively filter out child classes as well (enforcing an 
	 * explicit match).
	 */
	TArray< TSubclassOf<UEdGraphNode> > PermittedNodeTypes;
	
	/**
	 * A list of node types that should be filtered out. If a node class is 
	 * listed both here and in PermittedNodeTypes, then the exclusion wins (it 
	 * will be filtered out).
	 */
	TArray< TSubclassOf<UEdGraphNode> > RejectedNodeTypes;
	
	/**
	 * A list of classes that you want members for. If an action would produce
	 * a node with a TargetPin, and that pin is incompatible with one of these
	 * classes, then the action is filtered out. Extra data is cached in 
	 * FTargetClassFilterData so that it can be reused by filters efficiently.
	 */
	struct FTargetClassFilterData
	{
		UClass* TargetClass;
		TArray<FString> HiddenCategories;
		bool operator==(const FTargetClassFilterData& Other) const
		{
			return TargetClass == Other.TargetClass && HiddenCategories == Other.HiddenCategories;
		}
	};
	TArray<FTargetClassFilterData> TargetClasses;

	/** Helper to add a class to the TargetClasses, TargetClass may already be in the array */
	static UE_API void AddUnique(TArray<FTargetClassFilterData>& ToArray, UClass* TargetClass);

	/** Helper to add a class to the TargetClasses, fills out FTargetClassFilterData */
	static UE_API void Add(TArray<FTargetClassFilterData>& ToArray, UClass* TargetClass);

	/** Cached reference to the BluprintGraphModule, which has extra rejection tests: */
	class FBlueprintGraphModule* BluprintGraphModule;

	/** Filter for asset references */
	TSharedPtr<IAssetReferenceFilter> AssetReferenceFilter;

	/**
	 * Users can extend the filter and add their own rejection tests with this
	 * method. We use rejection "IsFiltered" tests rather than inclusive tests 
	 * because it is more optimal to whittle down the list of actions early.
	 *
	 * @param  RejectionTestDelegate	The rejection test you wish to add to this filter.
	 */
	UE_API void AddRejectionTest(FRejectionTestDelegate RejectionTestDelegate);

	/**
	 * @param  RejectionTest	a test this filter will run to cull blueprint actions
	 * @param  Flags	List of flags that will be added in the provided rejection test
	 */
	UE_API void AddRejectionTest(TSharedRef<FActionFilterTest> RejectionTest);

	/**
	 * Query to check and see if the specified action gets filtered out by this 
	 * (and any and'd/or'd filters). NOT marked const to keep 
	 * FIsFilteredDelegates from recursively calling.
	 * 
	 * @param  BlueprintAction	The node-spawner you wish to test.
	 * @return False if the action passes the filter, otherwise false (the action got filtered out).
	 */
	UE_API bool IsFiltered(FBlueprintActionInfo& BlueprintAction);

	/**
	 * Appends another filter to be utilized in IsFiltered() queries, extending  
	 * the query to be: IsFilteredByThis() || Rhs.IsFiltered()
	 *
	 * NOTE: Appending additional filters can hinder performance (as the
	 *       additional filter has to loop over its tests foreach database entry)
	 * 
	 * @param  Rhs	The filter to append to this one.
	 * @return This.
	 */
	UE_API FBlueprintActionFilter const& operator|=(FBlueprintActionFilter const& Rhs);

	/**
	 * Appends another filter to be utilized in IsFiltered() queries, extending  
	 * the query to be: IsFilteredByThis() && Rhs.IsFiltered()
	 *
	 * NOTE: Appending additional filters can hinder performance (as the
	 *       additional filter has to loop over its tests foreach database entry)
	 * 
	 * @param  Rhs	The filter to append to this one.
	 * @return This.
	 */
	UE_API FBlueprintActionFilter const& operator&=(FBlueprintActionFilter const& Rhs);

	/**
	 * @return TRUE if any of the given configuration flags are set on this filter.
	 */
	inline bool HasAnyFlags(EFlags InFlags) const
	{
		return (FilterFlags & InFlags) != 0;
	}

	/**
	 * @return TRUE if all of the given configuration flags are set on this filter.
	 */
	inline bool HasAllFlags(EFlags InFlags) const
	{
		return (FilterFlags & InFlags) == InFlags;
	}

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	static UE_API bool IsFilterTestTraceLoggingEnabled();
	static UE_API bool IsFilterTestStatsLoggingEnabled();
	UE_API TArray<FString> GetFilterTestProfile();
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING

private:
	/**
	 * Query to check and see if the specified action gets filtered out by this 
	 * (does not take into consideration any and'd/or'd filters).
	 * 
	 * @param  BlueprintAction	The node-spawner you wish to test.
	 * @return False if the action passes the filter, otherwise true (the action got filtered out).
	 */
	UE_API bool IsFilteredByThis(FBlueprintActionInfo& BlueprintAction) const;

	/** Configuration flags for this filter. */
	EFlags FilterFlags = BPFILTER_NoFlags;

	/** Set of rejection tests for this specific filter. */
	TArray<FRejectionTestDelegate> FilterTests;

	/** Filters to be logically and'd in with the IsFilteredByThis() result. */
	TArray<FBlueprintActionFilter> AndFilters;

	/** Alternative filters to be logically or'd in with the IsFilteredByThis() result. */
	TArray<FBlueprintActionFilter> OrFilters;

#if ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
	struct FFilterTestProfileRecord
	{
		uint64 TestFuncPtr = 0;
		float TotalTimeMs = 0.0f;
		int32 NumIterations = 0;
		int32 NumFilteredOut = 0;
		bool bOutputBeginEventEmitted = false;
	};

	mutable TMap<int32, FFilterTestProfileRecord> FilterTestProfiles;

	UE_API FFilterTestProfileRecord* BeginFilterTestProfileEvent(int32 FilterTestIndex, uint64 FilterTestFuncPtr) const;
	UE_API void EndFilterTestProfileEvent(FFilterTestProfileRecord* InEvent, bool bWasFiltered, double StartTime) const;
#endif	// ENABLE_BLUEPRINT_ACTION_FILTER_PROFILING
};

ENUM_CLASS_FLAGS(FBlueprintActionFilter::EFlags);

#undef UE_API
