// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Engine/DeveloperSettings.h"
#include "FindInBlueprints.h"
#include "Internationalization/Text.h"
#include "Kismet2/Breakpoint.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Kismet2/WatchedPin.h"
#include "Misc/NamePermissionList.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintEditorSettings.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UObject;
struct FAssetData;
struct FBPEditorBookmarkNode;
struct FEditedDocumentInfo;
struct FGuid;

UENUM()
enum ESaveOnCompile : int
{
	SoC_Never UMETA(DisplayName="Never"),
	SoC_SuccessOnly UMETA(DisplayName="On Success Only"),
	SoC_Always UMETA(DisplayName = "Always"),
};

/** Blueprint Editor settings that are different for each
*	blueprint.
*	See FKismetDebugUtilities for helper functions
*/
USTRUCT()
struct FPerBlueprintSettings 
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FBlueprintBreakpoint> Breakpoints;

	UPROPERTY()
	TArray<FBlueprintWatchedPin> WatchedPins;

	bool operator==(const FPerBlueprintSettings& Other) const
	{
		return Breakpoints == Other.Breakpoints && WatchedPins == Other.WatchedPins;
	}
};

template<> struct TStructOpsTypeTraits<FPerBlueprintSettings> : public TStructOpsTypeTraitsBase2<FPerBlueprintSettings>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

USTRUCT()
struct FAdditionalBlueprintCategory
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Category)
	FText Name;

	UPROPERTY(EditAnywhere, Category = Category)
	FSoftClassPath ClassFilter;
};

UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UBlueprintEditorSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	// Style Settings

	/** Should arrows indicating data/execution flow be drawn halfway along wires? */
	UPROPERTY(EditAnywhere, config, Category=VisualStyle, meta=(DisplayName="Draw midpoint arrows in Blueprints"))
	bool bDrawMidpointArrowsInBlueprints;

	/** Determines if lightweight tutorial text shows up at the top of empty blueprint graphs */
	UPROPERTY(EditAnywhere, config, Category = VisualStyle)
	bool bShowGraphInstructionText;

	/** If true, fade nodes which are not connected to the selected nodes */
	UPROPERTY(EditAnywhere, config, Category = VisualStyle)
	bool bHideUnrelatedNodes;

	/** If true, use short tooltips whenever possible */
	UPROPERTY(EditAnywhere, config, Category = VisualStyle)
	bool bShowShortTooltips;

	/** If true, show an icon next to function parameter nodes. */
	UPROPERTY(EditAnywhere, config, Category = VisualStyle)
	bool bShowFunctionParameterIcon;

	/** If true, show an icon next to function local variable nodes. */
	UPROPERTY(EditAnywhere, config, Category = VisualStyle)
	bool bShowFunctionLocalVariableIcon;

	// Workflow Settings

	/** 
	  * If enabled Input Action nodes will hide unsupported trigger pins and give warnings when using unsupported triggers.
	  * This warning only works with triggers set up in an Input Action, not Input Mapping Contexts. 
	  */
	UPROPERTY(EditAnywhere, Config, Category=Workflow, meta=(DisplayName="Enhanced Input: Enable Input Trigger Support Warnings"))
	bool bEnableInputTriggerSupportWarnings;
	
	/** If enabled, we'll save off your chosen target setting based off of the context (allowing you to have different preferences based off what you're operating on). */
	UPROPERTY(EditAnywhere, config, Category=Workflow, meta=(DisplayName="Context Menu: Divide Context Target Preferences"))
	bool bSplitContextTargetSettings;

	/** If enabled, then ALL component functions are exposed to the context menu (when the contextual target is a component owner). Ignores "ExposeFunctionCategories" metadata for components. */
	UPROPERTY(EditAnywhere, config, Category=Workflow, meta=(DisplayName="Context Menu: Expose All Sub-Component Functions"))
	bool bExposeAllMemberComponentFunctions;

	/** If enabled, then a separate section with your Blueprint favorites will be pinned to the top of the context menu. */
	UPROPERTY(EditAnywhere, config, Category=Workflow, meta=(DisplayName="Context Menu: Show Favorites Section"))
	bool bShowContextualFavorites;

	/** If enabled, deprecated functions will be visible in the context menu and will be available for override implementation. By default, functions marked as deprecated are not exposed in either case. */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bExposeDeprecatedFunctions;

	/** If enabled, then call-on-member actions will be spawned as a single node (instead of a GetMember + FunctionCall node). */
	UPROPERTY(EditAnywhere, config, Category=Workflow)
	bool bCompactCallOnMemberNodes;

	/** If enabled, then your Blueprint favorites will be uncategorized, leaving you with less nested categories to sort through. */
	UPROPERTY(EditAnywhere, config, Category=Workflow)
	bool bFlattenFavoritesMenus;

	/** If enabled, then you'll be able to directly connect arbitrary object pins together (a pure cast node will be injected automatically). */
	UPROPERTY(EditAnywhere, config, Category=Workflow)
	bool bAutoCastObjectConnections;

	/** If true will show the viewport tab when simulate is clicked. */
	UPROPERTY(EditAnywhere, config, Category=Workflow)
	bool bShowViewportOnSimulate;

	/** If set will spawn default "ghost" event nodes in new Blueprints, modifiable in the [DefaultEventNodes] section of EditorPerProjectUserSettings */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bSpawnDefaultBlueprintNodes;

	/** If set will exclude components added in a Blueprint class Construction Script from the component details view */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bHideConstructionScriptComponentsInDetailsView;

	/** If set, double clicking on a call function node will attempt to navigate an open C++ editor to the native source definition */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bNavigateToNativeFunctionsFromCallNodes;

	/** Double click to navigate up to the parent graph */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	bool bDoubleClickNavigatesToParent;

	/** Allows for pin types to be promoted to others, i.e. float to double */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	bool bEnableTypePromotion;

	/** When enabled the the list of compatible nodes will be shown when a connection is attempted between incompatible pins */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	bool bShowPanelContextMenuForIncompatibleConnections;

	/** If a pin type is within this list, then it will never be marked as a possible promotable function. */
	UPROPERTY(config, EditAnywhere, Category = Workflow, meta=(EditCondition="bEnableTypePromotion"))
	TSet<FName> TypePromotionPinDenyList;

	/** List of additional category names to show in Blueprints, optionally filtered by parent class type. */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	TArray<FAdditionalBlueprintCategory> AdditionalBlueprintCategories;

	/** How to handle previously-set breakpoints on reload. */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	EBlueprintBreakpointReloadMethod BreakpointReloadMethod;

	/** If enabled, pin tooltips during PIE will be interactive */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	bool bEnablePinValueInspectionTooltips;

	/** Whether to enable namespace importing and filtering features in the Blueprint editor */
	UPROPERTY(config, EditAnywhere, Category = Workflow)
	bool bEnableNamespaceEditorFeatures;

	// A list of namespace identifiers that the Blueprint editor should always import by default. Requires Blueprint namespace features to be enabled and only applies to the current local user. Editing this list will also cause any visible Blueprint editor windows to be closed.
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (EditCondition = "bEnableNamespaceEditorFeatures", DisplayName = "Global Namespace Imports (Local User Only)"))
	TArray<FString> NamespacesToAlwaysInclude;

	/** When the Blueprint graph context menu is invoked (e.g. by right-clicking in the graph or dragging off a pin), do not block the UI while populating the available actions list. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (DisplayName = "Enable Non-Blocking Context Menu"))
	bool bEnableContextMenuTimeSlicing;

	/** The maximum amount of time (in milliseconds) allowed per frame for Blueprint graph context menu building when the non-blocking option is enabled. Larger values will complete the menu build in fewer frames, but will also make the UI feel less responsive. Smaller values will make the UI feel more responsive, but will also take longer to fully populate the menu. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, AdvancedDisplay, meta = (EditCondition = "bEnableContextMenuTimeSlicing", DisplayName = "Context Menu: Non-Blocking Per-Frame Threshold (ms)", ClampMin = "1"))
	int32 ContextMenuTimeSlicingThresholdMs;

	/** If enabled, invoking the Blueprint graph context menu with one or more compatible assets selected in the Content Browser will generate an additional set of pre-bound menu actions when the "Context Sensitive" option is enabled. For example, selecting a Static Mesh asset in the Content Browser will result in an extra "Add Static Mesh Component" menu action that's already bound to the selected asset. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (DisplayName = "Context Menu: Include Pre-Bound Actions for Selected Assets"))
	bool bIncludeActionsForSelectedAssetsInContextMenu;

	/** Only generate pre-bound "Add Component" actions when there is a single asset selected in the Content Browser. If more than one asset is selected, pre-bound "Add Component" actions will not be generated. Enabling this option can improve UI responsiveness and decrease the time it takes to build the context menu, while still preserving the ability to include actions pre-bound to the selected asset. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (EditCondition = "bIncludeActionsForSelectedAssetsInContextMenu", DisplayName = "Context Menu: Pre-Bound Actions: Restrict to Single Selection"))
	bool bLimitAssetActionBindingToSingleSelectionOnly;

	/** When generating pre-bound "Add Component" actions, any selected assets that are not yet loaded will be synchronously loaded as part of building the Blueprint Graph context menu. Enabling this option will ensure that all pre-bound actions for all selected assets are included in the menu, but load times may also affect editor UI responsiveness while the context menu is building. */
	UPROPERTY(EditAnywhere, config, Category = Workflow, meta = (EditCondition = "bIncludeActionsForSelectedAssetsInContextMenu", DisplayName = "Context Menu: Pre-Bound Actions: Always Load Selected Asset(s)"))
	bool bLoadSelectedAssetsForContextMenuActionBinding;

	/** If enabled, assets containing Blueprint instances (e.g. maps) will not be marked dirty when default values are edited, unless it results in the instance becoming realigned with the new default value. */
	UPROPERTY(EditAnywhere, config, Category = Workflow)
	bool bDoNotMarkAllInstancesDirtyOnDefaultValueChange;

	// Experimental

	/** If enabled, then placed cast nodes will default to their "pure" form (meaning: without execution pins). */
	UPROPERTY(EditAnywhere, config, AdvancedDisplay, Category = Experimental, meta = (DisplayName = "Default to Using Pure Cast Nodes"))
	bool bFavorPureCastNodes;

	// Compiler Settings

	/** Determines when to save Blueprints post-compile */
	UPROPERTY(EditAnywhere, config, Category = Compiler)
	TEnumAsByte<ESaveOnCompile> SaveOnCompile;

	/** When enabled, if a blueprint has compiler errors, then the graph will jump and focus on the first node generating an error */
	UPROPERTY(EditAnywhere, config, Category = Compiler)
	bool bJumpToNodeErrors;

	/** If enabled, nodes with execution pins can be disabled via context menu. Disabled nodes will not be compiled, but also will not break existing connections. */
	UPROPERTY(EditAnywhere, config, Category = Compiler, meta = (DisplayName = "Show Node Enable/Disable Options"))
	bool bAllowExplicitImpureNodeDisabling;

	// Developer Settings

	/** If enabled, tooltips on action menu items will show the associated action's signature id (can be used to setup custom favorites menus).*/
	UPROPERTY(EditAnywhere, config, Category = DeveloperTools)
	bool bShowActionMenuItemSignatures;

	/** If enabled, blueprint nodes in the event graph will display with unique names rather than their display name. */
	UPROPERTY(EditAnywhere, config, Category = DeveloperTools, meta = (DisplayName = "Display Unique Names for Blueprint Nodes"))
	bool bBlueprintNodeUniqueNames;

	// Perf Settings

	/** The node template cache is used to speed up blueprint menuing. This determines the peak data size for that cache. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, config, Category = Performance, DisplayName = "Node-Template Cache Cap (MB)", meta = (ClampMin = "0", UIMin = "0"))
	float NodeTemplateCacheCapMB;

	// Find-in-Blueprint Settings

	/** Whether to enable the "Index All" action in the Find-in-Blueprints search window when blueprint assets with an out-of-date index (search metadata) are found and whether to allow automatic resaving. WARNING: Only allow "Index All" if your project is small enough that all assets can be loaded in memory at once. Only enable saving if you are allowed to potentially checkout and resave all assets. */
	UPROPERTY(EditAnywhere, config, Category = FindInBlueprints)
	EFiBIndexAllPermission AllowIndexAllBlueprints;

	/** If true we'll automatically search for references to a variable when its type is changed */
	UPROPERTY(EditAnywhere, config, Category = FindInBlueprints)
	bool bSearchForReferencesWhenVariableTypeChanged;

	/** If set we'll show the inherited variables in the My Blueprint view. */
	UPROPERTY(config)
	bool bShowInheritedVariables;

	/** If set interface functions will always show in the overrides menu, even if they are already shown in the interfaces menu */
	UPROPERTY(config)
	bool bAlwaysShowInterfacesInOverrides;

	/** If set then the parent class will be listed next to the override function name in the overrides function menu */
	UPROPERTY(config)
	bool bShowParentClassInOverrides;

	/** If set we'll show empty sections in the My Blueprint view. */
	UPROPERTY(config)
	bool bShowEmptySections;

	/** If set we'll show the access specifier of functions in the My Blueprint view */
	UPROPERTY(config)
	bool bShowAccessSpecifier;

	/** Blueprint bookmark database */
	UPROPERTY(config)
	TMap<FGuid, FEditedDocumentInfo> Bookmarks;

	/** Blueprint bookmark nodes (for display) */
	UPROPERTY(config)
	TArray<FBPEditorBookmarkNode> BookmarkNodes;

	/** Maps Blueprint path to settings such as breakpoints */
	UPROPERTY(config)
	TMap<FString, FPerBlueprintSettings> PerBlueprintSettings;

	/** If enabled, comment nodes will be included in the tree view display in the Bookmarks tab. */
	UPROPERTY(config)
	bool bIncludeCommentNodesInBookmarksTab;

	/** If enabled, only the bookmarks for the current document will be shown in the Bookmarks tab. */
	UPROPERTY(config)
	bool bShowBookmarksForCurrentDocumentOnlyInTab;

	/** Blueprint graph editor "Quick Jump" command bindings */
	UPROPERTY(config)
	TMap<int32, FEditedDocumentInfo> GraphEditorQuickJumps;

	/** Whether to enable namespace filtering features in the Blueprint editor */
	// @todo_namespaces - Remove this if/when dependent code is changed to utilize the single setting above.
	UPROPERTY(Transient)
	bool bEnableNamespaceFilteringFeatures;

	/** Whether to enable namespace importing features in the Blueprint editor */
	// @todo_namespaces - Remove this if/when dependent code is changed to utilize the single setting above.
	UPROPERTY(Transient)
	bool bEnableNamespaceImportingFeatures;

	/** Whether to inherit the set of imported namespaces from the parent class hierarchy */
	// @todo_namespaces - Remove this if/when this becomes a permanent setting. For now this is experimental.
	UPROPERTY(Transient)
	bool bInheritImportedNamespacesFromParentBP;

	/**
	 * Any blueprint deriving from one of these base classes will be allowed to recompile during Play-in-Editor
	 * (This setting exists both as an editor preference and project setting, and will be allowed if listed in either place) 
	 */
	UE_DEPRECATED(5.6, "This is now the default behavior (true). If you have classes that should not compile during PIE add them to BaseClassesToDisallowRecompilingDuringPlayInEditor.")
	UPROPERTY(config)
	TArray<TSoftClassPtr<UObject>> BaseClassesToAllowRecompilingDuringPlayInEditor;
	
	/** 
	 * Any blueprint deriving from one of these base classes will be disallowed to recompile during Play-in-Editor
	 * (This setting exists both as an editor preference and project setting, and will be allowed if listed in either place) 
	 */
	UPROPERTY(EditAnywhere, config, Category=Play, meta=(AllowAbstract, ShowObjectRootClass))
	TArray<TSoftClassPtr<UObject>> BaseClassesToDisallowRecompilingDuringPlayInEditor;

	/** Get allowed functions permissions list */
	FPathPermissionList& GetFunctionPermissions() { return FunctionPermissions; }

	/** Get allowed structs permissions list */
	FPathPermissionList& GetStructPermissions() { return StructPermissions; }

	/** Get allowed enums permissions list */
	FPathPermissionList& GetEnumPermissions() { return EnumPermissions; }

	/** Get allowed pin categories permissions list */
	FNamePermissionList& GetPinCategoryPermissions() { return PinCategoryPermissions; }

	using FOnIsClassAllowed = TDelegate<bool(const UClass* InClass)>;
	using FOnIsClassPathAllowed = TDelegate<bool(const FTopLevelAssetPath& InClassPath)>;
	using FOnIsFunctionAllowedForAsset = TDelegate<bool(const UBlueprint* InBlueprint, const FName InFunctionName)>;
	
	/** Delegates called to determine whether a class type is allowed */
	UE_API void RegisterIsClassAllowedDelegate(const FName OwnerName, FOnIsClassAllowed Delegate);
	UE_API void UnregisterIsClassAllowedDelegate(const FName OwnerName);
	UE_API bool IsClassAllowed(const UClass* InClass) const;
	bool HasClassFiltering() const { return IsClassAllowedDelegates.Num() > 0; }

	UE_API void RegisterIsClassPathAllowedDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate);
	UE_API void UnregisterIsClassPathAllowedDelegate(const FName OwnerName);
	UE_API bool IsClassPathAllowed(const FTopLevelAssetPath& InClassPath) const;
	bool HasClassPathFiltering() const { return IsClassPathAllowedDelegates.Num() > 0; }

	UE_API void RegisterIsClassAllowedOnPinDelegate(const FName OwnerName, FOnIsClassAllowed Delegate);
	UE_API void UnregisterIsClassAllowedOnPinDelegate(const FName OwnerName);
	UE_API bool IsClassAllowedOnPin(const UClass* InClass) const;
	bool HasClassOnPinFiltering() const { return IsClassAllowedOnPinDelegates.Num() > 0; }

	UE_API void RegisterIsClassPathAllowedOnPinDelegate(const FName OwnerName, FOnIsClassPathAllowed Delegate);
	UE_API void UnregisterIsClassPathAllowedOnPinDelegate(const FName OwnerName);
	UE_API bool IsClassPathAllowedOnPin(const FTopLevelAssetPath& InClassPath) const;
	bool HasClassPathOnPinFiltering() const { return IsClassPathAllowedOnPinDelegates.Num() > 0; }

	/** Delegates called to determine whether a function is allowed for a given Blueprint asset */
	UE_API void RegisterIsFunctionAllowedForAssetDelegate(const FName OwnerName, FOnIsFunctionAllowedForAsset Delegate);
	UE_API void UnregisterIsFunctionAllowedForAssetDelegate(const FName OwnerName);

	UE_API bool IsFunctionAllowed(const UBlueprint* InBlueprint, const FName InFunctionName) const;

	UE_API bool IsFunctionAllowedForAsset(const UBlueprint* InBlueprint, const FName InFunctionName) const;

	static UE_API bool IsFunctionAllowedForPermissions(const UBlueprint* InBlueprint, const FName FunctionName, const FPathPermissionList& InFunctionPermissions);
	
private:
	/** All function permissions */
	FPathPermissionList FunctionPermissions;

	/** All struct permissions */
	FPathPermissionList StructPermissions;

	/** All enum permissions */
	FPathPermissionList EnumPermissions;

	/** All pin category permissions */
	FNamePermissionList PinCategoryPermissions;

	/** Delegates called to determine whether a class type is allowed to be displayed */
	TMap<FName, FOnIsClassAllowed> IsClassAllowedDelegates;
	TMap<FName, FOnIsClassPathAllowed> IsClassPathAllowedDelegates;

	/** Delegates called to determine whether a class type is allowed to be displayed on a pin */
	TMap<FName, FOnIsClassAllowed> IsClassAllowedOnPinDelegates;
	TMap<FName, FOnIsClassPathAllowed> IsClassPathAllowedOnPinDelegates;

	/** Delegates called to determine whether a function is allowed for a given Blueprint asset */
	TMap<FName, FOnIsFunctionAllowedForAsset> IsFunctionAllowedForAssetDelegates;

protected:
	//~ Begin UObject Interface
	UE_API virtual void PostInitProperties();
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
	
	UE_API void OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldName);
	
	UE_API void OnAssetRemoved(UObject* Object);
};

#undef UE_API
