// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"

class FBlueprintActionFilter;
struct FBlueprintActionInfo;
struct FAssetBlueprintGraphActions;

class FBlueprintGraphModule : public IModuleInterface
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FActionMenuRejectionTest, FBlueprintActionFilter const&, FBlueprintActionInfo&);
	/**
	 * Provides access to a list of additional rejection tests that will be  
	 * utilized when constructing Blueprint context/action menus. Helpful to 
	 * game modules, so they can inject their own custom filtering (and hide 
	 * options that may be undesired for their specific project).
	 *
	 * These rejection tests will be ran after all the standard filter tests, 
	 * and in the order that they are in this array.
	 * 
	 * @return An array of additional rejection tests for FBlueprintActionFilter.
	 */
	TArray<FActionMenuRejectionTest>& GetExtendedActionMenuFilters() { return ExtendedMenuFilters; }

	virtual void ShutdownModule() override;
	BLUEPRINTGRAPH_API void RegisterGraphAction(const UClass* AssetType, TUniquePtr<FAssetBlueprintGraphActions> Action);
	BLUEPRINTGRAPH_API void UnregisterGraphAction(const UClass* AssetType);
	const FAssetBlueprintGraphActions* GetAssetBlueprintGraphActions(const UClass* AssetType) const;

private:
	/** Customizations for specific assets that want to customize their interactions with the Blueprint Graph window */
	TMap<const UClass*, TUniquePtr< FAssetBlueprintGraphActions>> AssetBlueprintGraphActions;

	TArray<FActionMenuRejectionTest> ExtendedMenuFilters;
};
