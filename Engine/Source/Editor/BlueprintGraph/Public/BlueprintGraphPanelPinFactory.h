// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EdGraphUtilities.h"
#include "Templates/SharedPointer.h"

#define UE_API BLUEPRINTGRAPH_API

class SGraphPin;

class FBlueprintGraphPanelPinFactory: public FGraphPanelPinFactory
{
	UE_API virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;
};

#undef UE_API
