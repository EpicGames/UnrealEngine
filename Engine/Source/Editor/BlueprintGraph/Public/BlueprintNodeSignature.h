// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraphNode;

struct FBlueprintNodeSignature
{
public:
	FBlueprintNodeSignature() {}
	UE_API FBlueprintNodeSignature(FString const& UserString);
	UE_API FBlueprintNodeSignature(TSubclassOf<UEdGraphNode> NodeClass);

	/**
	 * 
	 * 
	 * @param  NodeClass	
	 * @return 
	 */
	UE_API void SetNodeClass(TSubclassOf<UEdGraphNode> NodeClass);

	/**
	 * 
	 * 
	 * @param  SignatureObj	
	 * @return 
	 */
	UE_API void AddSubObject(FFieldVariant SignatureObj);

	/**
	 * 
	 * 
	 * @param  Value	
	 * @return 
	 */
	UE_API void AddKeyValue(FString const& KeyValue);

	/**
	 * 
	 * 
	 * @param  SignatureKey	
	 * @param  Value	
	 * @return 
	 */
	UE_API void AddNamedValue(FName SignatureKey, FString const& Value);

	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API bool IsValid() const;

	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API FString const& ToString() const;

	/**
	*
	*
	* @return
	*/
	UE_API FGuid const& AsGuid() const;

private:
	/**
	 * 
	 * 
	 * @return 
	 */
	UE_API void MarkDirty();

private:
	typedef TMap<FName, FString> FSignatureSet;
	FSignatureSet SignatureSet;

	mutable FGuid   CachedSignatureGuid;
	mutable FString CachedSignatureString;
};

#undef UE_API
