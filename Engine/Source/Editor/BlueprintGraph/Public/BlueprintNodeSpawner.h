// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintActionFilter.h"
#include "BlueprintNodeBinder.h"
#include "BlueprintNodeSignature.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTemplate.h"
#include "Textures/SlateIcon.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BlueprintNodeSpawner.generated.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraph;
class UEdGraphNode;

/*******************************************************************************
 * FBlueprintActionUiSpec
 ******************************************************************************/

struct FBlueprintActionUiSpec
{
	FBlueprintActionUiSpec()
		: IconTint(FLinearColor::White)
	{}

	FBlueprintActionUiSpec(FBlueprintActionUiSpec&& Other)
	{
		Move(Other);
	}

	FBlueprintActionUiSpec& operator=(FBlueprintActionUiSpec&& Other)
	{
		if (this != &Other)
		{
			Move(Other);
		}
		return *this;
	}

	FBlueprintActionUiSpec(const FBlueprintActionUiSpec& Other)
	{
		Copy(Other);
	}

	FBlueprintActionUiSpec& operator=(const FBlueprintActionUiSpec& Other)
	{
		if (this != &Other)
		{
			Copy(Other);
		}
		return *this;
	}

	~FBlueprintActionUiSpec() = default;

	FText   MenuName;
	FText   Category;
	FText   Tooltip;
	FText	Keywords;
	FSlateIcon Icon;
	FLinearColor IconTint;
	FString DocLink;
	FString DocExcerptTag;

private:
	void Move(FBlueprintActionUiSpec& Other)
	{
		MenuName = MoveTemp(Other.MenuName);
		Category = MoveTemp(Other.Category);
		Tooltip = MoveTemp(Other.Tooltip);
		Keywords = MoveTemp(Other.Keywords);
		Icon = MoveTemp(Other.Icon);
		IconTint = MoveTemp(Other.IconTint);
		DocLink = MoveTemp(Other.DocLink);
		DocExcerptTag = MoveTemp(Other.DocExcerptTag);
	}

	void Copy(const FBlueprintActionUiSpec& Other)
	{
		MenuName = Other.MenuName;
		Category = Other.Category;
		Tooltip = Other.Tooltip;
		Keywords = Other.Keywords;
		Icon = Other.Icon;
		IconTint = Other.IconTint;
		DocLink = Other.DocLink;
		DocExcerptTag = Other.DocExcerptTag;
	}
};

/*******************************************************************************
 * UBlueprintNodeSpawner
 ******************************************************************************/

/**
 * Intended to be wrapped and used by FBlueprintActionMenuItem. Rather than 
 * sub-classing the menu item, we choose to subclass this instead (for 
 * different node types). That way, we get the type inference that comes with  
 * UObjects (and we don't have to continuously compare identification strings). 
 */
UCLASS(MinimalAPI, Transient)
class UBlueprintNodeSpawner : public UObject, public IBlueprintNodeBinder
{
	GENERATED_UCLASS_BODY()
	DECLARE_DELEGATE_TwoParams(FCustomizeNodeDelegate, UEdGraphNode*, bool);
	DECLARE_DELEGATE_ThreeParams(FUiSpecOverrideDelegate, FBlueprintActionContext const&, FBindingSet const&, FBlueprintActionUiSpec*);

public:
	/**
	 * Creates a new UBlueprintNodeSpawner for the specified node class. Sets
	 * the allocated spawner's NodeClass and CustomizeNodeDelegate fields from
	 * the supplied parameters.
	 *
	 * @param  NodeClass			The node type that you want the spawner to spawn.
	 * @param  Outer				Optional outer for the new spawner (if left null, the transient package will be used).
	 * @param  PostSpawnDelegate    A delegate to perform specialized node setup post-spawn.
	 * @return A newly allocated instance of this class.
	 */
	static UE_API UBlueprintNodeSpawner* Create(TSubclassOf<UEdGraphNode> const NodeClass, UObject* Outer = nullptr, FCustomizeNodeDelegate PostSpawnDelegate = FCustomizeNodeDelegate());
	
	/**
	 * Templatized version of the above Create() method (where we specify the 
	 * spawner's node class through the template argument).
	 *
	 * @param  Outer				Optional outer for the new spawner (if left null, the transient package will be used).
	 * @param  PostSpawnDelegate    A delegate to perform specialized node setup post-spawn.
	 * @return A newly allocated instance of this class.
	 */
	template<class NodeType>
	static UBlueprintNodeSpawner* Create(UObject* Outer = nullptr, FCustomizeNodeDelegate PostSpawnDelegate = FCustomizeNodeDelegate());
	
public:
	/** Holds the node type that this spawner will instantiate. */
	UPROPERTY()
	TSubclassOf<UEdGraphNode> NodeClass;

	/** Defines how this spawner is presented in the ui */
	FBlueprintActionUiSpec  DefaultMenuSignature;

	/** A delegate to perform specialized node setup post-spawn (so you don't have to sub-class this for every node type). */
	FCustomizeNodeDelegate  CustomizeNodeDelegate;

	/** Provides a way to override DefaultMenuSignature based off blueprint/graph/menu context */
	FUiSpecOverrideDelegate DynamicUiSignatureGetter;

	// UObject interface
	UE_API virtual void BeginDestroy() override;
	// End UObject interface

	/**
	 * Not required, but intended to passively help speed up menu building 
	 * operations. Will cache a node-template (via GetTemplateNode), along with  
	 * any expensive text strings, to avoid constructing them all on demand.
	 */
	UE_API virtual void Prime();

	/**
	 * Takes the FBlueprintActionUiSpec that this was spawned with and attempts 
	 * to fill in any missing fields (by polling a template node).
	 * 
	 * @param  TargetGraph	Optional context for helping spawn the template node. 
	 * @return This spawner's default ui spec.
	 */
	UE_API FBlueprintActionUiSpec const& PrimeDefaultUiSpec(UEdGraph* TargetGraph = nullptr) const;

	/**
	 * We want to be able to compare spawners, and have a signature that is 
	 * rebuildable on subsequent runs. So, what makes each spawner unique is the 
	 * type of node that it spawns, and any fields the node would be initialized 
	 * with; that is what this returns.
	 * 
	 * @return A set of object-paths/names that distinguish this spawner from others.
	 */
	UE_API virtual FBlueprintNodeSignature GetSpawnerSignature() const;

	/**
	 * Takes the default FBlueprintActionUiSpec and modifies it dynamically to 
	 * accommodate the current context.
	 * 
	 * @param  Context		Context that this spawner will be presented under.
	 * @param  Bindings		Bindings that will be applied to the node post-spawn.
	 * @return A ui spec, detailing how to present this action in the menu.
	 */
	UE_API virtual FBlueprintActionUiSpec GetUiSpec(FBlueprintActionContext const& Context, FBindingSet const& Bindings) const;

	/**
	 * Takes care of spawning a node for the specified graph. Looks to see if 
	 * the supplied graph is transient, and if so, spawns a NOT fully formed
	 * node (intended for template use).
	 * 
	 * This function is intended to be overridden; sub-classes may return
	 * a pre-existing node, instead of a newly allocated one (for cases where
	 * only one instance of the node type can exist). Callers should check for 
	 * this case upon use.
	 * 
	 * @param  ParentGraph	The graph you want the node spawned into.
	 * @param  Bindings		
	 * @param  Location     Where you want the new node positioned in the graph.
	 * @return Null if it failed to spawn a node, otherwise a newly spawned node or possibly one that already existed.
	 */
	UE_API virtual UEdGraphNode* Invoke(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location) const;

	/**
	 * Retrieves a cached template for the node that this is set to spawn. Will
	 * NOT spawn one if it is not already cached.
	 *
	 * @return The cached template-node (if one already exists for this spawner).
	 */
	UE_API UEdGraphNode* GetCachedTemplateNode() const;

	/**
	 * Retrieves a cached template for the node that this is set to spawn. Will
	 * instantiate a new template if one didn't previously exist. If the 
	 * template-node is not compatible with any of our cached UEdGraph outers, 
	 * then we use TargetGraph as a model to create one that will work.
	 *
	 * @param  TargetGraph    Optional param that defines a compatible graph outer (used as an achetype if we don't have a compatible outer on hand).
	 * @param  Bindings    	  Objects to bind to the template node
	 * @return Should return a new/cached template-node (but could be null, or some pre-existing node... depends on the sub-class's Invoke() method).
	 */
	UE_API UEdGraphNode* GetTemplateNode(UEdGraph* TargetGraph = nullptr, FBindingSet const& Bindings = FBindingSet()) const;

	/**
	 * Removes the spawner's cached template node (if it has one), meaning that
	 * the next GetTemplateNode() call will spawn a new one (and that calls to 
	 * GetCachedTemplateNode() will return null).
	 */
	UE_API void ClearCachedTemplateNode() const;

	/**
	 * Determine if the node of this spawner's template's type should be filtered in the actions menu
	 */
	UE_API virtual bool IsTemplateNodeFilteredOut(FBlueprintActionFilter const& Filter) const;

	// IBlueprintNodeBinder interface
	virtual bool IsBindingCompatible(FBindingObject BindingCandidate) const override { return false; }
	virtual bool CanBindMultipleObjects() const override { return false; }
protected:
	virtual bool BindToNode(UEdGraphNode* Node, FBindingObject Binding) const override { return false; }
	// End IBlueprintNodeBinder interface

	/**
	 * Protected SpawnNode() that let's sub-classes specify their own post-spawn
	 * delegate and node class.
	 * 
	 * @param  NodeClass			The type of node you want spawned.
	 * @param  ParentGraph			The graph you want the node spawned into.
	 * @param  Bindings				The bindings to apply to the node (post spawn).
	 * @param  Location				Where you want the new node positioned in the graph.
	 * @param  PostSpawnDelegate	A delegate to run after spawning the node, but prior to allocating the node's pins.
	 * @return Null if it failed to spawn a node (if NodeClass is null), otherwise a newly spawned node.
	 */
	template <class NodeType>
	NodeType* SpawnNode(TSubclassOf<UEdGraphNode> NodeClass, UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location, FCustomizeNodeDelegate PostSpawnDelegate) const;

	/**
	 * Simplified version of the other SpawnNode(), that just let's sub-classes 
	 * specify their own post-spawn delegate (the node class is comes directly 
	 * from the template parameter).
	 * 
	 * @param  ParentGraph			The graph you want the node spawned into.
	 * @param  Bindings				The bindings to apply to the node (post spawn).
	 * @param  Location				Where you want the new node positioned in the graph.
	 * @param  PostSpawnDelegate	A delegate to run after spawning the node, but prior to allocating the node's pins.
	 * @return Null if it failed to spawn a node (if NodeClass is null), otherwise a newly spawned node.
	 */
	template <class NodeType>
	NodeType* SpawnNode(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location, FCustomizeNodeDelegate PostSpawnDelegate) const;

private:
	/**
	 * Does the actual node spawning. Creates a new node (of the specified type),
	 * sets the node's position, calls PostSpawnDelegate on the new node, and
	 * finally applies any bindings that were passed to it.
	 * 
	 * @param  NodeClass			The type of node you want spawned.
	 * @param  ParentGraph			The graph you want the node spawned into.
	 * @param  Bindings				The bindings to apply to the node (post spawn).
	 * @param  Location				Where you want the new node positioned in the graph.
	 * @param  PostSpawnDelegate	A delegate to run after spawning the node, but prior to allocating the node's pins.
	 * @return Null if it failed to spawn a node (if NodeClass is null), otherwise a newly spawned node.
	 */
	UE_API UEdGraphNode* SpawnEdGraphNode(TSubclassOf<UEdGraphNode> NodeClass, UEdGraph* ParentGraph, const FBindingSet& Bindings, FVector2D Location, FCustomizeNodeDelegate PostSpawnDelegate) const;
};

/*******************************************************************************
 * Templatized UBlueprintNodeSpawner Implementation
 ******************************************************************************/

//------------------------------------------------------------------------------
template<class NodeType>
UBlueprintNodeSpawner* UBlueprintNodeSpawner::Create(UObject* Outer, FCustomizeNodeDelegate PostSpawnDelegate)
{
	return Create(NodeType::StaticClass(), Outer, PostSpawnDelegate);
}

//------------------------------------------------------------------------------
template <class NodeType>
NodeType* UBlueprintNodeSpawner::SpawnNode(TSubclassOf<UEdGraphNode> InNodeClass, UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location, FCustomizeNodeDelegate PostSpawnDelegate) const
{
	return CastChecked<NodeType>(SpawnEdGraphNode(InNodeClass, ParentGraph, Bindings, Location, PostSpawnDelegate));
}

//------------------------------------------------------------------------------
template <class NodeType>
NodeType* UBlueprintNodeSpawner::SpawnNode(UEdGraph* ParentGraph, FBindingSet const& Bindings, FVector2D const Location, FCustomizeNodeDelegate PostSpawnDelegate) const
{
	return SpawnNode<NodeType>(NodeType::StaticClass(), ParentGraph, Bindings, Location, PostSpawnDelegate);
}

#undef UE_API
