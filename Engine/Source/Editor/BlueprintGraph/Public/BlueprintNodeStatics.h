// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompilerMisc.h"

#define UE_API BLUEPRINTGRAPH_API

class UEdGraphPin;
class UFunction;
class UK2Node;
template <typename FuncType> class TFunctionRef;

class FBlueprintNodeStatics
{
public:

	static UE_API UEdGraphPin* CreateSelfPin(UK2Node* Node, const UFunction* Function);

	static UE_API bool CreateParameterPinsForFunction(UK2Node* Node, const UFunction* Function, TFunctionRef<void(UEdGraphPin* /*Pin*/)> PostParameterPinCreatedCallback);
};

#undef UE_API
