// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintNodeSpawner.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "UObject/GCObject.h"

#define UE_API BLUEPRINTGRAPH_API

class FReferenceCollector;
class UBlueprint;
class UBlueprintNodeSpawner;
class UEdGraph;
class UEdGraphNode;

/**
 * Serves as a centralized data-store for all UBlueprintNodeSpawner node-
 * templates. Implemented this way (rather than internal to UBlueprintNodeSpawner) 
 * since node-templates require a UEdGraph/UBlueprint outer chain. Instead of 
 * instantiating a bunch of graphs/blueprints per UBlueprintNodeSpawner, we'd 
 * rather have a small centralized set here.
 */
class FBlueprintNodeTemplateCache : public FGCObject
{
public:
	/** */
	UE_API FBlueprintNodeTemplateCache();

	/**
	 * Retrieves a cached template associated with the supplied spawner. Will
	 * instantiate a new template if one didn't already exist. If the
	 * template-node is not compatible with any cached UEdGraph outer, then
	 * we use TargetGraph as a model to create one that will work.
	 * 
	 * @param  NodeSpawner	Acts as the key for the template lookup; takes care of spawning the template-node if one didn't already exist.
	 * @param  TargetGraph	Optional param that defines a compatible graph outer (used as an achetype if we don't have a compatible outer on hand).
	 * @return Should return a new/cached template-node (but could be null, or some pre-existing node... depends on the sub-class's Invoke() method).
	 */
	UE_API UEdGraphNode* GetNodeTemplate(UBlueprintNodeSpawner const* NodeSpawner, UEdGraph* TargetGraph = nullptr);

	/**
	 * Retrieves a cached template associated with the supplied spawner. Does 
	 * NOT attempt to allocate one if it doesn't exist.
	 * 
	 * @param  NodeSpawner	The spawner you want a template node for.
	 * @param  ENoInit Signifies to use this function over the other (mutating) GetNodeTemplate().
	 * @return Should return the cached template-node (if one exists, otherwise false).
	 */
	UE_API UEdGraphNode* GetNodeTemplate(UBlueprintNodeSpawner const* NodeSpawner, ENoInit) const;

	/**
	 * Wipes any nodes that were cached on behalf of the specified spawner 
	 * (should be called when NodeSpawner is destroyed, in case 
	 * GetNodeTemplate() was called for it).
	 * 
	 * @param  NodeSpawner	The spawner we want cached node-templates cleared for.
	 */
	UE_API void ClearCachedTemplate(UBlueprintNodeSpawner const* NodeSpawner);

	/**
	 * Utility method to help external systems identify if a graph they have 
	 * belongs here, to the FBlueprintNodeTemplateCache system.
	 * 
	 * @param  ParentGraph	The graph you want to check.
	 * @return True if the graph was allocated by a FBlueprintNodeTemplateCache (to house template nodes).
	 */
	static UE_API bool IsTemplateOuter(UEdGraph* ParentGraph);

	/**
	 * Approximates the current memory footprint of the entire cache 
	 * (instantiated UObject sizes + allocated container space).
	 * 
	 * @return The approximated total (in bytes) that this cache has allocated.
	 */
	UE_API int64 GetEstimateCacheSize() const;

	/**
	 * External systems can make changes that alter the memory footprint of the
	 * cache (like calling AllocateDefaultPins), and since we don't recalculate
	 * the cache's size every frame sometimes we need to update the internal 
	 * estimate.
	 * 
	 * @return The new approximated total (in bytes) that this cache has allocated.
	 */
	UE_API int64 RecalculateCacheSize();

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	// End FGCObject interface

private:
	/**
	 * Caches the supplied blueprint, so that it may be reused as an outer for
	 * template nodes (certain nodes types assume they'll have a graph outer, 
	 * with a blueprint outer beyond that, so we cannot just spawn nodes into 
	 * the transient package).
	 * 
	 * @param  Blueprint	The blueprint you wish to store for reuse.
	 * @return True if the blueprint was successfully cached (the cache could be full, and therefore this could fail).
	 */
	UE_API bool CacheBlueprintOuter(UBlueprint* Blueprint);

	/**
	 * Attempts to cache the supplied node, and associates it with the specified 
	 * spawner (so that we can remove it later if it is no longer needed).
	 * 
	 * @param  NodeSpawner	Acts as the key for the given template node.
	 * @param  NewNode		The template node you want stored.
	 * @return True if the node was successfully cached (the cache could be full, and therefore this could fail).
	 */
	UE_API bool CacheTemplateNode(UBlueprintNodeSpawner const* NodeSpawner, UEdGraphNode* NewNode);

private:
	/** 
	 * Unfortunately, we cannot nest template-nodes in the transient package. 
	 * Certain nodes operate on the assumption that they have a UEdGraph outer, 
	 * while a certain subset expect the graph to have a UBlueprint outer. This
	 * means we cannot spawn templates without a blueprint/graph to add them to.
	 * 
	 * This array holds intermediate blueprints that we use to parent the 
	 * template-nodes. Ideally we only need a small handful that are compatible 
	 * with all nodes.
	 */
	TArray<TObjectPtr<UBlueprint>> TemplateOuters;

	/** */
	TMap<UBlueprintNodeSpawner const*, TObjectPtr<UEdGraphNode>> NodeTemplateCache;


	/** 
	 * It can be costly to tally back up the estimated cache size every time an
	 * entry is added, so we keep this approximate tally of memory allocated for
	 * UObjects (owned by this system).
	 */
	int64 ApproximateObjectMem;
};

#undef UE_API
