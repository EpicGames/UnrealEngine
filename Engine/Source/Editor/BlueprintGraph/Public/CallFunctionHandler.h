// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "K2Node_CallFunction.h"
#include "KismetCompilerMisc.h"
#include "Templates/Casts.h"
#include "UObject/NameTypes.h"

class FKismetCompilerContext;
class UClass;
class UEdGraphPin;
class UFunction;
struct FBPTerminal;
struct FBlueprintCompiledStatement;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_CallFunction

class FKCHandler_CallFunction : public FNodeHandlingFunctor
{
public:
	FKCHandler_CallFunction(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	/**
	 * Searches for the function referenced by a graph node in the CallingContext class's list of functions,
	 * validates that the wiring matches up correctly, and creates an execution statement.
	 */
	void CreateFunctionCallStatement(FKismetFunctionContext& Context, UEdGraphNode* Node, UEdGraphPin* SelfPin);

private:
	// Get the name of the function to call from the node
	virtual FName GetFunctionNameFromNode(UEdGraphNode* Node) const;
	UClass* GetCallingContext(FKismetFunctionContext& Context, UEdGraphNode* Node);
	UClass* GetTrueCallingClass(FKismetFunctionContext& Context, UEdGraphPin* SelfPin);

public:

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual UFunction* FindFunction(FKismetFunctionContext& Context, UEdGraphNode* Node);
	virtual void Transform(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void CheckIfFunctionIsCallable(UFunction* Function, FKismetFunctionContext& Context, UEdGraphNode* Node);
	virtual void AdditionalCompiledStatementHandling(FKismetFunctionContext& Context, UEdGraphNode* Node, FBlueprintCompiledStatement& Statement) {}

private:
	TMap<UEdGraphPin*, FBPTerminal*> InterfaceTermMap;
};
