// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompilerMisc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UEdGraphPin;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_EventEntry

class FKCHandler_EventEntry : public FNodeHandlingFunctor
{
public:
	FKCHandler_EventEntry(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
};
