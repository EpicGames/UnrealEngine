// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "KismetCompilerMisc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UEdGraphPin;
class UFunction;
class UK2Node_MathExpression;
struct FBPTerminal;
struct FBlueprintCompiledStatement;
struct FKismetFunctionContext;

class FKCHandler_MathExpression : public FNodeHandlingFunctor
{
	FBlueprintCompiledStatement* GenerateFunctionRPN(UEdGraphNode* CurrentNode
		, FKismetFunctionContext& Context
		, UK2Node_MathExpression& MENode
		, FBPTerminal* ResultTerm
		, TMap<UEdGraphPin*, UEdGraphPin*>& InnerToOuterInput);

public:
	FKCHandler_MathExpression(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{}

	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* InNode) override;
	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;

	static bool CanBeCalledByMathExpression(const UFunction* Function);
};
