// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompilerMisc.h"

class FKismetCompilerContext;
class UEdGraphNode;
class UEdGraphPin;
struct FKismetFunctionContext;

//////////////////////////////////////////////////////////////////////////
// FKCHandler_VariableSet

class FKCHandler_VariableSet : public FNodeHandlingFunctor
{
public:
	FKCHandler_VariableSet(FKismetCompilerContext& InCompilerContext)
		: FNodeHandlingFunctor(InCompilerContext)
	{
	}

	virtual void RegisterNet(FKismetFunctionContext& Context, UEdGraphPin* Net) override;
	virtual void RegisterNets(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	void InnerAssignment(FKismetFunctionContext& Context, UEdGraphNode* Node, UEdGraphPin* VariablePin, UEdGraphPin* ValuePin);
	void GenerateAssigments(FKismetFunctionContext& Context, UEdGraphNode* Node);
	virtual void Compile(FKismetFunctionContext& Context, UEdGraphNode* Node) override;
	virtual void Transform(FKismetFunctionContext& Context, UEdGraphNode* Node) override;

protected:

	// Used for implicit casting.
	// Some nodes need to use the variable pin when performing a lookup in the implicit cast table.
	virtual bool UsesVariablePinAsKey() const { return false; }
};
